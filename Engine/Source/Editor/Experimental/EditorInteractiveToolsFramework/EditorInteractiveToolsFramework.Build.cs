// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class EditorInteractiveToolsFramework : ModuleRules
{
	public EditorInteractiveToolsFramework(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"InteractiveToolsFramework",
				"TypedElementFramework", 
				"UnrealEd",
				// ... add other public dependencies that you statically link with here ...
			}
			);			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"RenderCore",
				"Slate",
                "SlateCore",
                "InputCore",
				"EditorFramework",
                "ContentBrowser",
                "ApplicationCore",
				"Json",
				"JsonUtilities",
				"MeshDescription",
				"StaticMeshDescription",
                "EditorSubsystem",
                "PropertyEditor",
                "ToolWidgets",
                "TypedElementRuntime",
                "AnimationCore",
                "ViewportSnapping", // For ISnappingPolicy support
                "WorkspaceMenuStructure",
                // ... add private dependencies that you statically link with here ...
			}
            );
	}
}
