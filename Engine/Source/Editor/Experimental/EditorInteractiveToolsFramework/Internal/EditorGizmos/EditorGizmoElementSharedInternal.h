// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/ViewBasedTransformAdjusters.h"
#include "FrameTypes.h"
#include "Math/Vector.h"
#include "Misc/AxisDisplayInfo.h"
#include "ToolContextInterfaces.h"
#include "UObject/WeakInterfacePtr.h"

class ITransformGizmoSource;
class UGizmoElementBox;
class UGizmoViewContext;

namespace UE::Editor::InteractiveToolsFramework
{
	class IPlaneProvider;
}

namespace UE::Editor::InteractiveToolsFramework::Internal
{
	/** Adjuster to apply various scale adjustments provided by external sources. */
	class FTransformGizmoScaleAdjuster : public GizmoRenderingUtil::IViewBasedTransformAdjuster
	{
	public:
		explicit FTransformGizmoScaleAdjuster(const IToolsContextQueriesAPI* InQueriesAPI, const TWeakInterfacePtr<ITransformGizmoSource>& InTransformGizmoSource);

		virtual FTransform GetAdjustedComponentToWorld(const GizmoRenderingUtil::ISceneViewInterface& InView, const FTransform& InCurrentComponentToWorld) override;

	private:
		const IToolsContextQueriesAPI* QueriesAPI;
		TWeakInterfacePtr<ITransformGizmoSource> TransformGizmoSource;
	};

	template <EAxis::Type Axis>
	constexpr FVector GetAxisVector(const EAxisList::Type InAxisCoordinateSystem)
	{
		return FVector::ZeroVector;
	}

	template <>
	constexpr FVector GetAxisVector<EAxis::X>(const EAxisList::Type InAxisCoordinateSystem)
	{
		return FVector::XAxisVector;
	}

	template <>
	inline FVector GetAxisVector<EAxis::Y>(const EAxisList::Type InAxisCoordinateSystem)
	{
		return InAxisCoordinateSystem == EAxisList::LeftUpForward
			? -FVector::YAxisVector
			: FVector::YAxisVector;
	}

	template <>
	constexpr FVector GetAxisVector<EAxis::Z>(const EAxisList::Type InAxisCoordinateSystem)
	{
		return FVector::ZAxisVector;
	}

	template <EAxis::Type Axis>
	constexpr FVector GetAxisVector()
	{
		return GetAxisVector<Axis>(AxisDisplayInfo::GetAxisDisplayCoordinateSystem());
	}

	inline FVector GetAxisVector(EAxis::Type InAxis)
	{
		switch (InAxis)
		{
		case EAxis::X:
			return GetAxisVector<EAxis::X>();

		case EAxis::Y:
			return GetAxisVector<EAxis::Y>();

		case EAxis::Z:
			return GetAxisVector<EAxis::Z>();

		case EAxis::None:
		default:
			return FVector::ZeroVector;
		}
	}
	
	inline bool IsAxisSingular(const EAxisList::Type InAxisList)
	{
		if (InAxisList == EAxisList::X || InAxisList == EAxisList::Y || InAxisList == EAxisList::Z)
		{
			return true;
		}

		if (InAxisList == EAxisList::Left || InAxisList == EAxisList::Forward || InAxisList == EAxisList::Up)
		{
			return true;
		}

		if (InAxisList == EAxisList::Screen)
		{
			return true;
		}

		return false;
	}
	
	inline bool IsAxisPlanar(const EAxisList::Type InAxisList)
	{
		if (InAxisList == EAxisList::XY || InAxisList == EAxisList::XZ || InAxisList == EAxisList::YZ)
		{
			return true;
		}

		if (InAxisList == EAxisList::LU || InAxisList == EAxisList::LF || InAxisList == EAxisList::UF)
		{
			return true;
		}

		return false;
	}

	/**
	 * Compute Forward, Up, and Side vectors for the given axis.
	 * Vectors are returned so that they're aligned with the "positive" direction for that axis, depending on the provided AxisCoordinateSystem.
	 * @note: Not for use with an FFrame, which requires a specific orientation. The result may be invalid if used to construct one.
	 */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void GetSignedAxisBasis(EAxisList::Type InAxis, Math::TVector<RealType>& OutForward, Math::TVector<RealType>& OutUp, Math::TVector<RealType>& OutSide, const EAxisList::Type InAxisCoordinateSystem);

	/**
	 * Compute Forward, Up, and Side vectors for the given axis.
	 * Vectors are returned so that they're aligned with the "positive" direction for that axis, depending on the provided AxisCoordinateSystem.
	 * @note: Not for use with an FFrame, which requires a specific orientation. The result may be invalid if used to construct one.
	 */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void GetSignedAxisBasis(EAxisList::Type InAxis, Math::TVector<RealType>& OutForward, Math::TVector<RealType>& OutUp, Math::TVector<RealType>& OutSide);

	/**
	 * Compute Forward, Up, and Side vectors for the given axis.
	 * Vectors are returned so that when viewed along the axis (for example, top-down for Z),
	 * 'Side' points left and 'Up' points upward in that view. This can result in a negative
	 * Up vector in world space. The orientation is selected to provide a consistent basis
	 * for plane and other calculations that rely on a stable axis orientation relative to the input normal.
	 */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void GetAxisBasis(EAxisList::Type InAxis, Math::TVector<RealType>& OutForward, Math::TVector<RealType>& OutUp, Math::TVector<RealType>& OutSide, const EAxisList::Type InAxisCoordinateSystem);

	/**
	 * Compute Forward, Up, and Side vectors for the given axis.
	 * Vectors are returned so that when viewed along the axis (for example, top-down for Z),
	 * 'Side' points left and 'Up' points upward in that view. This can result in a negative
	 * Up vector in world space. The orientation is selected to provide a consistent basis
	 * for plane and other calculations that rely on a stable axis orientation relative to the input normal.
	 */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void GetAxisBasis(EAxisList::Type InAxis, Math::TVector<RealType>& OutForward, Math::TVector<RealType>& OutUp, Math::TVector<RealType>& OutSide);

	/** Returns an index (0-2) from the given AxisList. The AxisList should specify a single value. */
	int32 GetAxisIndex(const EAxisList::Type InAxisList);

	/** Utility to create a screen-aligned plane from the given ViewContext and world origin. */
	Geometry::FFrame3d MakeScreenAlignedPlane(const FVector& InPlaneOrigin, const UGizmoViewContext* InViewContext);

	/** Utility to create a plane for the given transform and coordinate space. */
	Geometry::FFrame3d MakeTransformedPlane(
		const FTransform& InTransform,
		const EToolContextCoordinateSystem InCoordinateSystem,
		const FVector& InForwardDirection,
		const FVector& InUpDirection,
		const FVector& InSideDirection);

	/** Utility to create a plane for the given transform, coordinate space and axis list. */
	Geometry::FFrame3d MakeTransformedPlaneForAxisList(
		const FTransform& InTransform,
		const UGizmoViewContext* InViewContext,
		const EToolContextCoordinateSystem InCoordinateSystem,
		const EAxisList::Type InAxisList,
		const TFunctionRef<const UGizmoElementBox*(const EAxisList::Type InAxisList)>& InGetPlanarElementFunc,
		const TFunctionRef<const IPlaneProvider*(const EAxisList::Type InAxisList)>& InGetAxisPlaneProvider);

	/**
	 * Modifies the given plane so that it, as much as possible, faces the camera while remaining aligned with the original.
	 * OutDirectionToRemove can be used to constrain derived movement to a single axis.
	 */
	Geometry::FFrame3d ModifyPlaneForView(
		const Geometry::FFrame3d& InPlane,
		const UGizmoViewContext* InViewContext,
		FVector& OutDirectionToRemove);

	/** 
	 * Returns a corrective sign for the given axis such that a positive angle results in a clockwise rotation, relative to that axis' plane. 
	 * To visualize the need for this, add a positive angle to each axis while is ortho mode for that axis and not how X and Y are negated.
	 * Take care to only apply this once, and only to angles. A quat applied to a rotation corrects for this (so in effect, a positive angle in a quat is inverted when applied to X, Y).
	 */
	int8 GetClockwiseAngleSignForAxis(const EAxis::Type InAxis);
}
