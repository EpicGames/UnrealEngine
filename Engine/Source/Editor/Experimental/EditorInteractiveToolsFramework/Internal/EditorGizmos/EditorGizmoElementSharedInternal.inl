// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorGizmos/EditorGizmoElementSharedInternal.h"
#include "Math/Vector.h"
#include "Misc/AxisDisplayInfo.h"

class ITransformGizmoSource;
class UGizmoElementBox;
class UGizmoViewContext;

namespace UE::Editor::InteractiveToolsFramework
{
	class IPlaneProvider;
}

namespace UE::Editor::InteractiveToolsFramework::Internal
{
	inline EAxisList::Type GetPlaneNormalAxis(EAxisList::Type InPlaneAxis)
	{
		if (InPlaneAxis == EAxisList::YZ)
		{
			return EAxisList::X;
		}

		if (InPlaneAxis == EAxisList::XZ)
		{
			return EAxisList::Y;
		}

		if (InPlaneAxis == EAxisList::XY)
		{
			return EAxisList::Z;
		}

		if (InPlaneAxis == EAxisList::LU)
		{
			return EAxisList::Forward;
		}

		if (InPlaneAxis == EAxisList::UF)
		{
			return EAxisList::Left;
		}

		if (InPlaneAxis == EAxisList::LF)
		{
			return EAxisList::Up;
		}

		ensure(false);

		return EAxisList::None;
	}

	/** Get Forward, Up, Side vectors for the given axis. */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void GetSignedAxisBasis(EAxisList::Type InAxis, Math::TVector<RealType>& OutForward, Math::TVector<RealType>& OutUp, Math::TVector<RealType>& OutSide, const EAxisList::Type InAxisCoordinateSystem)
	{
		// Only Y is affected by the coordinate system
		const FVector ModifiedY = FVector::YAxisVector * (InAxisCoordinateSystem == EAxisList::LeftUpForward ? -1 : 1);

		switch (InAxis)
		{
		case EAxisList::X:
		case EAxisList::Forward:
			OutForward = Math::TVector<RealType>::XAxisVector;
			OutUp = Math::TVector<RealType>::ZAxisVector;
			OutSide = ModifiedY;
			break;

		case EAxisList::Y:
		case EAxisList::Left:
			OutForward = ModifiedY;
			OutUp = Math::TVector<RealType>::ZAxisVector;
			OutSide = Math::TVector<RealType>::XAxisVector;
			break;

		case EAxisList::Z:
		case EAxisList::Up:
			OutForward = Math::TVector<RealType>::ZAxisVector;
			OutUp = Math::TVector<RealType>::XAxisVector;
			OutSide = ModifiedY;
			break;

			// Screen, Uniform etc all use the same forward/up/side vectors
		case EAxisList::Screen:
		case EAxisList::All:
		case EAxisList::LeftUpForward:
		default:
			OutForward = Math::TVector<RealType>::XAxisVector;
			OutUp = Math::TVector<RealType>::ZAxisVector;
			OutSide = ModifiedY;
			break;
		}
	}

	/** Get Forward, Up, Side vectors for the given axis. */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void GetSignedAxisBasis(EAxisList::Type InAxis, Math::TVector<RealType>& OutForward, Math::TVector<RealType>& OutUp, Math::TVector<RealType>& OutSide)
	{
		return GetSignedAxisBasis(InAxis, OutForward, OutUp, OutSide, AxisDisplayInfo::GetAxisDisplayCoordinateSystem());
	}

	/** Get Forward, Up, Side vectors for the given axis. */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void GetAxisBasis(EAxisList::Type InAxis, Math::TVector<RealType>& OutForward, Math::TVector<RealType>& OutUp, Math::TVector<RealType>& OutSide, const EAxisList::Type InAxisCoordinateSystem)
	{
		switch (InAxis)
		{
		case EAxisList::X:
		case EAxisList::Forward:
			OutForward = Math::TVector<RealType>::XAxisVector;
			OutUp = Math::TVector<RealType>::ZAxisVector;
			OutSide = Math::TVector<RealType>::YAxisVector;
			break;

		case EAxisList::Y:
		case EAxisList::Left:
			OutForward = Math::TVector<RealType>::YAxisVector;
			OutUp = Math::TVector<RealType>::ZAxisVector;
			OutSide = -Math::TVector<RealType>::XAxisVector;
			if (InAxisCoordinateSystem == EAxisList::LeftUpForward)
			{
				// Both must be inverted to create a valid FFrame
				OutForward *= -1;
				OutSide *= -1;
			}
			break;

		case EAxisList::Z:
		case EAxisList::Up:
			OutForward = Math::TVector<RealType>::ZAxisVector;
			OutUp = -Math::TVector<RealType>::XAxisVector;
			OutSide = Math::TVector<RealType>::YAxisVector;
			break;

			// Screen, Uniform etc all use the same forward/up/side vectors
		case EAxisList::Screen:
		case EAxisList::All:
		case EAxisList::LeftUpForward:
		default:
			OutForward = Math::TVector<RealType>::XAxisVector;
			OutUp = Math::TVector<RealType>::ZAxisVector;
			OutSide = Math::TVector<RealType>::YAxisVector;
			break;
		}
	}

	/** Get Forward, Up, Side vectors for the given axis. */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void GetAxisBasis(EAxisList::Type InAxis, Math::TVector<RealType>& OutForward, Math::TVector<RealType>& OutUp, Math::TVector<RealType>& OutSide)
	{
		return GetAxisBasis(InAxis, OutForward, OutUp, OutSide, AxisDisplayInfo::GetAxisDisplayCoordinateSystem());
	}
}
