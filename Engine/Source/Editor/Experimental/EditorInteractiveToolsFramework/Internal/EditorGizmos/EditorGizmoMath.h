// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "CircleTypes.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UGizmoViewContext;

namespace UE::Editor::GizmoMath
{
	/** Pushes the given rectangle to touch the outside of the given circle. */
	UE_API FBox2d PlaceRectOutsideCircle(const Geometry::FCircle2d& InCircle, const FBox2d& InRect);

	/**
	 * Clips the given line within the give box (2D).
	 * Returns true if clipped, where the start and end clipped parameters (normalized distance along line) are output.
	 * To get positions, use: LineStart + LineDelta * ClippedStart (or End)
	 */
	UE_API bool ClipLineToBox(
		const FVector2d& InLineStart,
		const FVector2d& InLineEnd,
		const FBox2d& InBox,
		FVector2d::FReal& OutClippedStart,
		FVector2d::FReal& OutClippedEnd);
	
	/** Compute (World) Plane -> Screen matrix. Use inverse for Screen -> Plane. */
	UE_API bool ComputePlaneToScreenMatrix(
		const UGizmoViewContext* InView,
		const UE::Geometry::FFrame3d& InPlane,
		Geometry::FMatrix3d& OutPlaneToScreen);
	
	/** 
	 * Compute a point on the given plane (from screen).
	* Note that the OutPoint will be in a correct location but is not a ray/plane intersection.
	 */
	UE_API bool ComputeProjectedPointOnPlaneFromScreen(
		const UGizmoViewContext* InView,
		const FVector2d& InScreenPoint,
		const UE::Geometry::FFrame3d& InPlane,
		FVector& OutPoint);

	/** 
	 * Compute a signed angle in the given plane (from screen). 
	 * Note that the OutPoint will be in a correct location but is not a ray/plane intersection.
	 */
	UE_API bool ComputeAngleInPlaneFromScreen(
		const UGizmoViewContext* InView,
		const FVector2d& InScreenPoint,
		const UE::Geometry::FFrame3d& InPlane,
		FVector& OutPoint,
		double& OutAngle);

	/** 
	 * Compute a point on the given plane from screen.
	 * This will compute a correct ray/plane intersection when applicable.
	 */
	UE_API bool ComputePointOnPlaneFromScreen(
		const UGizmoViewContext* InView,
		const FRay& InRay,
		const FVector2d& InScreenPoint,
		const UE::Geometry::FFrame3d& InPlane,
		FVector& OutPoint);

	UE_API bool ComputePointOnCircleFromScreen(
		const UGizmoViewContext* InView,
		const FVector2d& InScreenPoint,
		const double& InCircleRadius,
		const UE::Geometry::FFrame3d& InPlane,
		FVector& OutPointOnCircle);

	/** Compute the positive sign of a plane axis for a given view (which may differ to the plane orientation).
	 *  Used to determine the direction of manipulation relative to viewport drag.
	 *  Optionally provide the default sign to fallback to if the sign cannot be determined.
	 */
	UE_API float ComputePositivePlaneSignForView(
		const UGizmoViewContext* InView,
		const UE::Geometry::FFrame3d& InPlane,
		const float InDefaultSign = 1.0f);

	/** Convert three bools to an EAxisList::Type */
	UE_API EAxisList::Type GetAxisListFromBools(const bool bInEnableX, const bool bInEnableY, const bool bInEnableZ);

	/** Convert an EAxisList::Type to three bools indicating which axes are active.
	 * Returns true if all three axes are active, false otherwise.
	 * Based on FComponentElementLevelEditorViewportInteractionCustomization::ValidateScale::CheckActiveAxes
	 */
	[[maybe_unused]] UE_API bool GetBoolsFromAxisList(const EAxisList::Type InAxisList, bool bOutActiveAxis[3]);

	/** Return a Vector to be used as a multiplier for per-axis values. */
	UE_API FVector GetAxisMultiplier(const EAxisList::Type InAxisList, const FVector& InScreenSpaceMultiplier = FVector(0.0f, 1.0f, 1.0f));

	/** 
	 * Return a Vector representing the positive sign for each axis, depending on the provided coordinate system. 
	 * @example EAxisList::LeftUpForward will return FVector(1, -1, 1)
	 */
	UE_API FVector GetAxisCoordinateSystemMultiplier(const EAxisList::Type InAxisCoordinateSystem);

	/** 
	 * Return a Vector representing the positive sign for each axis, depending on the current coordinate system. 
	 * @example EAxisList::LeftUpForward will return FVector(1, -1, 1)
	 */
	UE_API FVector GetAxisCoordinateSystemMultiplier();
}

#undef UE_API
