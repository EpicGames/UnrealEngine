// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EditorGizmos/TransformGizmo.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

namespace UE::Editor::InteractiveToolsFramework
{
	/** Friended class to access UTransformGizmo internals, used externally. */
	struct FTransformGizmoAccessor
	{
		UE_API ETransformGizmoPartIdentifier GetLastHitPart(const UTransformGizmo& InGizmo) const;

		UE_API bool IsPartSelected(const UTransformGizmo& InGizmo, const ETransformGizmoPartIdentifier InPartId) const;

		UE_API FTransform GetGizmoTransform(const UTransformGizmo& InGizmo) const;
		UE_API UGizmoElementHitMultiTarget* GetHitTarget(const UTransformGizmo& InGizmo) const;
		UE_API EGizmoTransformMode GetCurrentMode(const UTransformGizmo& InGizmo) const;

		UE_API FVector2D GetScreenProjectedDirection(const UTransformGizmo& InGizmo, const FVector& InLocalAxis) const;
	};
}

#undef UE_API
