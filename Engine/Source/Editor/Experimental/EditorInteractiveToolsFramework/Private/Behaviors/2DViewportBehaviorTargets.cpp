// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/2DViewportBehaviorTargets.h"
#include "EditorViewportClient.h"

FEditor2DScrollBehaviorTarget::FEditor2DScrollBehaviorTarget(FEditorViewportClient* ViewportClientIn)
	: ViewportClient(ViewportClientIn)
{
}

FInputRayHit FEditor2DScrollBehaviorTarget::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	if (ViewportClient->IsOrtho())
	{
		return FInputRayHit(TNumericLimits<float>::Max()); // bHit is true. Depth is max to lose the standard tiebreaker
	}
	else
	{
		// Verify that ray is facing the proper direction
		if (PressPos.WorldRay.Direction.Z * PressPos.WorldRay.Origin.Z < 0)
		{
			return FInputRayHit(static_cast<float>(-PressPos.WorldRay.Origin.Z / PressPos.WorldRay.Direction.Z));
		}
	}
	return FInputRayHit();
}

void FEditor2DScrollBehaviorTarget::OnClickPress(const FInputDeviceRay& PressPos)
{
	if (ViewportClient->IsOrtho())
	{
		DragStart = FVector3d(PressPos.ScreenPosition.X, PressPos.ScreenPosition.Y, 0.0);
		OriginalCameraLocation = ViewportClient->GetViewLocation();
	}
	else
	{
		if (ensure(PressPos.WorldRay.Direction.Z * PressPos.WorldRay.Origin.Z < 0))
		{
			// Intersect with XY plane
			double T = -PressPos.WorldRay.Origin.Z / PressPos.WorldRay.Direction.Z;
			DragStart = FVector3d(
				PressPos.WorldRay.Origin.X + T * PressPos.WorldRay.Direction.X,
				PressPos.WorldRay.Origin.Y + T * PressPos.WorldRay.Direction.Y,
				0);

			OriginalCameraLocation = (FVector3d)ViewportClient->GetViewLocation();
		}
	}
}

void FEditor2DScrollBehaviorTarget::OnClickDrag(const FInputDeviceRay& DragPos)
{
	if (ViewportClient->IsPerspective() || ViewportClient->GetViewportType() == LVT_OrthoFreelook)
	{
		if (ensure(DragPos.WorldRay.Direction.Z * DragPos.WorldRay.Origin.Z < 0))
		{
			// Intersect a ray starting from the original position and using the new
			// ray direction. I.e., pretend the camera is not moving.

			double T = -OriginalCameraLocation.Z / DragPos.WorldRay.Direction.Z;
			FVector3d DragEnd = FVector3d(
				OriginalCameraLocation.X + T * DragPos.WorldRay.Direction.X,
				OriginalCameraLocation.Y + T * DragPos.WorldRay.Direction.Y,
				0);

			// We want to make it look like we are sliding the plane such that DragStart
			// ends up on DragEnd. For that, our camera will be moving the opposite direction.
			FVector3d CameraDisplacement = DragStart - DragEnd;
			check(CameraDisplacement.Z == 0);
			ViewportClient->SetViewLocation((FVector)(OriginalCameraLocation + CameraDisplacement));
		}
	}
	else
	{
		const float UnitsPerPixel = ViewportClient->GetOrthoUnitsPerPixel(ViewportClient->Viewport) * ViewportClient->GetDPIScale();
		const double DX = DragPos.ScreenPosition.X - DragStart.X;
		const double DY = DragPos.ScreenPosition.Y - DragStart.Y;

		FVector CameraDisplacement(0.0, 0.0, 0.0);
		switch (ViewportClient->GetViewportType())
		{
		case LVT_OrthoXY:
			CameraDisplacement = FVector(-DY * UnitsPerPixel, DX * UnitsPerPixel, 0.0);
			break;
		case LVT_OrthoXZ:
			CameraDisplacement = FVector(-DX * UnitsPerPixel, 0.0, DY * UnitsPerPixel);
			break;
		case LVT_OrthoYZ:
			CameraDisplacement = FVector(0.0, -DX * UnitsPerPixel, DY * UnitsPerPixel);
			break;
		case LVT_OrthoNegativeXY:
			CameraDisplacement = FVector(-DY * UnitsPerPixel, -DX * UnitsPerPixel, 0.0);
			break;
		case LVT_OrthoNegativeXZ:
			CameraDisplacement = FVector(DX * UnitsPerPixel, 0.0, DY * UnitsPerPixel);
			break;
		case LVT_OrthoNegativeYZ:
			CameraDisplacement = FVector(0.0, DX * UnitsPerPixel, DY * UnitsPerPixel);
			break;
		default:
			checkf(false, TEXT("Unexpected viewport type in FEditor2DScrollBehaviorTarget"));
			break;
		}

		ViewportClient->SetViewLocation(OriginalCameraLocation + CameraDisplacement);
	}
}

void FEditor2DScrollBehaviorTarget::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
}

void FEditor2DScrollBehaviorTarget::OnTerminateDragSequence()
{
}



// FEditor2DMouseWheelZoomBehaviorTarget

FEditor2DMouseWheelZoomBehaviorTarget::FEditor2DMouseWheelZoomBehaviorTarget(FEditorViewportClient* ViewportClientIn)
	: ViewportClient(ViewportClientIn)
{
	SetZoomAmount(DEFAULT_ZOOM_AMOUNT);
}

FInputRayHit FEditor2DMouseWheelZoomBehaviorTarget::ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos)
{
	// Always allowed to zoom with mouse wheel
	FInputRayHit ToReturn;
	ToReturn.bHit = true;
	return ToReturn;
}

void FEditor2DMouseWheelZoomBehaviorTarget::OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos)
{
	const FVector OriginalLocation = ViewportClient->GetViewLocation();
	// TODO: These behaviors should support other planes, not just XY. Using Abs(Z) allows us to at least position the camera on either size of the XY plane.
	const double TToPlane = -OriginalLocation.Z / CurrentPos.WorldRay.Direction.Z;
	const FVector NewLocation = OriginalLocation + (ZoomInProportion * TToPlane * CurrentPos.WorldRay.Direction);
	const double AbsZ = FMath::Abs(NewLocation.Z);

	ViewportClient->OverrideFarClipPlane(static_cast<float>(AbsZ - CameraFarPlaneWorldZ));
	ViewportClient->OverrideNearClipPlane(static_cast<float>(AbsZ * (1.0-CameraNearPlaneProportionZ)));

	// Don't zoom in so far that the XY plane lies in front of our near clipping plane, or else everything
	// will suddenly disappear.
	if (AbsZ > ViewportClient->GetNearClipPlane() && AbsZ > ZoomInLimit)
	{
		ViewportClient->SetViewLocation(NewLocation);
	}
}

void FEditor2DMouseWheelZoomBehaviorTarget::OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos)
{
	const FVector OriginalLocation = ViewportClient->GetViewLocation();
	// TODO: These behaviors should support other planes, not just XY. Using Abs(Z) allows us to at least position the camera on either size of the XY plane.
	const double TToPlane = -OriginalLocation.Z / CurrentPos.WorldRay.Direction.Z;
	const FVector NewLocation = OriginalLocation - (ZoomOutProportion * TToPlane * CurrentPos.WorldRay.Direction);
	const double AbsZ = FMath::Abs(NewLocation.Z);

	ViewportClient->OverrideFarClipPlane(static_cast<float>(AbsZ - CameraFarPlaneWorldZ));
	ViewportClient->OverrideNearClipPlane(static_cast<float>(AbsZ * (1.0 - CameraNearPlaneProportionZ)));

	if (AbsZ < ZoomOutLimit)
	{
		ViewportClient->SetViewLocation(NewLocation);
	}
}

void FEditor2DMouseWheelZoomBehaviorTarget::SetZoomAmount(double PercentZoomIn)
{
	check(PercentZoomIn < 100 && PercentZoomIn >= 0);

	ZoomInProportion = PercentZoomIn / 100;

	// Set the zoom out proportion such that (1 + ZoomOutProportion)(1 - ZoomInProportion) = 1
	// so that zooming in and then zooming out will return to the same zoom level.
	ZoomOutProportion = ZoomInProportion / (1 - ZoomInProportion);
}

void FEditor2DMouseWheelZoomBehaviorTarget::SetZoomLimits(double ZoomInLimitIn, double ZoomOutLimitIn)
{
	ZoomInLimit = ZoomInLimitIn;
	ZoomOutLimit = ZoomOutLimitIn;
}

void FEditor2DMouseWheelZoomBehaviorTarget::SetCameraFarPlaneWorldZ(double CameraFarPlaneWorldZIn)
{
	CameraFarPlaneWorldZ = CameraFarPlaneWorldZIn;
}

void FEditor2DMouseWheelZoomBehaviorTarget::SetCameraNearPlaneProportionZ(double CameraNearPlaneProportionZIn)
{
	CameraNearPlaneProportionZ = CameraNearPlaneProportionZIn;
}