// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/MultiButtonClickDragBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiButtonClickDragBehavior)

void UMultiButtonClickDragBehavior::Initialize()
{
	this->Initialize(this);
}

FInputCaptureRequest UMultiButtonClickDragBehavior::WantsCapture(const FInputDeviceState& InInput)
{
	bInClickDrag = false;	// should never be true here, but weird things can happen w/ focus

	if (IsAnyButtonDown(InInput) && (ModifierCheckFunc == nullptr || ModifierCheckFunc(InInput)) )
	{
		const FInputRayHit HitResult = CanBeginClickDragSequence(GetDeviceRay(InInput));
		if (HitResult.bHit)
		{
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.HitDepth);
		}
	}
	return FInputCaptureRequest::Ignore();
}


FInputCaptureUpdate UMultiButtonClickDragBehavior::BeginCapture(const FInputDeviceState& InInput, EInputCaptureSide Side)
{
	Modifiers.UpdateModifiers(InInput, Target);

	OnStateUpdated(InInput);
	
	OnClickPressInternal(InInput, Side);
	
	bInClickDrag = true;
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}


FInputCaptureUpdate UMultiButtonClickDragBehavior::UpdateCapture(const FInputDeviceState& InInput, const FInputCaptureData& Data)
{
	if (bUpdateModifiersDuringDrag)
	{
		Modifiers.UpdateModifiers(InInput, Target);
	}

	// We have to check the device before going further because we get passed captures from 
	// keyboard for modifier key press/releases, and those don't have valid ray data.
	if ((GetSupportedDevices() & InInput.InputDevice) == EInputDevices::None)
	{
		return FInputCaptureUpdate::Continue();
	}

	if (!IsAnyButtonDown(InInput)) 
	{
		OnClickReleaseInternal(InInput, Data);
		bInClickDrag = false;
		return FInputCaptureUpdate::End();
	}

	if (DidAnyButtonChangeState(InInput))
    {
		OnStateUpdated(InInput);
    }
	
	OnClickDragInternal(InInput, Data);
	return FInputCaptureUpdate::Continue();
}

FInputRayHit UMultiButtonClickDragBehavior::CanBeginClickDragSequence(const FInputDeviceRay& InPressPos)
{
	return CanBeginClickDragFunc(InPressPos);
}

void UMultiButtonClickDragBehavior::OnClickPress(const FInputDeviceRay& InPressPos)
{
	OnClickPressFunc(InPressPos);
}

void UMultiButtonClickDragBehavior::OnClickDrag(const FInputDeviceRay& InDragPos)
{
	OnClickDragFunc(InDragPos);
}

void UMultiButtonClickDragBehavior::OnClickRelease(const FInputDeviceRay& InReleasePos)
{
	OnClickReleaseFunc(InReleasePos);
}

void UMultiButtonClickDragBehavior::OnTerminateDragSequence()
{
	OnTerminateFunc();
}

void UMultiButtonClickDragBehavior::EnableButton(const FKey& InButton)
{
	const int32 Index = (InButton == EKeys::LeftMouseButton) ? 0 :
						(InButton == EKeys::MiddleMouseButton) ? 1 :
						(InButton == EKeys::RightMouseButton) ? 2 : INDEX_NONE;
	if (ensure(Index != INDEX_NONE))
	{
		CapturedInputs[Index] = 1;
	}
}

void UMultiButtonClickDragBehavior::DisableButton(const FKey& InButton)
{
	const int32 Index = (InButton == EKeys::LeftMouseButton) ? 0 :
						(InButton == EKeys::MiddleMouseButton) ? 1 :
						(InButton == EKeys::RightMouseButton) ? 2 : INDEX_NONE;
	if (ensure(Index != INDEX_NONE))
	{
		CapturedInputs[Index] = 0;
	}
}

bool UMultiButtonClickDragBehavior::HandlesLeftMouseButton() const
{
	return CapturedInputs[0] == 1;	
}

bool UMultiButtonClickDragBehavior::HandlesMiddleMouseButton() const
{
	return CapturedInputs[1] == 1;
}

bool UMultiButtonClickDragBehavior::HandlesRightMouseButton() const
{
	return CapturedInputs[2] == 1;
}

bool UMultiButtonClickDragBehavior::IsAnyButtonDown(const FInputDeviceState& InInput)
{
	if (InInput.IsFromDevice(EInputDevices::Mouse)) 
	{
		ActiveDevice = EInputDevices::Mouse;

		const FDeviceButtonState& Left = InInput.Mouse.Left;
		const FDeviceButtonState& Middle = InInput.Mouse.Middle;
		const FDeviceButtonState& Right = InInput.Mouse.Right;
		
		const bool bLeftDown = HandlesLeftMouseButton() ? Left.bDown : false;
		const bool bMiddleDown = HandlesMiddleMouseButton() ? Middle.bDown : false;
		const bool bRightDown = HandlesRightMouseButton() ? Right.bDown : false;
		
		return bLeftDown || bMiddleDown || bRightDown;
	}

	return false;
}

bool UMultiButtonClickDragBehavior::DidAnyButtonChangeState(const FInputDeviceState& InInput) const
{
	if (InInput.IsFromDevice(EInputDevices::Mouse))
	{
		const FDeviceButtonState& Left = InInput.Mouse.Left;
		const FDeviceButtonState& Middle = InInput.Mouse.Middle;
		const FDeviceButtonState& Right = InInput.Mouse.Right;

		const bool bLeftChanged = HandlesLeftMouseButton() ? Left.bPressed || Left.bReleased : false;
		const bool bMiddleChanged = HandlesMiddleMouseButton() ? Middle.bPressed || Middle.bReleased : false;
		const bool bRightChanged = HandlesRightMouseButton() ? Right.bPressed || Right.bReleased : false;

		return bLeftChanged || bMiddleChanged || bRightChanged;
	}
	return false;
}
