// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/SingleClickAndDragBehavior.h"

void USingleClickAndDragBehavior::Initialize(ISingleClickAndDragBehaviorTarget* InTarget)
{
	if (!ensure(InTarget))
	{
		return;
	}

	Target = InTarget;
}

FInputCapturePriority USingleClickAndDragBehavior::GetPriority()
{
	if (!bIsBeyondDragDistance && PendingDragPriority.IsSet())
	{
		return PendingDragPriority.GetValue();
	}
	return Super::GetPriority();
}

FInputCaptureRequest USingleClickAndDragBehavior::WantsCapture(const FInputDeviceState& InInputState)
{
	if (!Target)
	{
		return FInputCaptureRequest::Ignore();
	}

	// WantsCapture can be called by CollectWantsCapture even while this behavior already owns capture
	// (e.g. InputRouter::HandleCapturedMouseInput calls CollectWantsCapture for modifier updates).
	// When we already have capture, skip all logic to avoid resetting InitialMouseDownRay.
	if (bHasActiveCapture)
	{
		return FInputCaptureRequest::Ignore();
	}

	if (IsPressed(InInputState) && (ModifierCheckFunc == nullptr || ModifierCheckFunc(InInputState)))
	{
		const FInputDeviceRay DeviceRay = GetDeviceRay(InInputState);
		const FInputRayHit HitResult = Target->CanBeginSingleClickAndDragSequence(DeviceRay);

		if (HitResult.bHit)
		{
			if (!InitialMouseDownRay.IsSet())
			{
				InitialMouseDownRay = DeviceRay;
				bIsBeyondDragDistance = false;
			}
			else
			{
				UpdateDragDistance(InInputState);
			}
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.HitDepth);
		}
	}
	else if (InitialMouseDownRay.IsSet() && IsDown(InInputState) && (ModifierCheckFunc == nullptr || ModifierCheckFunc(InInputState)))
	{
		const FInputRayHit HitResult = Target->CanBeginSingleClickAndDragSequence(GetDeviceRay(InInputState));
		if (HitResult.bHit)
		{
			UpdateDragDistance(InInputState);
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.HitDepth);
		}
	}
	
	if (InitialMouseDownRay.IsSet())
	{
		InitialMouseDownRay.Reset();
	}
	
	return FInputCaptureRequest::Ignore();
}

FInputCaptureUpdate USingleClickAndDragBehavior::BeginCapture(const FInputDeviceState& InInputState, EInputCaptureSide InCaptureSide)
{
	bHasActiveCapture = true;

	if (Target)
	{
		Modifiers.UpdateModifiers(InInputState, Target);	
	}

	OnClickPressInternal(InInputState, InCaptureSide);
	bIsDragging = false;
	if (!InitialMouseDownRay.IsSet())
	{	
		InitialMouseDownRay = GetDeviceRay(InInputState);
	}

	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}

FInputCaptureUpdate USingleClickAndDragBehavior::UpdateCapture(const FInputDeviceState& InInputState, const FInputCaptureData& InData)
{
	// We have to check the device before going further because we get passed captures from 
	// keyboard for modifier key press/releases, and those don't have valid ray data.
	if ((GetSupportedDevices() & InInputState.InputDevice) == EInputDevices::None)
	{
		return FInputCaptureUpdate::Continue();
	}

	if (!bIsDragging)
	{
		UpdateDragDistance(InInputState);
		
		if (bIsBeyondDragDistance)
		{
			bIsDragging = true;
			OnDragStartedInternal(InInputState, InData);
			return FInputCaptureUpdate::Continue();
		}
		else if (IsReleased(InInputState))
		{
			if (bIsBeyondDragDistance)
			{
				bHasActiveCapture = false;
				InitialMouseDownRay.Reset();
				return FInputCaptureUpdate::End();
			}


			OnClickReleaseInternal(InInputState, InData);
			return FInputCaptureUpdate::End();
		}

		return FInputCaptureUpdate::Continue();
	}

	if (IsReleased(InInputState))
	{
		OnClickReleaseInternal(InInputState, InData);
		return FInputCaptureUpdate::End();
	}
	else
	{
		OnClickDragInternal(InInputState, InData);
		return FInputCaptureUpdate::Continue();
	}		
}

void USingleClickAndDragBehavior::ForceEndCapture(const FInputCaptureData& InData)
{
	bHasActiveCapture = false;
	InitialMouseDownRay.Reset();

	if (Target)
	{
		Target->OnTerminateSingleClickAndDragSequence();
	}
}

void USingleClickAndDragBehavior::UpdateDragDistance(const FInputDeviceState& InInputState)
{
	if (InitialMouseDownRay.IsSet())
	{
		const float DistanceSquared = static_cast<float>((InInputState.Mouse.Position2D - InitialMouseDownRay.GetValue().ScreenPosition).SizeSquared());
		bIsBeyondDragDistance = DistanceSquared >= (DragDistanceThreshold * DragDistanceThreshold);
	}
	else
	{
		bIsBeyondDragDistance = false;
	}
}

FInputDeviceRay USingleClickAndDragBehavior::GetDeviceRay(const FInputDeviceState& InInputDeviceState)
{
	if (InInputDeviceState.IsFromDevice(EInputDevices::Mouse))
	{
		ActiveDevice = EInputDevices::Mouse;
		const FVector2D& MousePosition = UsesUnboundedCursor() ? InInputDeviceState.Mouse.UnboundedPosition2D : InInputDeviceState.Mouse.Position2D;
		const FRay& WorldRay = UsesUnboundedCursor() ? InInputDeviceState.Mouse.UnboundedWorldRay : InInputDeviceState.Mouse.WorldRay;
		return FInputDeviceRay(WorldRay, MousePosition);
	}
	return FInputDeviceRay(FRay(FVector::ZeroVector, FVector(0, 0, 1), true));
}

bool USingleClickAndDragBehavior::UsesUnboundedCursor() const
{
	return UseUnboundedCursor.IsSet() ? UseUnboundedCursor.Get() : false;
}

void USingleClickAndDragBehavior::SetUsesUnboundedCursor(TAttribute<bool> InUsesUnboundedCursor)
{
	UseUnboundedCursor = InUsesUnboundedCursor;
}

void USingleClickAndDragBehavior::SetPendingDragPriority(TOptional<FInputCapturePriority> InPriority)
{
	PendingDragPriority = InPriority;
}

void USingleClickAndDragBehavior::OnClickPressInternal(const FInputDeviceState& InInput, EInputCaptureSide InSide)
{
	if (Target)
	{
		Target->OnClickPress(GetDeviceRay(InInput));
	}
}

void USingleClickAndDragBehavior::OnDragStartedInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData)
{
	if (Target)
	{
		Target->OnDragStart(GetDeviceRay(InInput));
	}
}

void USingleClickAndDragBehavior::OnClickDragInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData)
{
	if (Target)
	{
		Target->OnClickDrag(GetDeviceRay(InInput));
	}
}

void USingleClickAndDragBehavior::OnClickReleaseInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData)
{
	bHasActiveCapture = false;

	if (Target)
	{
		Target->OnClickRelease(GetDeviceRay(InInput), bIsDragging);
	}
	
	InitialMouseDownRay.Reset();
}

void ULocalSingleClickAndDragBehavior::Initialize(ISingleClickAndDragBehaviorTarget* InTarget)
{
	FallbackTarget = InTarget;
	USingleClickAndDragBehavior::Initialize(this);
}

FInputRayHit ULocalSingleClickAndDragBehavior::CanBeginSingleClickAndDragSequence(const FInputDeviceRay& InPressPos)
{
	if (CanBeginSingleClickAndDragSequenceFunc)
	{
		return CanBeginSingleClickAndDragSequenceFunc(InPressPos);
	}
	if (FallbackTarget)
	{
		return FallbackTarget->CanBeginSingleClickAndDragSequence(InPressPos);
	}
	return FInputRayHit();
}

void ULocalSingleClickAndDragBehavior::OnClickPress(const FInputDeviceRay& InPressPos)
{
	if (OnClickPressFunc)
	{
		OnClickPressFunc(InPressPos);
	}
	else if (FallbackTarget)
	{
		FallbackTarget->OnClickPress(InPressPos);
	}
}

void ULocalSingleClickAndDragBehavior::OnDragStart(const FInputDeviceRay& InDragPos)
{
	if (OnDragStartFunc)
	{
		OnDragStartFunc(InDragPos);
	}
	else if (FallbackTarget)
	{
		FallbackTarget->OnDragStart(InDragPos);
	}
}

void ULocalSingleClickAndDragBehavior::OnClickDrag(const FInputDeviceRay& InDragPos)
{
	if (OnClickDragFunc)
	{
		OnClickDragFunc(InDragPos);
	}
	else if (FallbackTarget)
	{
		FallbackTarget->OnClickDrag(InDragPos);
	}
}

void ULocalSingleClickAndDragBehavior::OnClickRelease(const FInputDeviceRay& InReleasePos, bool bInIsDragOperation)
{
	if (OnClickReleaseFunc)
	{
		OnClickReleaseFunc(InReleasePos, bInIsDragOperation);
	}
	else if (FallbackTarget)
	{
		FallbackTarget->OnClickRelease(InReleasePos, bInIsDragOperation);
	}
}

void ULocalSingleClickAndDragBehavior::OnTerminateSingleClickAndDragSequence()
{
	if (OnTerminateSingleClickAndDragSequenceFunc)
	{
		OnTerminateSingleClickAndDragSequenceFunc();
	}
	else if (FallbackTarget)
	{
		FallbackTarget->OnTerminateSingleClickAndDragSequence();
	}
}

