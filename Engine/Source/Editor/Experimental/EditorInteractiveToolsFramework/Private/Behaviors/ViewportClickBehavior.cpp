// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/ViewportClickBehavior.h"

#include "ViewportInteractions/ViewportInteraction.h"

void UViewportClickBehavior::Initialize(IViewportClickBehaviorTarget* InTarget)
{
	check (InTarget != nullptr);
	Target = InTarget;
	
	DefaultPriority = UE::Editor::ViewportInteractions::CLICK_PRIORITY;
}

FInputCapturePriority UViewportClickBehavior::GetPriority()
{
	if (const FInputCapturePriority* Priority = CachedCapturePriority.GetPtrOrNull())
	{
		return *Priority;
	}
	return Super::GetPriority();
}

FInputCaptureRequest UViewportClickBehavior::WantsCapture(const FInputDeviceState& InInputState)
{
	if (!Target)
	{
		return FInputCaptureRequest::Ignore();
	}

	if (!CachedCapturePriority.IsSet())
	{
		CachedCapturePriority = GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage::Start, InInputState);
	}
	else if (InInputState.IsFromDevice(EInputDevices::Mouse))
	{
		TOptional<FInputCapturePriority> NewPriority = GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage::Continue, InInputState);
		// If in a click, WantsCapture() is occuring as this behavior is defending in a capture stealing pass.
		// A fresh unset priority means that this click wants to resolve with the "ClickUp" event.
		// Retaining the previous capture priority gives the behavior a chance to win and close out in UpdateCapture().
		// A set priority always updates as any additional modifier keys needs to be tracked. 
		if (NewPriority.IsSet() || !bIsClicking)
		{
			CachedCapturePriority = NewPriority;
		}
	}
	
	if (!CachedCapturePriority.IsSet())
	{
		return FInputCaptureRequest::Ignore();
	}
	
	const FInputRayHit HitResult = Target->IsHitByClick(GetDeviceRay(InInputState));
	if (!HitResult.bHit)
	{
		CachedCapturePriority.Reset();
		return FInputCaptureRequest::Ignore();
	}
	
	return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.HitDepth);
}

FInputCaptureUpdate UViewportClickBehavior::BeginCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide)
{
	OnStateUpdatedInternal(InputState);
	OnClickDownInternal(InputState);
	bIsClicking = true;
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}

FInputCaptureUpdate UViewportClickBehavior::UpdateCapture(const FInputDeviceState& InputState, const FInputCaptureData& CaptureData)
{
	// This happens when a keyboard event is routed to the active mouse capture for updating modifier states
	// The behavior should always continue here so that subsequent mouse updates can end capture more correctly	
	if (!InputState.IsFromDevice(EInputDevices::Mouse))
	{
		return FInputCaptureUpdate::Continue();
	}
	
	if (Bindings.DidAnyBindingStateChange(InputState))
	{
		OnStateUpdatedInternal(InputState);
	}
	
	CachedCapturePriority = GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage::Continue, InputState);
	if (!CachedCapturePriority.IsSet())
	{
		OnClickUpInternal(InputState);
		bIsClicking = false;
		return FInputCaptureUpdate::End();
	}
	
	return FInputCaptureUpdate::Continue();
}

void UViewportClickBehavior::ForceEndCapture(const FInputCaptureData& CaptureData)
{
	CachedCapturePriority.Reset();
	bIsClicking = false;
	if (Target)
	{
		Target->OnForceEndCapture();
	}
}

void UViewportClickBehavior::SetBindings(const TArray<UE::Editor::ViewportInteractions::FButtonBinding>& InBindings)
{
	Bindings = InBindings;
}

FInputDeviceRay UViewportClickBehavior::GetDeviceRay(const FInputDeviceState& InInputDeviceState)
{
	if (InInputDeviceState.IsFromDevice(EInputDevices::Mouse))
	{
		const FVector2D& MousePosition = GetMousePosition(InInputDeviceState);
		const FRay& WorldRay = InInputDeviceState.Mouse.WorldRay;

		return FInputDeviceRay(WorldRay, MousePosition);
	}

	return FInputDeviceRay(FRay(FVector::ZeroVector, FVector(0, 0, 1), true));
}

const FVector2D& UViewportClickBehavior::GetMousePosition(const FInputDeviceState& InInputDeviceState) const
{
	return InInputDeviceState.Mouse.Position2D;
}

void UViewportClickBehavior::OnClickDownInternal(const FInputDeviceState& InputState)
{
	if (Target)
	{
		Target->OnClickDown(GetDeviceRay(InputState));
	}
}

void UViewportClickBehavior::OnClickUpInternal(const FInputDeviceState& InputState)
{
	if (Target)
	{
		Target->OnClickUp(GetDeviceRay(InputState));
	}
}

void UViewportClickBehavior::OnStateUpdatedInternal(const FInputDeviceState& InInputDeviceState)
{
	if (Target)
	{
		Target->OnStateUpdated(InInputDeviceState);
	}
}

TOptional<FInputCapturePriority> UViewportClickBehavior::GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage Stage, const FInputDeviceState& InputState) const
{
	const int32 Complexity = Bindings.GetBindingComplexity(Stage, InputState);
	if (Complexity <= 0)
	{
		return TOptional<FInputCapturePriority>();
	}
	
	return DefaultPriority.MakeHigher(Complexity * FInputCapturePriority::BINDING_COMPLEXITY_PRIORITY_STEP);
}

