// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/ViewportClickDragBehavior.h"

#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

namespace UE::Editor::ViewportInteractions
{
	TAutoConsoleVariable<float> DragThresholdOverride(
		TEXT("Editor.ViewportInteractions.DragThresholdOverride"),
		-1.0f,
		TEXT("Set to a >= 0 value to override all drag threshold values.")
	);
}

void UViewportClickDragBehavior::Initialize(IViewportClickDragBehaviorTarget* InTarget)
{
	check(InTarget != nullptr);
	Target = InTarget;
	
	DefaultPriority = UE::Editor::ViewportInteractions::DRAG_PRIORITY;
	DragConfirmedPriority = UE::Editor::ViewportInteractions::DRAG_CONFIRMED_PRIORITY;
}

void UViewportClickDragBehavior::SetDragConfirmedPriority(FInputCapturePriority InDragConfirmedPriority)
{
	DragConfirmedPriority = InDragConfirmedPriority;
}

FInputCapturePriority UViewportClickDragBehavior::GetPriority()
{
	if (const FInputCapturePriority* Priority = CachedCapturePriority.GetPtrOrNull())
	{
		return *Priority;
	}
	return Super::GetPriority();
}

FInputCaptureRequest UViewportClickDragBehavior::WantsCapture(const FInputDeviceState& InInputState)
{
	const FVector2D& MousePosition = GetMousePosition(InInputState);

	if (!MouseTraveledDistance.IsSet())
	{
		CachedCapturePriority = GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage::Start, InInputState);
		if (CachedCapturePriority.IsSet())
		{
			// Make sure behavior knows we are not dragging yet
			bDragging = false;

			// Traveled distance is 0.0f if mouse was just pressed
			MouseTraveledDistance = 0.0f;

			// Cache button down location
			MouseDownPosition = MousePosition;
		}
		else
		{
			return FInputCaptureRequest::Ignore();
		}
	}
	else
	{
		if (InInputState.IsFromDevice(EInputDevices::Mouse))
		{
			CachedCapturePriority = GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage::Continue, InInputState);
			if (!CachedCapturePriority.IsSet())
			{
				MouseTraveledDistance = TOptional<float>();
				return FInputCaptureRequest::Ignore();
			}
		}
		
		MouseTraveledDistance = static_cast<float>(FVector2D::Distance(MousePosition, MouseDownPosition));
	}

	const FInputRayHit HitResult = Target->CanBeginClickDragSequence(GetDeviceRay(InInputState));
	if (!HitResult.bHit)
	{
		MouseTraveledDistance = TOptional<float>();
		CachedCapturePriority.Reset();
		return FInputCaptureRequest::Ignore();
	}
	
	return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.HitDepth);
}

FInputCaptureUpdate UViewportClickDragBehavior::BeginCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide)
{
	if (Target)
	{
		Target->OnBeginCapture(GetDeviceRay(InputState));
	}

	OnStateUpdatedInternal(InputState);
	
	if (!bDragging && (!RequireDistanceThresholdOnCapture.Get(false) || HasMovedEnoughForDrag()))
	{
		bDragging = true;
		OnDragStartInternal(InputState);
	}
	
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}

FInputCaptureUpdate UViewportClickDragBehavior::UpdateCapture(const FInputDeviceState& InputState, const FInputCaptureData& CaptureData)
{
	const FVector2D& MousePosition = GetMousePosition(InputState);

	// This happens when a keyboard event is routed to the active mouse capture for updating modifier states
	// The behavior should always continue here so that subsequent mouse updates can end capture more correctly	
	if (!InputState.IsFromDevice(EInputDevices::Mouse))
	{
		return FInputCaptureUpdate::Continue();
	}
	
	if (GetBindings().DidAnyBindingStateChange(InputState))
	{
		OnStateUpdatedInternal(InputState);
	}
	
	CachedCapturePriority = GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage::Continue, InputState);
	if (!CachedCapturePriority.IsSet())
	{
		if (bDragging)
		{
			OnDragEndInternal(InputState, CaptureData);
		}
		MouseTraveledDistance = TOptional<float>();
		Target->OnEndCapture(IViewportClickDragBehaviorTarget::EEndCaptureReason::End);
		return FInputCaptureUpdate::End();
	}
	
	MouseTraveledDistance = static_cast<float>(FVector2D::Distance(MousePosition, MouseDownPosition));

	if (!bDragging && HasMovedEnoughForDrag())
	{
		bDragging = true;
		OnDragStartInternal(InputState);
	}

	if (bDragging)
	{
		OnClickDragInternal(InputState, CaptureData);
	}
	
	return FInputCaptureUpdate::Continue();
}

void UViewportClickDragBehavior::ForceEndCapture(const FInputCaptureData& CaptureData)
{
	MouseTraveledDistance = TOptional<float>();
	CachedCapturePriority.Reset();

	if (bDragging)
	{
		OnDragEndInternal(FInputDeviceState(), CaptureData);
	}
	
	Target->OnEndCapture(IViewportClickDragBehaviorTarget::EEndCaptureReason::Forced);
}

void UViewportClickDragBehavior::SetBindings(const TArray<UE::Editor::ViewportInteractions::FButtonBinding>& InBindings)
{
	Bindings = InBindings;
}

void UViewportClickDragBehavior::SetBindings(const FGetBindingsDelegate& InDelegate)
{
	GetBindingsDelegate = InDelegate;	
}

FInputDeviceRay UViewportClickDragBehavior::GetDeviceRay(const FInputDeviceState& InInputDeviceState)
{
	if (InInputDeviceState.IsFromDevice(EInputDevices::Mouse))
	{
		const FVector2D& MousePosition = GetMousePosition(InInputDeviceState);
		const FRay& WorldRay = UsesUnboundedCursor() ? InInputDeviceState.Mouse.UnboundedWorldRay : InInputDeviceState.Mouse.WorldRay;

		return FInputDeviceRay(WorldRay, MousePosition);
	}

	return FInputDeviceRay(FRay(FVector::ZeroVector, FVector(0, 0, 1), true));
}

const FVector2D& UViewportClickDragBehavior::GetMousePosition(const FInputDeviceState& InInputDeviceState) const
{
	return UsesUnboundedCursor() ? InInputDeviceState.Mouse.UnboundedPosition2D : InInputDeviceState.Mouse.Position2D;
}

bool UViewportClickDragBehavior::UsesUnboundedCursor() const
{
	return UnboundedCursor.IsSet() ? UnboundedCursor.Get() : false;
}

void UViewportClickDragBehavior::SetUsesUnboundedCursor(TAttribute<bool> InUsesUnboundedCursor)
{
	UnboundedCursor = InUsesUnboundedCursor;
}

void UViewportClickDragBehavior::SetRequireDistanceThresholdOnCapture(TAttribute<bool> InRequireDistanceThresholdOnCapture)
{
	RequireDistanceThresholdOnCapture = InRequireDistanceThresholdOnCapture; 
}

void UViewportClickDragBehavior::ResetDrag(const FInputDeviceState& InputState)
{
	if (bDragging)
	{
		OnDragEndInternal(InputState, {});
	}
	
	MouseDownPosition = GetMousePosition(InputState);
	MouseTraveledDistance = 0.0f;
}

void UViewportClickDragBehavior::OnDragStartInternal(const FInputDeviceState& InputState)
{
	Target->OnDragStart(GetDeviceRay(InputState));
}

void UViewportClickDragBehavior::OnClickDragInternal(const FInputDeviceState& InputState, const FInputCaptureData& Data)
{
	Target->OnDrag({
		.Ray = GetDeviceRay(InputState),
		.ScreenDelta = UsesUnboundedCursor() ? InputState.Mouse.UnboundedMouseDelta2D : InputState.Mouse.Delta2D
	});
}

void UViewportClickDragBehavior::OnDragEndInternal(const FInputDeviceState& InputState, const FInputCaptureData& Data)
{
	Target->OnDragEnd(GetDeviceRay(InputState));
	bDragging = false;
}

void UViewportClickDragBehavior::OnStateUpdatedInternal(const FInputDeviceState& InInputDeviceState)
{
	Target->OnStateUpdated(InInputDeviceState);
}

bool UViewportClickDragBehavior::HasMovedEnoughForDrag() const
{
	if (const float* TraveledDistance = MouseTraveledDistance.GetPtrOrNull())
	{
		const float Override = UE::Editor::ViewportInteractions::DragThresholdOverride.GetValueOnAnyThread();
		const float EffectiveThreshold = Override >= 0.0f ? Override : DragDistanceThreshold;
	
		if (*TraveledDistance > EffectiveThreshold)
		{
			return true;
		}
	}

	return false;
}

const UE::Editor::ViewportInteractions::FButtonBindings& UViewportClickDragBehavior::GetBindings() const
{
	if (GetBindingsDelegate.IsBound())
	{
		return GetBindingsDelegate.Execute();
	}
	return Bindings;
}

TOptional<FInputCapturePriority> UViewportClickDragBehavior::GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage Stage, const FInputDeviceState& InputState) const
{
	FInputCapturePriority BasePriority = DefaultPriority;
	
	if (HasMovedEnoughForDrag())
	{
		BasePriority = DragConfirmedPriority;
	}
	
	const int32 Complexity = GetBindings().GetBindingComplexity(Stage, InputState);
	if (Complexity <= 0)
	{
		return TOptional<FInputCapturePriority>();
	}
	
	return BasePriority.MakeHigher(Complexity * FInputCapturePriority::BINDING_COMPLEXITY_PRIORITY_STEP);
}
