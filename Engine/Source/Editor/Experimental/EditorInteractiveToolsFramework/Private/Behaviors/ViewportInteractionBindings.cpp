// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportInteractionBindings.h"

#include "Algo/Find.h"

namespace UE::Editor::ViewportInteractions
{

FButtonBindings::FButtonBindings(const TArray<FButtonBinding>& InBindings)
	: Bindings(InBindings)
{
	bRequiresTrigger = Algo::FindByPredicate(Bindings, [](const FButtonBinding& InBinding) { return InBinding.bTriggersStart; }) != nullptr;
}

int32 FButtonBindings::GetBindingComplexity(EInputStage Stage, const FInputDeviceState& InputDeviceState) const
{
	int32 Complexity = 0;
	bool bHasTrigger = false;

	for (const FButtonBinding& Binding : Bindings)
	{
		ProcessBinding(Stage, InputDeviceState, Binding, Complexity, bHasTrigger);
		if (Complexity == INDEX_NONE)
		{
			return INDEX_NONE;
		}
	}
	
	return (!bRequiresTrigger || bHasTrigger) ? Complexity : INDEX_NONE;
}

bool FButtonBindings::DidAnyBindingStateChange(const FInputDeviceState& InputDeviceState) const
{
	for (const FButtonBinding& Binding : Bindings)
	{
		if (TSharedPtr<FUICommandInfo> Command = Binding.Command.Pin())
		{
			for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
			{
				EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
				const FInputChord& Chord = *Command->GetActiveChord(ChordIndex);
				if (Chord.IsValidChord())
				{
					const FDeviceButtonState State = InputDeviceState.GetButtonState(Chord.Key);
					if (State.bPressed || State.bReleased)
					{
						return true;
					}
				}
			}
		}
		else
		{
			const FDeviceButtonState State = InputDeviceState.GetButtonState(Binding.Chord.Key);
			if (State.bPressed || State.bReleased)
			{
				return true;
			}
		}
	}
	
	return false;
}

void FButtonBindings::ProcessBinding(EInputStage Stage, const FInputDeviceState& InputDeviceState, const FButtonBinding& Binding, int32& OutComplexity, bool& bOutTrigger)
{
	if (TSharedPtr<FUICommandInfo> Command = Binding.Command.Pin())
	{
		for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
		{
			EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
			const FInputChord& Chord = *Command->GetActiveChord(ChordIndex);
			if (Chord.IsValidChord())
			{
				// Use temporary state, as _either_ of the bound chords could be used
				int32 ChordComplexity = 0;
				bool bHasTrigger = false;
			
				ProcessBindingChord(Stage, InputDeviceState, Binding, Chord, ChordComplexity, bHasTrigger);
				if (ChordComplexity == INDEX_NONE)
				{
					continue;
				}
				
				if (ChordComplexity > 0)
				{
					OutComplexity += ChordComplexity;
					bOutTrigger = bOutTrigger || bHasTrigger; 
					return;
				}
			}
		}
		
		// No bound chord was valid, and this binding is required, so it fails
		if (Stage == EInputStage::Start ? Binding.bRequiredToStart : Binding.bRequiredToContinue)
		{
			OutComplexity = INDEX_NONE;
		}
	}
	else
	{
		ProcessBindingChord(Stage, InputDeviceState, Binding, Binding.Chord, OutComplexity, bOutTrigger);
	}
}

void FButtonBindings::ProcessBindingChord(EInputStage Stage, const FInputDeviceState& InputDeviceState, const FButtonBinding& Binding, const FInputChord& Chord, int32& OutComplexity, bool& bOutTrigger)
{
	const FDeviceButtonState State = InputDeviceState.GetButtonState(Chord.Key);
	
	const bool bRequired = Stage == EInputStage::Start ? Binding.bRequiredToStart : Binding.bRequiredToContinue;
	
#define CHECK_REQUIRED(value) if (value) { ++OutComplexity; } else if (bRequired) { OutComplexity = INDEX_NONE; return; }

	CHECK_REQUIRED(State.bDown);
	
	if (Chord.NeedsShift())
	{
		CHECK_REQUIRED(InputDeviceState.bShiftKeyDown);
	}
	
	if (Chord.NeedsAlt())
	{
		CHECK_REQUIRED(InputDeviceState.bAltKeyDown);
	}
	
	if (Chord.NeedsControl())
	{
		CHECK_REQUIRED(InputDeviceState.bCtrlKeyDown);
	}
	
	if (Chord.NeedsCommand())
	{
		CHECK_REQUIRED(InputDeviceState.bCmdKeyDown);
	}
	
#undef CHECK_REQUIRED
	
	bOutTrigger = bOutTrigger || (Binding.bTriggersStart && (Stage == EInputStage::Start ? State.bPressed : State.bDown));
}

}
