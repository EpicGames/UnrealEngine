// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/ViewportQuickToggleInputBehavior.h"
#include "Editor.h"

void UViewportQuickToggleInputBehavior::Initialize(IQuickToggleBehaviorTarget* InTarget, const FKey& InKey)
{
	Super::Initialize(InTarget, InKey);

	QuickToggleTarget = InTarget;
	KeyPressedTime.Init(0.0, 1);
}

void UViewportQuickToggleInputBehavior::Initialize(IQuickToggleBehaviorTarget* InTarget, const TArray<FKey>& InKeys)
{
	Super::Initialize(InTarget, InKeys);

	QuickToggleTarget = InTarget;
	KeyPressedTime.Init(0.0, InKeys.Num());
}

FInputCaptureRequest UViewportQuickToggleInputBehavior::WantsCapture(const FInputDeviceState& InputState)
{
	return Super::WantsCapture(InputState);
}

bool UViewportQuickToggleInputBehavior::WantsForceEndCapture()
{
	return true;
}

void UViewportQuickToggleInputBehavior::ForceEndCapture(const FInputCaptureData& CaptureData)
{
	// Do NOT call Super — UKeyInputBehavior::ForceEndCapture would call Target->OnForceEndCapture()
	// again via the base-class path, resulting in a double call for bRequireAllKeys=false.

	if (QuickToggleTarget)
	{
		QuickToggleTarget->OnForceEndCapture();
	}

	KeyPressedTime.Init(0.0, KeyPressedTime.Num());
}

void UViewportQuickToggleInputBehavior::OnKeyPressedInternal(const FInputDeviceState& Input)
{
	Super::OnKeyPressedInternal(Input);

	const int32 KeyIndex = TargetKeys.Find(Input.Keyboard.ActiveKey.Button);
	if (KeyIndex > -1)
	{
		KeyPressedTime[KeyIndex] = FPlatformTime::Seconds();
	}
}

void UViewportQuickToggleInputBehavior::OnKeyReleasedInternal(const FInputDeviceState& Input)
{
	if (QuickToggleTarget && CanQuickToggle(Input))
	{

		FKey Key = Input.Keyboard.ActiveKey.Button;
		const int32 KeyIndex = TargetKeys.Find(Key);
		if (KeyIndex > -1)
		{
			const double TimeSinceLastPress = FPlatformTime::Seconds() - KeyPressedTime[KeyIndex];
			if (TimeSinceLastPress <= QuickToggleDelay)
			{
				QuickToggleTarget->OnQuickToggle(Key);
			}
		}
	}

	Super::OnKeyReleasedInternal(Input);
}

bool UViewportQuickToggleInputBehavior::CanQuickToggle(const FInputDeviceState& Input) const
{
	return CanQuickToggleFunc == nullptr || CanQuickToggleFunc(Input);
}

