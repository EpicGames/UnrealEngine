// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "HAL/IConsoleManager.h"

template <typename ValueType>
class TCVarToggle
{
public:
	explicit TCVarToggle(const FString& InName)
		: Name(InName)
	{
		if (IConsoleVariable* FoundCVar = IConsoleManager::Get().FindConsoleVariable(*InName, false))
		{
			FoundCVar->GetValue(Value);
			OnChangedHandle = FoundCVar->OnChangedDelegate().AddRaw(this, &TCVarToggle<ValueType>::OnChanged);
		}
	}

	~TCVarToggle()
	{
		if (IConsoleVariable* FoundCVar = IConsoleManager::Get().FindConsoleVariable(*Name, false))
		{
			FoundCVar->OnChangedDelegate().Remove(OnChangedHandle);
		}
	}

	ValueType GetValue() const { return Value; }

	void SetValue(const ValueType& InValue)
	{
		if (IConsoleVariable* FoundCVar = IConsoleManager::Get().FindConsoleVariable(*Name, false))
		{
			FoundCVar->Set(InValue);
			Value = InValue;
		}
	}

private:
	void OnChanged(IConsoleVariable* InCVar)
	{
		if (InCVar)
		{
			InCVar->GetValue(Value);
		}
	}

private:
	FDelegateHandle OnChangedHandle;
	const FString Name;
	ValueType Value;
};
