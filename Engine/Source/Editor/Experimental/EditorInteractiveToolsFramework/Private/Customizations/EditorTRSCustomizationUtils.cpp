// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTRSCustomizationUtils.h"

#include "PropertyHandle.h"
#include "PropertyPath.h"
#include "UObject/WeakObjectPtr.h"

namespace UE::Editor::GizmoSettings::Private
{
	const void* GetNestedPropertyValue(const void* InRootContainer, const TSharedPtr<FPropertyPath>& InPropertyPath)
	{
		if (!InRootContainer || !InPropertyPath.IsValid() || InPropertyPath->GetNumProperties() == 0)
		{
			return nullptr;
		}

		const void* CurrentContainer = InRootContainer;

		// Walk through the property path
		for (int32 PathIdx = 0; PathIdx < InPropertyPath->GetNumProperties(); ++PathIdx)
		{
			const FPropertyInfo& PropertyInfo = InPropertyPath->GetPropertyInfo(PathIdx);
			const FProperty* Property = PropertyInfo.Property.Get();
    
			if (!Property || !CurrentContainer)
			{
				return nullptr;
			}
    
			// Handle different property types
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				// For struct properties, get the container pointer
				CurrentContainer = Property->ContainerPtrToValuePtr<void>(CurrentContainer);
			}
			else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				// For array properties, get the array element
				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(CurrentContainer));
				const int32 Index = PropertyInfo.ArrayIndex;
        
				if (!ArrayHelper.IsValidIndex(Index))
				{
					return nullptr;
				}
        
				CurrentContainer = ArrayHelper.GetRawPtr(Index);
			}
			else
			{
				// For other property types, get the value pointer
				CurrentContainer = Property->ContainerPtrToValuePtr<void>(CurrentContainer);
			}
		}

		return CurrentContainer;
	}

	const void* GetDefaultPropertyValue(const FPropertyOuterVariant& InOuter, const TSharedPtr<FPropertyPath>& InPropertyPath, TSharedPtr<FStructOnScope>& OutDefaultStruct)
	{
		if (!InOuter.IsValid() || !InPropertyPath.IsValid())
		{
			return nullptr;
		}

		if (InOuter.Struct.IsValid())
		{
			OutDefaultStruct = MakeShared<FStructOnScope>(InOuter.Struct->GetStruct());
			return GetNestedPropertyValue(OutDefaultStruct->GetStructMemory(), InPropertyPath);
		}

		if (InOuter.Object.IsValid())
		{
			OutDefaultStruct.Reset();
			const UObject* DefaultOuterObject = InOuter.Object->StaticClass()->GetDefaultObject();
			return GetNestedPropertyValue(DefaultOuterObject, InPropertyPath);
		}

		ensureAlwaysMsgf(
			false,
			TEXT("GetDefaultPropertyValue: Property '%s' had no outers to get a default value from."), 
			*InPropertyPath->ToString());

		return nullptr;
	}

	const void* GetDefaultPropertyValue(const TSharedPtr<IPropertyHandle>& InPropertyHandle, TSharedPtr<FStructOnScope>& OutDefaultStruct)
	{
		const FPropertyOuterVariant PropertyOuter = GetPropertyOuter(InPropertyHandle);
		if (!PropertyOuter.IsValid())
		{
			return nullptr;
		}

		const TSharedPtr<FPropertyPath> PropertyPath = GetPropertyPathRelativeToOuter(InPropertyHandle, PropertyOuter);
		if (!PropertyPath.IsValid() || PropertyPath->GetNumProperties() == 0)
		{
			return nullptr;
		}

		// Get Default Value
		const void* DefaultValuePtr = GetDefaultPropertyValue(PropertyOuter, PropertyPath, OutDefaultStruct);
		if (!DefaultValuePtr)
		{
			return nullptr;
		}

		return DefaultValuePtr;
	}

	const void* GetDefaultPropertyValue(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		TSharedPtr<FStructOnScope> DefaultStruct = nullptr;
		return GetDefaultPropertyValue(InPropertyHandle, DefaultStruct);
	}

	TSharedPtr<FPropertyPath> GetPropertyPathRelativeToStructOfType(const TSharedPtr<IPropertyHandle>& InPropertyHandle, const UScriptStruct* InStructType)
	{
		return GetPropertyPathRelativeToOuter(InPropertyHandle, FPropertyOuterVariant(InStructType->GetOwnerStruct()));
	}

	TSharedPtr<FPropertyPath> GetPropertyPathRelativeToOuter(const TSharedPtr<IPropertyHandle>& InPropertyHandle, const FPropertyOuterVariant& InOuter)
	{
		const bool bIsStructOuter = InOuter.Struct.IsValid();
		const bool bIsObjectOuter = InOuter.Object.IsValid();

		if (!ensure(bIsStructOuter || bIsObjectOuter))
		{
			return nullptr;
		}

		TSharedPtr<FPropertyPath> PropertyPath = InPropertyHandle->CreateFPropertyPath();
		auto GetPropertyPathRelativeToOuterInternal = [&](const TUniqueFunction<bool(int32 InPathSegmentIndex, FProperty* InProperty)>&& InMatchFunc)
		{
			for (int32 PathSegmentIndex = 0; PathSegmentIndex < PropertyPath->GetNumProperties(); ++PathSegmentIndex)
			{
				const FPropertyInfo& PropertyInfo = PropertyPath->GetPropertyInfo(PathSegmentIndex);
				if (FProperty* PathSegmentProperty = PropertyInfo.Property.Get())
				{
					if (InMatchFunc(PathSegmentIndex, PathSegmentProperty))
					{
						// Found
						break;
					}
				}
			}

			return PropertyPath;
		};

		if (bIsStructOuter)
		{
			return GetPropertyPathRelativeToOuterInternal([&PropertyPath, OuterStructType = InOuter.Struct->GetStruct()](int32 InPathSegmentIndex, FProperty* InProperty)
			{
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
				{
					if (StructProperty->Struct == OuterStructType)
					{
						// Trim the path up until this segment
						PropertyPath = PropertyPath->TrimRoot(InPathSegmentIndex + 1);
						return true;
					}
				}

				return false;
			});
		}

		if (bIsObjectOuter)
		{
			return GetPropertyPathRelativeToOuterInternal([&PropertyPath, OuterObjectType = InOuter.Object->StaticClass()](int32 InPathSegmentIndex, FProperty* InProperty)
			{
				if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
				{
					if (ObjectProperty->PropertyClass == OuterObjectType)
					{
						// Trim the path up until this segment
						PropertyPath = PropertyPath->TrimRoot(InPathSegmentIndex + 1);
						return true;
					}
				}

				return false;
			});
		}

		return nullptr;
	}

	FPropertyOuterVariant GetPropertyOuter(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		if (!InPropertyHandle.IsValid())
		{
			return FPropertyOuterVariant();
		}

		TArray<TSharedPtr<FStructOnScope>> OuterStructs;
		InPropertyHandle->GetOuterStructs(OuterStructs);

		if (!OuterStructs.IsEmpty())
		{
			const TSharedPtr<FStructOnScope> FirstOuterStruct = OuterStructs[0];
			return FPropertyOuterVariant(FirstOuterStruct);
		}

		TArray<UObject*> OuterObjects;
		InPropertyHandle->GetOuterObjects(OuterObjects);
		if (!OuterObjects.IsEmpty())
		{
			UObject* FirstOuterObject = OuterObjects[0];
			return FPropertyOuterVariant(FirstOuterObject);
		}

		// We should never get here
		const FString PropertyPathString = FString(InPropertyHandle->GetPropertyPath());
		ensureAlwaysMsgf(
			false,
			TEXT("GetDefaultPropertyValue: Property '%s' had no valid outers."), 
			*PropertyPathString);

		return FPropertyOuterVariant();
	}
};
