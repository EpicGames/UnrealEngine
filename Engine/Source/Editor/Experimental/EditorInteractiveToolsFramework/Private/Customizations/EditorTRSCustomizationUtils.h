// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FStructOnScope;
class IPropertyHandle;
class FPropertyPath;

namespace UE::Editor::GizmoSettings::Private
{
	/** UObject or StructOnScope */
	struct FPropertyOuterVariant
	{
		FPropertyOuterVariant() = default;

		explicit FPropertyOuterVariant(UObject* InObject)
			: Object(MakeWeakObjectPtr(InObject))
			, bIsValid(InObject != nullptr)
		{
		}

		explicit FPropertyOuterVariant(const TSharedPtr<FStructOnScope>& InStruct)
			: Struct(InStruct)
			, bIsValid(InStruct.IsValid())
		{
		}

		TWeakObjectPtr<UObject> Object = nullptr;
		TSharedPtr<FStructOnScope> Struct = nullptr;

		bool IsValid() const { return bIsValid; }

	private:
		bool bIsValid = false;
	};

	const void* GetNestedPropertyValue(const void* InRootContainer, const TSharedPtr<FPropertyPath>& InPropertyPath);

	/** Get's the default value by constructing a default struct/object of the outer. */
	const void* GetDefaultPropertyValue(const TSharedPtr<IPropertyHandle>& InPropertyHandle);

	TSharedPtr<FPropertyPath> GetPropertyPathRelativeToStructOfType(const TSharedPtr<IPropertyHandle>& InPropertyHandle, const UScriptStruct* InStructType);

	TSharedPtr<FPropertyPath> GetPropertyPathRelativeToOuter(const TSharedPtr<IPropertyHandle>& InPropertyHandle, const FPropertyOuterVariant& InOuter);

	/** Gets the first valid outer. */
	FPropertyOuterVariant GetPropertyOuter(const TSharedPtr<IPropertyHandle>& InPropertyHandle);
}
