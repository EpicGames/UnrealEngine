// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailGroup.h"
#include "PropertyPath.h"
#include "TransformGizmoEditorSettings.h"

namespace UE::Editor::GizmoSettings::Private
{
	namespace GizmoSettingsLocals
	{
		const void* GetNestedPropertyValue(const void* InRootContainer, const TSharedPtr<FPropertyPath>& InPropertyPath)
		{
			if (!InRootContainer || !InPropertyPath.IsValid() || InPropertyPath->GetNumProperties() == 0)
			{
				return nullptr;
			}

			const void* CurrentContainer = InRootContainer;
    
			// Walk through the property path
			for (int32 PathIdx = 0; PathIdx < InPropertyPath->GetNumProperties(); ++PathIdx)
			{
				const FPropertyInfo& PropertyInfo = InPropertyPath->GetPropertyInfo(PathIdx);
				const FProperty* Property = PropertyInfo.Property.Get();
        
				if (!Property || !CurrentContainer)
				{
					return nullptr;
				}
        
				// Handle different property types
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					// For struct properties, get the container pointer
					CurrentContainer = Property->ContainerPtrToValuePtr<void>(CurrentContainer);
				}
				else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					// For array properties, get the array element
					FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(CurrentContainer));
					const int32 Index = PropertyInfo.ArrayIndex;
            
					if (!ArrayHelper.IsValidIndex(Index))
					{
						return nullptr;
					}
            
					CurrentContainer = ArrayHelper.GetRawPtr(Index);
				}
				else
				{
					// For other property types, get the value pointer
					CurrentContainer = Property->ContainerPtrToValuePtr<void>(CurrentContainer);
				}
			}
    
			return CurrentContainer;
		}

		TSharedPtr<FPropertyPath> GetPropertyPathRelativeToStructOfType(const TSharedPtr<IPropertyHandle>& InPropertyHandle, const UScriptStruct* InStructType)
		{
			TSharedPtr<FPropertyPath> PropertyPath = InPropertyHandle->CreateFPropertyPath();
			for (int32 PathSegmentIndex = 0; PathSegmentIndex < PropertyPath->GetNumProperties(); ++PathSegmentIndex)
			{
				const FPropertyInfo& PropertyInfo = PropertyPath->GetPropertyInfo(PathSegmentIndex);
				if (FProperty* PathSegmentProperty = PropertyInfo.Property.Get())
				{
					if (FStructProperty* StructProperty = CastField<FStructProperty>(PathSegmentProperty))
					{
						if (StructProperty->Struct == InStructType)
						{
							// Trim the path up until this segment
							PropertyPath = PropertyPath->TrimRoot(PathSegmentIndex + 1);
							break;
						}
					}
				}
			}
			return PropertyPath;
		}
	}
}

void FTransformGizmoEditorSettingsCustomizationBase::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	InDetailBuilder.GetObjectsBeingCustomized(Objects);

	if (Objects.IsEmpty())
	{
		return;
	}

	const TSharedPtr<IPropertyHandle> GizmoParametersPropertyHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, GizmosParameters));

	const TSharedPtr<IPropertyHandle> IsolatedPropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GetTargetPropertyName());
	if (!ensure(IsolatedPropertyHandle))
	{
		return;
	}

	// Hide others
	{
		TArray<FName> AllCategories;
		InDetailBuilder.GetCategoryNames(AllCategories);

		// AllCategories.Remove(IsolatedPropertyHandle->GetDefaultCategoryName());
		AllCategories.Remove(GetTargetCategoryName());

		for (const FName& CategoryName : AllCategories)
		{
			IDetailCategoryBuilder& CategoryBuilder = InDetailBuilder.EditCategory(CategoryName);
			// CategoryBuilder.SetIsEmpty(true);

			TArray<TSharedRef<IPropertyHandle>> AllCategoryProperties;
			CategoryBuilder.GetDefaultProperties(AllCategoryProperties);

			for (TSharedRef<IPropertyHandle> Property : AllCategoryProperties)
			{
				if (Property != IsolatedPropertyHandle)
				{
					InDetailBuilder.HideProperty(Property);
				}
			}
		}
	}

	IsolateCategory(InDetailBuilder, IsolatedPropertyHandle.ToSharedRef(), GetTargetCategoryName());
}

void FTransformGizmoEditorSettingsCustomizationBase::HideAllProperties(IDetailLayoutBuilder& InDetailBuilder)
{
	TArray<FName> AllCategories;
	InDetailBuilder.GetCategoryNames(AllCategories);

	for (const FName& CategoryName : AllCategories)
	{
		IDetailCategoryBuilder& CategoryBuilder = InDetailBuilder.EditCategory(CategoryName);
		CategoryBuilder.SetIsEmpty(true);

		TArray<TSharedRef<IPropertyHandle>> AllCategoryProperties;
		CategoryBuilder.GetDefaultProperties(AllCategoryProperties);
		for (TSharedRef<IPropertyHandle> Property : AllCategoryProperties)
		{
			InDetailBuilder.HideProperty(Property);
		}
	}
}

void FTransformGizmoEditorSettingsCustomizationBase::IsolateCategory(IDetailLayoutBuilder& InDetailBuilder, const TSharedRef<IPropertyHandle>& InContainerProperty, const FName InCategoryName)
{
	// Only hide the category if children will be re-added to different categories.
	// If all children share the same category as InCategoryName, HideCategory would
	// permanently suppress them since they'd be re-added to the hidden category.
	bool bAllChildrenInSameCategory = true;
	{
		uint32 NumChildren;
		if (InContainerProperty->GetNumChildren(NumChildren) == FPropertyAccess::Success)
		{
			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = InContainerProperty->GetChildHandle(ChildIndex);
				if (ChildProperty.IsValid())
				{
					const FName ChildCategory = ChildProperty->GetDefaultCategoryName();
					if (!ChildCategory.IsEqual(InCategoryName))
					{
						bAllChildrenInSameCategory = false;
						break;
					}
				}
			}
		}
	}

	if (!bAllChildrenInSameCategory)
	{
		InDetailBuilder.HideCategory(InCategoryName);
	}

	const auto ResetToDefault = FResetToDefaultOverride::Create(
		FIsResetToDefaultVisible::CreateSP(this, &FTransformGizmoEditorSettingsCustomizationBase::IsResetToDefaultVisible),
		FResetToDefaultHandler::CreateSP(this, &FTransformGizmoEditorSettingsCustomizationBase::OnResetToDefault),
		true);

	TMap<FName, TArray<IDetailGroup*>> CategoryGroups;
	auto EditGroup = [&](const FName InCatName, const FName InGroupName, const FText& InGroupText) -> IDetailGroup*
	{
		IDetailCategoryBuilder& CategoryBuilder = InDetailBuilder.EditCategory(InCatName);
		TArray<IDetailGroup*>& Groups = CategoryGroups.FindOrAdd(InCatName, TArray<IDetailGroup*>());
		if (IDetailGroup** ExistingGroup = Groups.FindByPredicate(
			[&](const IDetailGroup* InGroup)
			{
				return InGroup->GetGroupName() == InGroupName;
			}))
		{
			return *ExistingGroup; // Group already exists
		}

		IDetailGroup* Group = &CategoryBuilder.AddGroup(InGroupName, InGroupText, false, true);
		Groups.Add(Group);
		return Group;
	};

	// If this category is encountered beyond the base level/depth 0, it's promoted to its parent category.
	const FName NestedCategoryName = InCategoryName;

	TFunction<void(const TSharedPtr<IPropertyHandle>&, const int32, const FName)> AddProperty;
	AddProperty = [&](const TSharedPtr<IPropertyHandle>& InPropertyHandle, const int32 InDepth = 0, const FName InCategoryNameOverride = NAME_None) -> void
	{
		if (!InPropertyHandle.IsValid())
		{
			return;
		}

		uint32 NumChildren;
		if (InPropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success && NumChildren > 0)
		{
			const FName ParentCategoryName = InPropertyHandle->GetDefaultCategoryName();

			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = InPropertyHandle->GetChildHandle(ChildIndex);
				if (!ChildProperty.IsValid())
				{
					continue;
				}

				FName ChildCategoryName = ChildProperty->GetDefaultCategoryName();
				ChildCategoryName = ChildCategoryName.IsEqual(NestedCategoryName) ? ParentCategoryName : ChildCategoryName;

				const FName CategoryNameToUse = InCategoryNameOverride.IsNone()
					? ChildCategoryName
					: InCategoryNameOverride;

				const FName ChildGroupName = ChildCategoryName;

				if (ChildProperty->HasMetaData("ShowOnlyInnerProperties"))
				{
					AddProperty(ChildProperty, InDepth + 1, CategoryNameToUse);
					continue;
				}

				IDetailCategoryBuilder& Category = InDetailBuilder.EditCategory(CategoryNameToUse);

				if (!InCategoryNameOverride.IsNone()
					&& !ChildGroupName.IsEqual(InCategoryNameOverride))
				{
					const FText& ChildGroupText = ChildProperty->GetDefaultCategoryText();

					EditGroup(CategoryNameToUse, ChildGroupName, ChildGroupText)
						->AddPropertyRow(ChildProperty.ToSharedRef())
						.ShowPropertyButtons(true)
						.IsEnabled(IsPropertyEnabledAttribute.IsSet() ? IsPropertyEnabledAttribute : true)
						.OverrideResetToDefault(ResetToDefault);
				}
				else
				{
					Category.AddProperty(ChildProperty)
					.ShowPropertyButtons(true)
					.IsEnabled(IsPropertyEnabledAttribute.IsSet() ? IsPropertyEnabledAttribute : true)
					.OverrideResetToDefault(ResetToDefault);
				}
			}
		}
	};

	AddProperty(InContainerProperty, 0, NAME_None);
}

bool FTransformGizmoEditorSettingsCustomizationBase::IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	using namespace UE::Editor::GizmoSettings::Private::GizmoSettingsLocals;
	
	const FProperty* Property = InPropertyHandle->GetProperty();
	if (!Property)
	{
		return false;
	}

	TSharedPtr<FPropertyPath> PropertyPath = GetPropertyPathRelativeToStructOfType(InPropertyHandle, GetTargetStructType());
	if (!PropertyPath.IsValid() || PropertyPath->GetNumProperties() == 0)
	{
		return false;
	}

	// Get Property Value
	void* ValuePtr = nullptr;
	if (InPropertyHandle->GetValueData(ValuePtr) != FPropertyAccess::Success || !ValuePtr)
	{
		return false;
	}

	// Get Default Value
	const void* DefaultValuePtr = GetNestedPropertyValue(GetDefaultValue(), PropertyPath);
	if (!DefaultValuePtr)
	{
		return false;
	}

	// If the value is already set to the default, don't show the reset button
	return !Property->Identical(ValuePtr, DefaultValuePtr);
}

void FTransformGizmoEditorSettingsCustomizationBase::OnResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	using namespace UE::Editor::GizmoSettings::Private::GizmoSettingsLocals;

	const FProperty* Property = InPropertyHandle->GetProperty();
	if (!Property)
	{
		return;
	}

	TSharedPtr<FPropertyPath> PropertyPath = GetPropertyPathRelativeToStructOfType(InPropertyHandle, GetTargetStructType());
	if (!PropertyPath.IsValid() || PropertyPath->GetNumProperties() == 0)
	{
		return;
	}

	// Get Property Value
	void* ValuePtr = nullptr;
	if (InPropertyHandle->GetValueData(ValuePtr) != FPropertyAccess::Success || !ValuePtr)
	{
		return;
	}

	// Get Default Value
	const void* DefaultValuePtr = GetNestedPropertyValue(GetDefaultValue(), PropertyPath);
	if (!DefaultValuePtr)
	{
		return;
	}

	Property->CopyCompleteValue(ValuePtr, DefaultValuePtr);
	InPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	GetMutableDefault<UTransformGizmoEditorSettings>()->SaveConfig();
}
