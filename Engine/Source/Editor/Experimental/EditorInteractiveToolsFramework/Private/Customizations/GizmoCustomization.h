// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Misc/Attribute.h"

class IPropertyHandle;

/**
 * Base detail customization that isolates a single property section from UTransformGizmoEditorSettings::GizmosParameters.
 *
 * Derived classes specify which property, category, and struct type to target. The base class hides all other
 * categories and re-adds the target property's children with custom reset-to-default behavior that compares
 * against a CDO-like default value provided by the subclass.
 */
class FTransformGizmoEditorSettingsCustomizationBase
	: public IDetailCustomization
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;

protected:
	/** Hides every category and property in the detail layout. */
	void HideAllProperties(IDetailLayoutBuilder& InDetailBuilder);

	/**
	 * Hides the original category for @p InContainerProperty and re-adds its children with per-property
	 * reset-to-default overrides and optional enabled-state gating via IsPropertyEnabledAttribute.
	 */
	void IsolateCategory(IDetailLayoutBuilder& InDetailBuilder, const TSharedRef<IPropertyHandle>& InContainerProperty, const FName InCategoryName);

	/** Returns true when the property's current value differs from its default, indicating the reset arrow should be shown. */
	bool IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> InPropertyHandle) const;

	/** Copies the default value into the property and saves the config. */
	void OnResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

	/** Returns the FName of the child property within FGizmosParameters that this customization displays (e.g. "Interaction", "Style", "Debug"). */
	virtual FName GetTargetPropertyName() const = 0;

	/** Returns the detail-panel category name under which the target property's children are shown. */
	virtual FName GetTargetCategoryName() const = 0;

	/** Returns the UScriptStruct describing the target property's type, used to build relative property paths for reset-to-default comparison. */
	virtual const UScriptStruct* GetTargetStructType() const = 0;

	/** Returns a pointer to a default-constructed instance of the target struct, used as the reference for reset-to-default. */
	virtual const void* GetDefaultValue() const = 0;

	/** Optional attribute controlling whether displayed properties are interactive. Set by derived classes that gate on a CVar or other condition. */
	TAttribute<bool> IsPropertyEnabledAttribute;
};