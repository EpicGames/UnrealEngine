// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoDebugCustomization.h"

#include "TransformGizmoEditorSettings.h"

FGizmoDebugSettingsCustomization::FGizmoDebugSettingsCustomization()
{
	CVarDebugDraw = MakeShared<TCVarToggle<bool>>(TEXT("Gizmos.DebugDraw"));
	IsPropertyEnabledAttribute = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateRaw(this, &FGizmoDebugSettingsCustomization::IsPropertyEnabled));
}

FName FGizmoDebugSettingsCustomization::GetTargetPropertyName() const
{
	return GET_MEMBER_NAME_CHECKED(FGizmosParameters, Debug);
}

FName FGizmoDebugSettingsCustomization::GetTargetCategoryName() const
{
	static FLazyName CategoryName(TEXT("Debug"));
	return CategoryName;
}

const UScriptStruct* FGizmoDebugSettingsCustomization::GetTargetStructType() const
{
	static const UScriptStruct* StructType = FGizmoDebugSettings::StaticStruct();
	return StructType;
}

const void* FGizmoDebugSettingsCustomization::GetDefaultValue() const
{
	return &DefaultSettings;
}

bool FGizmoDebugSettingsCustomization::IsPropertyEnabled() const
{
	return CVarDebugDraw.IsValid() && CVarDebugDraw->GetValue();
}
