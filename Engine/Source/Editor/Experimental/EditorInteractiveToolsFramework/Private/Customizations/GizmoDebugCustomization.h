// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CVarToggle.h"
#include "EditorGizmos/EditorTRSGizmo.h"
#include "GizmoCustomization.h"
#include "IPropertyTypeCustomization.h"

/** Details panel customization for FGizmoDebugSettings, shown in the transform gizmo editor settings. Gated behind a CVar toggle for debug drawing. */
class FGizmoDebugSettingsCustomization
	: public FTransformGizmoEditorSettingsCustomizationBase
{
public:
	FGizmoDebugSettingsCustomization();

protected:
	//~ Begin FTransformGizmoEditorSettingsCustomizationBase Interface

	/** Returns the property name for the debug settings struct within the gizmo settings. */
	virtual FName GetTargetPropertyName() const override;

	/** Returns the category name under which the debug settings are displayed. */
	virtual FName GetTargetCategoryName() const override;

	/** Returns the UScriptStruct type descriptor for FGizmoDebugSettings. */
	virtual const UScriptStruct* GetTargetStructType() const override;

	/** Returns the default FGizmoDebugSettings values used for reset-to-default. */
	virtual const void* GetDefaultValue() const override;

	//~ End FTransformGizmoEditorSettingsCustomizationBase Interface

private:
	/** Returns true if the debug draw CVar is currently enabled. */
	bool IsPropertyEnabled() const;

private:
	/** Default settings instance used as the reference value for the details panel. */
	FGizmoDebugSettings DefaultSettings;

	/** CVar toggle controlling whether the debug draw settings section is visible and active. */
	TSharedPtr<TCVarToggle<bool>> CVarDebugDraw;
};