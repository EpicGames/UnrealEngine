// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoInteractionCustomization.h"

#include "DetailBuilderTypes.h"
#include "TransformGizmoEditorSettings.h"
#include "IDetailChildrenBuilder.h"
#include "IStructureDataProvider.h"

FName FTransformGizmoInteractionCustomization::GetTargetPropertyName() const
{
	return GET_MEMBER_NAME_CHECKED(FGizmosParameters, Interaction);
}

FName FTransformGizmoInteractionCustomization::GetTargetCategoryName() const
{
	static FLazyName CategoryName(TEXT("Interaction"));
	return CategoryName;
}

const UScriptStruct* FTransformGizmoInteractionCustomization::GetTargetStructType() const
{
	static const UScriptStruct* StructType = FTransformGizmoInteraction::StaticStruct();
	return StructType;
}

const void* FTransformGizmoInteractionCustomization::GetDefaultValue() const
{
	return &DefaultInteraction;
}
