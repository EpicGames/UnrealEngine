// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorGizmos/EditorTRSGizmo.h"
#include "GizmoCustomization.h"
#include "IPropertyTypeCustomization.h"

/** Details panel customization for FTransformGizmoInteraction, shown in the transform gizmo editor settings under the Interaction category. */
class FTransformGizmoInteractionCustomization
	: public FTransformGizmoEditorSettingsCustomizationBase
{
protected:
	virtual FName GetTargetPropertyName() const override;
	virtual FName GetTargetCategoryName() const override;
	virtual const UScriptStruct* GetTargetStructType() const override;
	virtual const void* GetDefaultValue() const override;

private:
	/** Default interaction settings used as the reference for reset-to-default. */
	FTransformGizmoInteraction DefaultInteraction;
};