// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoElementShared.h"
#include "DetailWidgetRow.h"
#include "EditorGizmos/EditorGizmoElementShared.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "Misc/Attribute.h"
#include "PropertyEditorUtils.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SBox.h"

class IPropertyHandle;

template <class StructType, typename ValueType = typename StructType::FValueType
	UE_REQUIRES(std::is_same_v<typename StructType::FValueType, ValueType>)>
class TGizmoPerStateValueCustomization
	: public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<TGizmoPerStateValueCustomization<StructType, ValueType>>();
	}

	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override
	{
		StructPropertyHandle = InStructPropertyHandle;

		// Resolve state value handles
		{
			DefaultPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(StructType, Default));
			HoverPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(StructType, Hover));
			InteractPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(StructType, Interact));
			SelectPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(StructType, Select));
			SubduePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(StructType, Subdue));
		}

		auto MakeStateWidget = [this](const TSharedPtr<IPropertyHandle>& InStateValueHandle) -> TSharedRef<SWidget>
		{
			const TSharedRef<SWidget> InnerValueWidget = MakeStateValueWidget(InStateValueHandle);

			constexpr float BorderPadding = 2.0f;

			FOptionalSize HeightOverride = GetHeightOverride();
			if (HeightOverride.IsSet())
			{
				HeightOverride = HeightOverride.Get() - BorderPadding;
			}

			return SNew(SBorder)
				.Padding(BorderPadding)
				.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.RoundedSolidBackground"))
				.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.InputOutline"))
				.VAlign(VAlign_Fill)
				[
					SNew(SBox)
					.Padding(0.0f)
					.MinDesiredHeight(HeightOverride)
					[
						InnerValueWidget
					]
				];
		};

		const FMargin GridSlotPadding(2.0f, 0.0f, 2.0f, 0.0f);
		constexpr EVerticalAlignment GridSlotVAlign = VAlign_Fill;

		InHeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SBox)
			.Padding(0.0f)
			.WidthOverride(500.0f)
			.HeightOverride(GetHeightOverride())
			.VAlign(VAlign_Fill)
			[
				SNew(SGridPanel)
				.FillColumn(0, 1.0f)
				.FillColumn(1, 1.0f)
				.FillColumn(2, 1.0f)
				.FillColumn(3, 1.0f)
				.FillColumn(4, 1.0f)

				+ SGridPanel::Slot(0, 0)
				.Padding(GridSlotPadding)
				.VAlign(GridSlotVAlign)
				[
					MakeStateWidget(DefaultPropertyHandle)
				]

				+ SGridPanel::Slot(1, 0)
				.Padding(GridSlotPadding)
				.VAlign(GridSlotVAlign)
				[
					MakeStateWidget(HoverPropertyHandle)
				]

				+ SGridPanel::Slot(2, 0)
				.Padding(GridSlotPadding)
				.VAlign(GridSlotVAlign)
				[
					MakeStateWidget(InteractPropertyHandle)
				]

				+ SGridPanel::Slot(3, 0)
				.Padding(GridSlotPadding)
				.VAlign(GridSlotVAlign)
				[
					MakeStateWidget(SelectPropertyHandle)
				]

				+ SGridPanel::Slot(4, 0)
				.Padding(GridSlotPadding)
				.VAlign(GridSlotVAlign)
				[
					MakeStateWidget(SubduePropertyHandle)
				]
			]
		];
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override
	{
		if (StructPropertyHandle.IsValid() && StructPropertyHandle->IsValidHandle())
		{
			uint32 NumChildren = 0;
			if (StructPropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success)
			{
				for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
				{
					TSharedPtr<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex);
					if (ChildHandle.IsValid() && ChildHandle->IsValidHandle())
					{
						InChildBuilder.AddProperty(ChildHandle.ToSharedRef());
					}
				}
			}
		}		
	}

protected:
	FOptionalSize GetHeightOverride()
	{
		return { };
	}
	
	TSharedRef<SWidget> MakeStateValueWidget(const TSharedPtr<IPropertyHandle>& InStateValueHandle)
	{
		if (InStateValueHandle.IsValid() && InStateValueHandle->IsValidHandle())
		{
			TSharedPtr<IPropertyHandle> ValueHandle = InStateValueHandle->GetChildHandle(0);
			if (ValueHandle.IsValid() && ValueHandle->IsValidHandle())
			{
				return ValueHandle->CreatePropertyValueWidget();
			}
			else
			{
				return InStateValueHandle->CreatePropertyValueWidget();
			}
		}

		return SNullWidget::NullWidget;
	}

	FSlateColor GetBorderColor(TSharedPtr<SWidget> InWidget) const
	{
		static const FSlateColor HoveredColor = FAppStyle::Get().GetSlateColor("Colors.Hover");
		static const FSlateColor DefaultColor = FAppStyle::Get().GetSlateColor("Colors.InputOutline");
		return InWidget.IsValid() && InWidget->IsHovered() ? HoveredColor : DefaultColor;
	}

private:
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	TSharedPtr<IPropertyHandle> DefaultPropertyHandle;
	TSharedPtr<IPropertyHandle> HoverPropertyHandle;
	TSharedPtr<IPropertyHandle> InteractPropertyHandle;
	TSharedPtr<IPropertyHandle> SelectPropertyHandle;
	TSharedPtr<IPropertyHandle> SubduePropertyHandle;
};

template <>
inline FOptionalSize TGizmoPerStateValueCustomization<FGizmoPerStateValueLinearColor>::GetHeightOverride()
{
	return 55.0f;
}

using FGizmoPerStateValueDoubleCustomization = TGizmoPerStateValueCustomization<FGizmoPerStateValueDouble>;
using FGizmoPerStateValueLinearColorCustomization = TGizmoPerStateValueCustomization<FGizmoPerStateValueLinearColor>;
using FGizmoPerStateValueMaterialVariantCustomization = TGizmoPerStateValueCustomization<FGizmoPerStateValueMaterialVariant>;
