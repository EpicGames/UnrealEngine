// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoStyleCustomization.h"

#include "IDetailChildrenBuilder.h"
#include "IStructureDataProvider.h"

FName FTransformGizmoStyleCustomization::GetTargetPropertyName() const
{
	return GET_MEMBER_NAME_CHECKED(FGizmosParameters, Style);
}

FName FTransformGizmoStyleCustomization::GetTargetCategoryName() const
{
	static FLazyName CategoryName(TEXT("Style"));
	return CategoryName;
}

const UScriptStruct* FTransformGizmoStyleCustomization::GetTargetStructType() const
{
	static const UScriptStruct* StructType = FTransformGizmoStyle::StaticStruct();
	return StructType;
}

const void* FTransformGizmoStyleCustomization::GetDefaultValue() const
{
	return &DefaultStyle;
}
