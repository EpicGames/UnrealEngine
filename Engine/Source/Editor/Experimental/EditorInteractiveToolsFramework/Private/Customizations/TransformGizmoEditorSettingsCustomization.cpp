// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformGizmoEditorSettingsCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorModeManager.h"
#include "EditorTRSCustomizationUtils.h"
#include "EditorTRSGizmoPresets.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailGroup.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "PropertyPath.h"
#include "Settings/EditorProjectSettings.h"
#include "Settings/EditorStyleSettings.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "TransformGizmoEditorSettings.h"
#include "UnrealWidgetFwd.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SCoordinateSystemMask.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "TransformGizmoEditorSettingsCustomization"

namespace TransformGizmoEditorSettingsCustomizationLocals
{
	static constexpr FLazyName ColumnId_Setting("Setting");
	static constexpr FLazyName ColumnId_Current("Current");
	static constexpr FLazyName ColumnId_Preset("Preset");

	using namespace UE::Editor::InteractiveToolsFramework;

	/** Table row for the preset comparison tooltip. */
	class SPresetComparisonRow : public SMultiColumnTableRow<TSharedPtr<FPresetSettingComparison>>
	{
	public:
		SLATE_BEGIN_ARGS(SPresetComparisonRow) {}
			SLATE_ARGUMENT(TSharedPtr<FPresetSettingComparison>, Item)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
		{
			Item = InArgs._Item;
			SMultiColumnTableRow<TSharedPtr<FPresetSettingComparison>>::Construct(
				FSuperRowType::FArguments()
				.Style(&FAppStyle::GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow")), 
				InOwnerTable);
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
		{
			if (!Item.IsValid())
			{
				return SNullWidget::NullWidget;
			}

			const FSlateFontInfo RowFont = FCoreStyle::GetDefaultFontStyle("Regular", 9);

			if (InColumnName == ColumnId_Setting)
			{
				return SNew(STextBlock)
					.Text(Item->SettingName)
					.Font(RowFont)
					.IsEnabled(Item->bCurrentValueDiffersToPresetValue)
					.Margin(FMargin(4.0f, 2.0f));
			}

			if (Item->bIsBool)
			{
				const bool bValue = (InColumnName == ColumnId_Current) ? Item->bCurrentValue : Item->bPresetValue;
				return SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(FMargin(4.0f, 1.0f))
					[
						SNew(SCheckBox)
						.IsChecked(bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
						.IsEnabled(Item->bCurrentValueDiffersToPresetValue)
					];
			}

			const FText& ValueText = (InColumnName == ColumnId_Current) ? Item->CurrentValue : Item->PresetValue;
			return SNew(STextBlock)
				.Text(ValueText)
				.Font(RowFont)
				.IsEnabled(Item->bCurrentValueDiffersToPresetValue)
				.Margin(FMargin(4.0f, 2.0f));
		}

	private:
		TSharedPtr<FPresetSettingComparison> Item;
	};

	/** Recursively searches for a nested property handle given an initial parent handle and a list of property names representing the path. */
	TSharedPtr<IPropertyHandle> GetNestedPropertyHandle(const TSharedPtr<IPropertyHandle>& InParentHandle, const std::initializer_list<FName> InPropertyPath)
	{
		TSharedPtr<IPropertyHandle> CurrentHandle = InParentHandle;
		for (const FName PropertyName : InPropertyPath)
		{
			CurrentHandle = CurrentHandle->GetChildHandle(PropertyName);
			if (!CurrentHandle.IsValid())
			{
				return nullptr;
			}
		}

		return CurrentHandle;
	}
	
	void SetPropertyClamping(
		const TSharedPtr<IPropertyHandle>& InPropertyHandle,
		const TOptional<float>& InUIMin,
		const TOptional<float>& InUIMax,
		const TOptional<float>& InClampMin,
		const TOptional<float>& InClampMax)
	{
		auto ApplyIfSet = [InPropertyHandle](FName InKey, const TOptional<float>& InValue)
		{
			if (!InValue.IsSet())
			{
				return;
			}

			InPropertyHandle->SetInstanceMetaData(InKey, FString::Printf(TEXT("%.1f"), InValue.GetValue()));
		};

		constexpr FLazyName UIMinKey = FLazyName("UIMin");
		constexpr FLazyName UIMaxKey = FLazyName("UIMax");
		constexpr FLazyName ClampMinKey = FLazyName("ClampMin");
		constexpr FLazyName ClampMaxKey = FLazyName("ClampMax");

		ApplyIfSet(UIMinKey, InUIMin);
		ApplyIfSet(UIMaxKey, InUIMax);
		ApplyIfSet(ClampMinKey, InClampMin);
		ApplyIfSet(ClampMaxKey, InClampMax);
	}

	void SetPropertyAsMultiplier(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		InPropertyHandle->SetInstanceMetaData("Units", "Multiplier");
	}

	FText MakeToolTipTextWithDefaultValue(const TSharedPtr<IPropertyHandle>& InPropertyHandle, const FText& InDefaultValueText)
	{
		FText ToolTipText = InPropertyHandle->GetToolTipText();
		if (ToolTipText.IsEmpty())
		{
			ToolTipText = FText::Format(
			LOCTEXT("EmptyToolTipWithDefaultValueFormat", "(Default: {0})"),
				InDefaultValueText);
		}
		else
		{
			ToolTipText = FText::Format(
			LOCTEXT("ToolTipWithDefaultValueFormat", "{0}\n(Default: {1})"),
			ToolTipText,
			InDefaultValueText);
		}

		return ToolTipText;
	}
}

TSharedRef<IDetailCustomization> FTransformGizmoEditorSettingsCustomization::MakeInstance()
{
	return MakeShared<FTransformGizmoEditorSettingsCustomization>();
}

FTransformGizmoEditorSettingsCustomization::FTransformGizmoEditorSettingsCustomization()
{
	IsUsingNewGizmosAttribute = TAttribute<bool>::Create(
		[]()
		{
			return UEditorInteractiveGizmoManager::GetTransformGizmoVersion() >= 1;
		});

	IsUsingLegacyGizmosAttribute = TAttribute<bool>::Create(
		[]()
		{
			return UEditorInteractiveGizmoManager::GetTransformGizmoVersion() == 0;
		});
}

void FTransformGizmoEditorSettingsCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	using namespace TransformGizmoEditorSettingsCustomizationLocals;

	WeakDetailBuilder = InDetailBuilder;

	UTransformGizmoEditorSettings* TransformGizmoEditorSettings = GetMutableDefault<UTransformGizmoEditorSettings>();
	if (!ensure(TransformGizmoEditorSettings))
	{
		return;
	}

	UEditorStyleSettings* EditorStyleSettings = GetMutableDefault<UEditorStyleSettings>();
	if (!ensure(EditorStyleSettings))
	{
		return;
	}

	auto MakeResetToDefault = [this](TFunction<const void*()> InGetDefaultValueFunc)
	{
		return FResetToDefaultOverride::Create(
		FIsResetToDefaultVisible::CreateLambda([this, InGetDefaultValueFunc](TSharedPtr<IPropertyHandle> InPropertyHandle)
		{
			return IsResetToDefaultVisible(InPropertyHandle, InGetDefaultValueFunc);	
		}),
		FResetToDefaultHandler::CreateLambda([this, InGetDefaultValueFunc](TSharedPtr<IPropertyHandle> InPropertyHandle)
		{
			OnResetToDefault(InPropertyHandle, InGetDefaultValueFunc);
		}),
		true);
	};

	static TSet<FName> CategoryNamesToDisplay{
		"GizmoPreset",
		"GizmoAppearance",
		"GizmoInteraction",
		"Legacy"
	};

	// Hide Default categories
	{
		TArray<FName> CategoryNamesToHide;
		InDetailBuilder->GetCategoryNames(CategoryNamesToHide);
		
		TSet<FName> DefaultCategoryNamesSet(CategoryNamesToHide);
		CategoryNamesToHide = DefaultCategoryNamesSet.Difference(CategoryNamesToDisplay).Array();

		for (FName CategoryNameToHide : CategoryNamesToHide)
		{
			InDetailBuilder->HideCategory(CategoryNameToHide);
		}
	}
	
	IDetailCategoryBuilder& GizmoPresetCategory = InDetailBuilder->EditCategory(
		"GizmoPreset", 
		LOCTEXT("GizmoPresetCategoryName", "Gizmo Preset"));
	{
		GizmoPresetCategory.SetCategoryVisibility(true);
		GizmoPresetCategory.SetSortOrder(-1);

		// Preset Combobox
		{
			const FText PresetDisplayName = LOCTEXT("PresetDisplayName", "Preset");
			GizmoPresetCategory
			.AddCustomRow(PresetDisplayName)
			.RowTag(TEXT("Preset"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(PresetDisplayName)
				.ToolTipText(LOCTEXT("PresetTooltip", "Select a predefined configuration preset to quickly apply a set of gizmo style and interaction settings."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				GeneratePresetComboBoxContent()
			]
			.EditCondition(IsUsingNewGizmosAttribute, { });
		}
	}
	
	const TSharedPtr<IPropertyHandle> GizmoParametersPropertyHandle = InDetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, GizmosParameters));

	IDetailCategoryBuilder& GizmoInteractionCategory = InDetailBuilder->EditCategory(
		"GizmoInteraction",
		LOCTEXT("GizmoInteractionCategoryName", "Gizmo Interaction"));
	{
		GizmoInteractionCategory.SetCategoryVisibility(true);
		GizmoInteractionCategory.SetSortOrder(0);
		
		const TSharedPtr<IPropertyHandle> DragDuplicateOnRotationProperty = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, bDragDuplicateOnRotation));
		GizmoInteractionCategory.AddProperty(DragDuplicateOnRotationProperty)
		.EditCondition(IsUsingNewGizmosAttribute, { });
		
		const TSharedPtr<IPropertyHandle> ScreenSpaceNudgeProperty = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, bScreenSpaceNudge));
		GizmoInteractionCategory.AddProperty(ScreenSpaceNudgeProperty)
		.EditCondition(IsUsingNewGizmosAttribute, { });

		TSharedPtr<IPropertyHandle> InteractionPropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, Interaction));

		// Additional Coordinate Systems
		{
			const TSharedPtr<IPropertyHandle> EnableExplicitPropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, bEnableExplicit));
			const TSharedPtr<IPropertyHandle> AdditionalCoordinateSystemsPropertyHandle = InteractionPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformGizmoInteraction, AdditionalCoordinateSystems));

			const FText AdditionalCoordinateSystemsDisplayName = LOCTEXT("AdditionalCoordinateSystemsDisplayName", "Additional Coordinate Systems");
			GizmoInteractionCategory
			.AddCustomRow(AdditionalCoordinateSystemsDisplayName)
			.RowTag("AdditionalCoordinateSystems")
			.EditCondition(IsUsingNewGizmosAttribute, { })
			.NameContent()
			[
				SNew(STextBlock)
				.Text(AdditionalCoordinateSystemsDisplayName)
				.ToolTipText(LOCTEXT("AdditionalCoordinateSystems_Tooltip", "Select which additional coordinate systems are available for the transform gizmo. World and Local are always enabled."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				// @todo: remove use of GizmoParameters after EditorTRS -> EditorITF
				SNew(SCoordinateSystemBitmask)
				.ValuesToHide({
					ECoordSystem::COORD_World, 
					ECoordSystem::COORD_Local
				})
				.Value_Lambda([EnableExplicitPropertyHandle]() -> TOptional<int32>
				{
					int32 AdditionalCoordinateSystems = INDEX_NONE; // All enabled (all bits on)
					if (!EnableExplicitPropertyHandle.IsValid())
					{
						return TOptional<int32>();
					}

					bool bIsExplicitEnabled = false;
					if (EnableExplicitPropertyHandle->GetValue(bIsExplicitEnabled) == FPropertyAccess::Success)
					{
						if (bIsExplicitEnabled)
						{
							AdditionalCoordinateSystems |= (1 << COORD_Explicit);
						}
						else
						{
							AdditionalCoordinateSystems &= ~(1 << COORD_Explicit);
						}

						return AdditionalCoordinateSystems;
					}

					return TOptional<int32>();
				})
				.OnSetValue_Lambda([AdditionalCoordinateSystemsPropertyHandle, EnableExplicitPropertyHandle](int32 InNewValue)
				{
					if (AdditionalCoordinateSystemsPropertyHandle.IsValid())
					{
						AdditionalCoordinateSystemsPropertyHandle->SetValue(InNewValue);
						
						bool bIsExplicitEnabled = (InNewValue & (1 << COORD_Explicit)) != 0;
						if (EnableExplicitPropertyHandle.IsValid())
						{
							EnableExplicitPropertyHandle->SetValue(bIsExplicitEnabled);	
						}
					}
				})
			];
		}

		const TSharedPtr<IPropertyHandle> IndirectScaleIsUniformPropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, bUniformIndirectScale));
		GizmoInteractionCategory.AddProperty(IndirectScaleIsUniformPropertyHandle)
		.EditCondition(IsUsingNewGizmosAttribute, { });

		const TSharedPtr<IPropertyHandle> PersistHandleSelectionPropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, bPersistHandleSelection));
		GizmoInteractionCategory.AddProperty(PersistHandleSelectionPropertyHandle)
		.OverrideResetToDefault(MakeResetToDefault([]()
		{
			static FTransformGizmoInteraction DefaultInteraction;
			return &DefaultInteraction.bPersistHandleSelection;
		}))
		.EditCondition(IsUsingNewGizmosAttribute, { });

		const TSharedPtr<IPropertyHandle> SequentialIndirectAxesButtonsPropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, bSequentialIndirectAxesButtons));
		GizmoInteractionCategory.AddProperty(SequentialIndirectAxesButtonsPropertyHandle)
		.EditCondition(IsUsingNewGizmosAttribute, { });
		
		const TSharedPtr<IPropertyHandle> AdditiveIndirectAxesPropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, bAdditiveIndirectAxes));
		GizmoInteractionCategory.AddProperty(AdditiveIndirectAxesPropertyHandle)
		.EditCondition(IsUsingNewGizmosAttribute, { });

		const TSharedPtr<IPropertyHandle> ScreenSpaceIndirectOrthographicManipulationPropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, bScreenSpaceIndirectOrthographicManipulation));
		GizmoInteractionCategory.AddProperty(ScreenSpaceIndirectOrthographicManipulationPropertyHandle)
		.EditCondition(IsUsingNewGizmosAttribute, { });

		const TSharedPtr<IPropertyHandle> DefaultRotateModePropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, RotateMode));
		GizmoInteractionCategory.AddProperty(DefaultRotateModePropertyHandle)
		.EditCondition(IsUsingNewGizmosAttribute, { })
		.OverrideResetToDefault(MakeResetToDefault([]() -> EAxisRotateMode::Type*
		{
			static EAxisRotateMode::Type DefaultRotateMode = FGizmoElementRotateInteraction::GetDefaultRotateMode();
			return &DefaultRotateMode;
		}));

		const TSharedPtr<IPropertyHandle> ScaleTypePropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, ScaleType));
		GizmoInteractionCategory.AddProperty(ScaleTypePropertyHandle)
		.EditCondition(IsUsingNewGizmosAttribute, { })
		.OverrideResetToDefault(MakeResetToDefault([]() -> EGizmoTransformScaleType*
		{
			static EGizmoTransformScaleType DefaultScaleType = FGizmoElementScaleInteraction::GetDefaultScaleType();
			return &DefaultScaleType;
		}));
	}

	IDetailCategoryBuilder& GizmoAppearanceCategory = InDetailBuilder->EditCategory(
		"GizmoAppearance",
		LOCTEXT("GizmoAppearanceCategoryName", "Gizmo Appearance"));
	{
		GizmoAppearanceCategory.SetSortOrder(1);

		const TSharedPtr<IPropertyHandle> StylePropertyHandle = GizmoParametersPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmosParameters, Style));

		// Axis Line Length
		{
			constexpr float UIMaxAxisSizeMultiplier = 10.0f;
			
			const TSharedPtr<IPropertyHandle> AxisSizeMultiplierPropertyHandle = StylePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformGizmoStyle, AxisSizeMultiplier));
			SetPropertyClamping(AxisSizeMultiplierPropertyHandle, { }, UIMaxAxisSizeMultiplier, { }, { });
			SetPropertyAsMultiplier(AxisSizeMultiplierPropertyHandle);

			GizmoAppearanceCategory.AddProperty(AxisSizeMultiplierPropertyHandle)
			.DisplayName(LOCTEXT("AxisLineLengthDisplayName", "Axis Line Length"))
			.ToolTip(MakeToolTipTextWithDefaultValue(AxisSizeMultiplierPropertyHandle, FText::AsNumber(FTransformGizmoStyle().AxisSizeMultiplier)))
			.OverrideResetToDefault(MakeResetToDefault([]()
			{
				static FTransformGizmoStyle DefaultStyle;
				return &DefaultStyle.AxisSizeMultiplier;
			}))
			.EditCondition(IsUsingNewGizmosAttribute, { });
		}

		// Axis Line Thickness
		{
			// Only in the UI, you can type in higher values
			constexpr float UIMaxLineThickness = 8.0f;

			const TSharedPtr<IPropertyHandle> AxisLineThicknessPropertyHandle = StylePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformGizmoStyle, LineThicknessMultiplier));
			SetPropertyClamping(AxisLineThicknessPropertyHandle, { }, UIMaxLineThickness, { }, { });
			SetPropertyAsMultiplier(AxisLineThicknessPropertyHandle);

			GizmoAppearanceCategory.AddProperty(AxisLineThicknessPropertyHandle)
			.DisplayName(LOCTEXT("AxisLineThicknessDisplayName", "Axis Line Thickness"))
			.ToolTip(MakeToolTipTextWithDefaultValue(AxisLineThicknessPropertyHandle, FText::AsNumber(FTransformGizmoStyle().LineThicknessMultiplier)))
			.OverrideResetToDefault(MakeResetToDefault([]()
			{
				static FTransformGizmoStyle DefaultStyle;
				return &DefaultStyle.LineThicknessMultiplier;
			}))
			.EditCondition(IsUsingNewGizmosAttribute, { });
		}

		// Handle Size
		{
			constexpr float UIMaxHandleSize = 4.0f;

			const TSharedPtr<IPropertyHandle> HandleSizePropertyHandle = StylePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformGizmoStyle, HandleSizeMultiplier));
			SetPropertyClamping(HandleSizePropertyHandle, { }, UIMaxHandleSize, { }, { });
			SetPropertyAsMultiplier(HandleSizePropertyHandle);

			GizmoAppearanceCategory.AddProperty(HandleSizePropertyHandle)
			.DisplayName(LOCTEXT("HandleSizeDisplayName", "Handle Size"))
			.ToolTip(MakeToolTipTextWithDefaultValue(HandleSizePropertyHandle, FText::AsNumber(FTransformGizmoStyle().HandleSizeMultiplier)))
			.OverrideResetToDefault(MakeResetToDefault([]()
			{
				static FTransformGizmoStyle DefaultStyle;
				return &DefaultStyle.HandleSizeMultiplier;
			}))
			.EditCondition(IsUsingNewGizmosAttribute, { });
		}

		// Planar Axis Offset
		{
			const TSharedPtr<IPropertyHandle> PlanarAxisOffsetPropertyHandle = StylePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformGizmoStyle, PlanarAxisOffsetFromOrigin));
			GizmoAppearanceCategory.AddProperty(PlanarAxisOffsetPropertyHandle)
			.DisplayName(LOCTEXT("PlanarAxisOffsetDisplayName", "Planar Axis Offset (from origin)"))
			.ToolTip(MakeToolTipTextWithDefaultValue(PlanarAxisOffsetPropertyHandle, FText::AsNumber(FTransformGizmoStyle().PlanarAxisOffsetFromOrigin)))
			.OverrideResetToDefault(MakeResetToDefault([]()
			{
				static FTransformGizmoStyle DefaultStyle;
				return &DefaultStyle.PlanarAxisOffsetFromOrigin;
			}))
			.EditCondition(IsUsingNewGizmosAttribute, { });
		}

		// (Visual) Elements/ShowFlags
		{
			const TSharedPtr<IPropertyHandle> ShowFlagsPropertyHandle = StylePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformGizmoStyle, ShowFlags));
			GizmoAppearanceCategory.AddProperty(ShowFlagsPropertyHandle)
			.DisplayName(LOCTEXT("ElementsDisplayName", "Element Visibility"))
			.ToolTip(LOCTEXT("ElementsTooltip", "Select what gizmo elements are visible."))
			.OverrideResetToDefault(MakeResetToDefault([]()
			{
				static FTransformGizmoStyle DefaultStyle;
				return &DefaultStyle.ShowFlags;
			}))
			.EditCondition(IsUsingNewGizmosAttribute, { });
		}

		// Arcball Opacity
		{
			const TSharedPtr<IPropertyHandle> ArcballOpacityPropertyHandle =
			GetNestedPropertyHandle(
				StylePropertyHandle,
				{
					GET_MEMBER_NAME_CHECKED(FTransformGizmoStyle, RotateStyle),
					GET_MEMBER_NAME_CHECKED(FGizmoElementRotateStyle, ArcballStyle),
					GET_MEMBER_NAME_CHECKED(FGizmoElementRotateArcballStyle, Colors),
					GET_MEMBER_NAME_CHECKED(FGizmoPerStateValueLinearColor, Default),
					GET_MEMBER_NAME_CHECKED(FLinearColor, A)
				});

			// Clamp Opacity
			SetPropertyClamping(ArcballOpacityPropertyHandle, 0.0f, 1.0f, 0.0f, 1.0f);

			GizmoAppearanceCategory.AddProperty(ArcballOpacityPropertyHandle)
			.DisplayName(LOCTEXT("ArcballOpacityDisplayName", "Arcball Opacity"))
			.ToolTip(MakeToolTipTextWithDefaultValue(ArcballOpacityPropertyHandle,
				FText::AsNumber(
					FTransformGizmoStyle().RotateStyle.ArcballStyle.Colors.Default.Get(FLinearColor::White).A)))
			.OverrideResetToDefault(MakeResetToDefault([]()
			{
				static FTransformGizmoStyle DefaultStyle;
				return &DefaultStyle.RotateStyle.ArcballStyle.Colors.GetDefaultValue().A;
			}))
			.EditCondition(IsUsingNewGizmosAttribute, { });
		}

		// Shading
		{
			const TSharedPtr<IPropertyHandle> ShadingTogglePropertyHandle = StylePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformGizmoStyle, bUseShading));
			GizmoAppearanceCategory.AddProperty(ShadingTogglePropertyHandle)
			.OverrideResetToDefault(MakeResetToDefault([]()
			{
				static FTransformGizmoStyle DefaultStyle;
				return &DefaultStyle.bUseShading;
			}))
			.EditCondition(IsUsingNewGizmosAttribute, { });

			const TSharedPtr<IPropertyHandle> ShadingAmbientPropertyHandle = StylePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformGizmoStyle, ShadingAmbient));
			GizmoAppearanceCategory.AddProperty(ShadingAmbientPropertyHandle)
			.OverrideResetToDefault(MakeResetToDefault([]()
			{
				static FTransformGizmoStyle DefaultStyle;
				return &DefaultStyle.ShadingAmbient;
			}))
			.EditCondition(IsUsingNewGizmosAttribute, { });

			const TSharedPtr<IPropertyHandle> ShadingGlossinessPropertyHandle = StylePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformGizmoStyle, ShadingSpecularGlossiness));
			GizmoAppearanceCategory.AddProperty(ShadingGlossinessPropertyHandle)
			.OverrideResetToDefault(MakeResetToDefault([]()
			{
				static FTransformGizmoStyle DefaultStyle;
				return &DefaultStyle.ShadingSpecularGlossiness;
			}))
			.EditCondition(IsUsingNewGizmosAttribute, { });
		}

		// Occlusion
		{
			const FText OcclusionSettingsGroupDisplayName = LOCTEXT("OcclusionSettingsGroupName", "Occlusion Settings");
			IDetailGroup& OcclusionGroup = GizmoAppearanceCategory.AddGroup(
				"OcclusionSettings",
				OcclusionSettingsGroupDisplayName);

			const FMargin IconPadding(4.0f, 0.0, 0.0f, 0.0f);

			FDetailWidgetRow& OcclusionGroupRow = OcclusionGroup.HeaderRow();
			OcclusionGroupRow
			.ShouldAutoExpand(true)
			.WholeRowContent()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)
				.ToolTipText(
					LOCTEXT(
						"OcclusionSettingsGroupTooltip",
						"Changing this occlusion value also updates the other occlusion setting in the General > Appearance section and affects spline occlusion."))
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(OcclusionSettingsGroupDisplayName)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(IconPadding)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Info"))
				]
			];

			const UEditorStyleSettings* DefaultEditorStyleSettings = GetDefault<UEditorStyleSettings>();

			const TSharedPtr<IPropertyHandle> OcclusionDitheringPatternSizePropertyHandle = InDetailBuilder->AddObjectPropertyData({ EditorStyleSettings }, GET_MEMBER_NAME_CHECKED(UEditorStyleSettings, OccludedDitheringPatternSize));
			OcclusionGroup.AddPropertyRow(OcclusionDitheringPatternSizePropertyHandle.ToSharedRef())
			.ToolTip(MakeToolTipTextWithDefaultValue(OcclusionDitheringPatternSizePropertyHandle, FText::AsNumber(DefaultEditorStyleSettings->OccludedDitheringPatternSize)));

			const TSharedPtr<IPropertyHandle> OcclusionOpacityPropertyHandle = InDetailBuilder->AddObjectPropertyData({ EditorStyleSettings }, GET_MEMBER_NAME_CHECKED(UEditorStyleSettings, OccludedDithering));
			OcclusionGroup.AddPropertyRow(OcclusionOpacityPropertyHandle.ToSharedRef())
			.ToolTip(MakeToolTipTextWithDefaultValue(OcclusionOpacityPropertyHandle, FText::AsNumber(DefaultEditorStyleSettings->OccludedDithering)));

			const TSharedPtr<IPropertyHandle> OcclusionBrightnessPropertyHandle = InDetailBuilder->AddObjectPropertyData({ EditorStyleSettings }, GET_MEMBER_NAME_CHECKED(UEditorStyleSettings, OccludedBrightness));
			OcclusionGroup.AddPropertyRow(OcclusionBrightnessPropertyHandle.ToSharedRef())
			.ToolTip(MakeToolTipTextWithDefaultValue(OcclusionBrightnessPropertyHandle, FText::AsNumber(DefaultEditorStyleSettings->OccludedBrightness)));
		}
	}
	
	IDetailCategoryBuilder& LegacyGizmoCategory = InDetailBuilder->EditCategory(
		"LegacyGizmo",
		LOCTEXT("LegacyGizmoCategoryName", "Legacy Gizmo"));
	{
		LegacyGizmoCategory.SetSortOrder(2);

		// Enable Legacy Gizmo
		// @note: This inverts the logic of bUseExperimentalGizmo and sets both flags together
		// so the UI only offers "New" or "Legacy" (the intermediate state is CVar-only).
		{
			const FText EnableLegacyGizmoDisplayName = LOCTEXT("UseLegacyGizmoDisplayName", "Enable Legacy");
			LegacyGizmoCategory
			.AddCustomRow(EnableLegacyGizmoDisplayName)
			.RowTag("bUseLegacyGizmo")
			.NameContent()
			[
				SNew(STextBlock)
				.Text(EnableLegacyGizmoDisplayName)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([]()
				{
					const bool bUseLegacyGizmo = UEditorInteractiveGizmoManager::GetTransformGizmoVersion() == 0;
					return bUseLegacyGizmo ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([](ECheckBoxState InCheckState)
				{
					const bool bEnableLegacyGizmo = (InCheckState == ECheckBoxState::Checked);

					// @note: when legacy is turned off, we always switch to the default 5.8 Gizmo, not the "old" experimental one.
					UEditorInteractiveGizmoManager::SetTransformGizmoVersion(bEnableLegacyGizmo ? 0 : 2);
					
					// Ensure that we switch away from the combined translation/rotation widget mode
					// SetTransformGizmoVersion() can't do this conveniently due to it running at module startup prior to the level editor being created.
					if (!bEnableLegacyGizmo && GLevelEditorModeTools().GetWidgetMode() == UE::Widget::WM_TranslateRotateZ)
					{
						GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_Translate);
					}
				})
			];
		}

		// Gizmo Size
		TSharedPtr<IPropertyHandle> GizmoSizePropertyHandle = InDetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, TransformGizmoSize));
		LegacyGizmoCategory.AddProperty(GizmoSizePropertyHandle)
		.EditCondition(IsUsingLegacyGizmosAttribute, { });

		// Enable Arcball Rotate
		TSharedPtr<IPropertyHandle> EnableArcballPropertyHandle = InDetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, bEnableArcballRotate));
		LegacyGizmoCategory.AddProperty(EnableArcballPropertyHandle)
		.EditCondition(IsUsingLegacyGizmosAttribute, { });

		// Enable Screen Rotate
		TSharedPtr<IPropertyHandle> EnableScreenRotatePropertyHandle = InDetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, bEnableScreenRotate));
		LegacyGizmoCategory.AddProperty(EnableScreenRotatePropertyHandle)
		.EditCondition(IsUsingLegacyGizmosAttribute, { });

		// Enable Axis Drawing for Transform Edit Widget
		TSharedPtr<IPropertyHandle> EnableAxisDrawingPropertyHandle = InDetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, bEnableAxisDrawing));
		LegacyGizmoCategory.AddProperty(EnableAxisDrawingPropertyHandle)
		.EditCondition(IsUsingLegacyGizmosAttribute, { });

		// Enable Combined Translate/Rotate Widget
		TSharedPtr<IPropertyHandle> EnableCombinedTranslateRotatePropertyHandle = InDetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, bEnableCombinedTranslateRotate));
		LegacyGizmoCategory.AddProperty(EnableCombinedTranslateRotatePropertyHandle)
		.EditCondition(IsUsingLegacyGizmosAttribute, { });
	}
}

void FTransformGizmoEditorSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Not used, but must be implemented
}

FText FTransformGizmoEditorSettingsCustomization::GetActivePresetDisplayName() const
{
	const FText CustomText = LOCTEXT("PresetCustom", "Custom");

	UE::Editor::InteractiveToolsFramework::FTransformGizmoPresetAccessor PresetAccessor;
	for (const FTransformGizmoPreset& Preset : FTransformGizmoPreset::GetBuiltInPresets())
	{
		if (PresetAccessor.CurrentSettingsMatchPreset(Preset))
		{
			return Preset.DisplayName;
		}
	}

	return CustomText;
}

void FTransformGizmoEditorSettingsCustomization::ApplyPreset(const FTransformGizmoPreset& InPreset)
{
	UE::Editor::InteractiveToolsFramework::FTransformGizmoPresetAccessor PresetAccessor;
	PresetAccessor.ApplyPreset(InPreset);

	// Save configs
	SaveConfigs();
}

void FTransformGizmoEditorSettingsCustomization::SaveConfigs()
{
	GetMutableDefault<UTransformGizmoEditorSettings>()->SaveConfig();
}

TSharedRef<IToolTip> FTransformGizmoEditorSettingsCustomization::GeneratePresetTooltip(const FTransformGizmoPreset& InPreset)
{
	using namespace UE::Editor::InteractiveToolsFramework;
	using namespace TransformGizmoEditorSettingsCustomizationLocals;

	constexpr FTransformGizmoPresetAccessor PresetAccessor;
	TArray<FPresetSettingComparison> RawComparisons = PresetAccessor.BuildComparisonWithPreset(InPreset);

	TSharedRef<TArray<TSharedPtr<FPresetSettingComparison>>> Items = MakeShared<TArray<TSharedPtr<FPresetSettingComparison>>>();
	Items->Reserve(RawComparisons.Num());
	for (FPresetSettingComparison& Comparison : RawComparisons)
	{
		Items->Add(MakeShared<FPresetSettingComparison>(MoveTemp(Comparison)));
	}

	const FSlateFontInfo HeaderFont = FCoreStyle::GetDefaultFontStyle("Bold", 9);

	const TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow)
		+ SHeaderRow::Column(ColumnId_Setting)
		.DefaultLabel(LOCTEXT("TooltipHeaderSetting", "Setting"))
		.FillWidth(0.5f)
		.HAlignHeader(HAlign_Left)
		.HAlignCell(HAlign_Left)
		+ SHeaderRow::Column(ColumnId_Current)
		.DefaultLabel(LOCTEXT("TooltipHeaderCurrent", "Current"))
		.FillWidth(0.25f)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Center)
		+ SHeaderRow::Column(ColumnId_Preset)
		.DefaultLabel(LOCTEXT("TooltipHeaderPreset", "Preset"))
		.FillWidth(0.25f)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Center);

	const TSharedRef<SListView<TSharedPtr<FPresetSettingComparison>>> ListView =
		SNew(SListView<TSharedPtr<FPresetSettingComparison>>)
		.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("ListView"))
		.ListItemsSource(&Items.Get())
		.HeaderRow(HeaderRow)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow_Lambda([Items](TSharedPtr<FPresetSettingComparison> InItem, const TSharedRef<STableViewBase>& InOwnerTable) -> TSharedRef<ITableRow>
		{
			return SNew(SPresetComparisonRow, InOwnerTable)
				.Item(InItem);
		});

	return SNew(SToolTip)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f)
		[
			SNew(STextBlock)
			.Text(InPreset.ToolTipText)
			.Font(FCoreStyle::GetDefaultFontStyle("Italic", 9)) 
			.ColorAndOpacity(FStyleColors::ForegroundHeader)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(300.0f)
		.Padding(0.0f, 4.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			.MinDesiredWidth(550.0f)
			[
				ListView
			]
		]
	];
}

TSharedRef<SWidget> FTransformGizmoEditorSettingsCustomization::GeneratePresetComboBoxContent()
{
	return SNew(SComboButton)
		.OnGetMenuContent_Lambda([WeakThis = AsWeak()]() -> TSharedRef<SWidget>
		{
			constexpr bool bCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bCloseWindowAfterMenuSelection, nullptr);

			if (const TSharedPtr<FTransformGizmoEditorSettingsCustomization> StrongThis = StaticCastSharedPtr<FTransformGizmoEditorSettingsCustomization>(WeakThis.Pin()))
			{
				for (const FTransformGizmoPreset& Preset : FTransformGizmoPreset::GetBuiltInPresets())
				{
					const TSharedRef<SWidget> EntryWidget =
						SNew(STextBlock)
						.Text(Preset.DisplayName)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.ToolTip(StrongThis->GeneratePresetTooltip(Preset));

					MenuBuilder.AddMenuEntry(
						FUIAction(FExecuteAction::CreateLambda([PresetCopy = Preset]()
						{
							FTransformGizmoEditorSettingsCustomization::ApplyPreset(PresetCopy);
						})),
						EntryWidget);
				}
			}

			return MenuBuilder.MakeWidget();
		})
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text_Lambda([WeakThis = AsWeak()]() -> FText
			{
				if (const TSharedPtr<FTransformGizmoEditorSettingsCustomization> StrongThis = StaticCastSharedPtr<FTransformGizmoEditorSettingsCustomization>(WeakThis.Pin()))
				{
					return StrongThis->GetActivePresetDisplayName();
				}

				return FText::GetEmpty();
			})
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

bool FTransformGizmoEditorSettingsCustomization::IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> InPropertyHandle, const TFunction<const void*()>& InGetDefaultValueFunc)
{
	using namespace  UE::Editor::GizmoSettings::Private;

	if (!InPropertyHandle.IsValid() || !InPropertyHandle->IsValidHandle())
	{
		return false;
	}

	const FProperty* Property = InPropertyHandle->GetProperty();
	if (!Property)
	{
		return false;
	}

	// Get Property Value
	void* ValuePtr = nullptr;
	if (InPropertyHandle->GetValueData(ValuePtr) != FPropertyAccess::Success || !ValuePtr)
	{
		return false;
	}

	// Get Default Value
	const void* DefaultValuePtr = InGetDefaultValueFunc();
	if (!DefaultValuePtr)
	{
		return false;
	}

	// If the value is already set to the default, don't show the reset button
	return !Property->Identical(ValuePtr, DefaultValuePtr);
}

void FTransformGizmoEditorSettingsCustomization::OnResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle, const TFunction<const void*()>& InGetDefaultValueFunc)
{
	using namespace UE::Editor::GizmoSettings::Private;

	if (!InPropertyHandle.IsValid() || !InPropertyHandle->IsValidHandle())
	{
		return;
	}

	const FProperty* Property = InPropertyHandle->GetProperty();
	if (!Property)
	{
		return;
	}
	
	// Get Property Value
	void* ValuePtr = nullptr;
	if (InPropertyHandle->GetValueData(ValuePtr) != FPropertyAccess::Success || !ValuePtr)
	{
		return;
	}

	// Get Default Value
	const void* DefaultValuePtr = InGetDefaultValueFunc();
	if (!DefaultValuePtr)
	{
		return;
	}

	Property->CopyCompleteValue(ValuePtr, DefaultValuePtr);
	InPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	SaveConfigs();
}

#undef LOCTEXT_NAMESPACE
