// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorTRSGizmoPresets.h"
#include "TransformGizmoEditorSettings.h"
#include "IDetailCustomization.h"
#include "Misc/Attribute.h"

class IToolTip;
class IPropertyHandle;

class FTransformGizmoEditorSettingsCustomization
	: public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	FTransformGizmoEditorSettingsCustomization();

	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	bool IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> InPropertyHandle, const TFunction<const void*()>& InGetDefaultValueFunc);
	void OnResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle, const TFunction<const void*()>& InGetDefaultValueFunc);

	/** Generate the preset combobox widget. */
	TSharedRef<SWidget> GeneratePresetComboBoxContent();

	/** Returns the current display name for the active preset (or "Custom"). */
	FText GetActivePresetDisplayName() const;

	/** Generates a rich tooltip widget showing a comparison table between current settings and a preset. */
	TSharedRef<IToolTip> GeneratePresetTooltip(const FTransformGizmoPreset& InPreset);

	/** Apply the given preset values to the settings objects. */
	static void ApplyPreset(const FTransformGizmoPreset& InPreset);

	/** Calls SaveConfig on the common Settings objects. */
	static void SaveConfigs();

private:
	TAttribute<bool> IsUsingNewGizmosAttribute;
	TAttribute<bool> IsUsingLegacyGizmosAttribute;
	TWeakPtr<IDetailLayoutBuilder> WeakDetailBuilder;
};
