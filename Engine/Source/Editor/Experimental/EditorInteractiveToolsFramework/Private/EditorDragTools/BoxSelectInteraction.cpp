// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDragTools/BoxSelectInteraction.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "Components/PrimitiveComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorDragTools/EditorViewportClientProxy.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Engine/LevelStreaming.h"
#include "EngineUtils.h"
#include "GameFramework/Volume.h"
#include "LevelEditorViewport.h"
#include "Model.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "UnrealEdGlobals.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

namespace UE::EditorDragTools::Private
{
TArray<FTypedElementHandle> GetElementsIntersectingBox(
	const AActor* Actor,
	const FBox& InBox,
	const IEditorViewportClientProxy* InEditorViewportClientProxy,
	const FWorldSelectionElementArgs& SelectionArgs
)
{
	if (InEditorViewportClientProxy && InEditorViewportClientProxy->IsActorVisible(Actor))
	{
		if (FTypedElementHandle ActorHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor))
		{
			if (TTypedElement<ITypedElementWorldInterface> WorldElement =
					UTypedElementRegistry::GetInstance()->GetElement<ITypedElementWorldInterface>(ActorHandle))
			{
				return WorldElement.GetSelectionElementsInBox(InBox, SelectionArgs);
			}
		}
	}

	return {};
}

void AddHoverEffect(const FViewportHoverTarget& InHoverTarget)
{
	FLevelEditorViewportClient::AddHoverEffect(InHoverTarget);
	FLevelEditorViewportClient::HoveredObjects.Add(InHoverTarget);
}

void RemoveHoverEffect(const FViewportHoverTarget& InHoverTarget)
{
	FSetElementId Id = FLevelEditorViewportClient::HoveredObjects.FindId(InHoverTarget);
	if (Id.IsValidId())
	{
		FLevelEditorViewportClient::RemoveHoverEffect(InHoverTarget);
		FLevelEditorViewportClient::HoveredObjects.Remove(Id);
	}
}

} // namespace UE::EditorDragTools::Private

UBoxSelectInteraction::UBoxSelectInteraction()
{
	InteractionName = UE::Editor::ViewportInteractions::BoxSelect;

	if (UViewportClickDragBehavior* ClickDragBehavior = ClickDragInputBehavior.Get())
	{
		ClickDragBehavior->SetBindings({ UE::Editor::ViewportInteractions::FButtonBinding(EKeys::LeftMouseButton).TriggersStart() });
	}
}

FInputRayHit UBoxSelectInteraction::CanBeginClickDragSequence(const FInputDeviceRay& InPressPos)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		if (EditorViewportClient->IsOrtho())
		{
			return FInputRayHit(TNumericLimits<float>::Max());
		}
	}

	return FInputRayHit();
}

void UBoxSelectInteraction::OnDragStart(const FInputDeviceRay& InPressPos)
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetMouseCursorOverride(EMouseCursor::Default);
	}

	FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient();
	if (!EditorViewportClient)
	{
		return;
	}

	FViewport* Viewport = EditorViewportClient->Viewport;
	if (!Viewport)
	{
		return;
	}

	OnActivateTool().Broadcast();
	bIsDragging = true;

	const FVector2D MouseDownPosition = ClickDragInputBehavior->GetMouseDownPosition();
	Start = FVector(MouseDownPosition.X, MouseDownPosition.Y, 0.0);
	End = Start;

	FLevelEditorViewportClient::ClearHoverFromObjects();

	// Create a list of bsp models to check for intersection with the box
	ModelsToCheck.Reset();
	// Do not select BSP if its not visible
	if (EditorViewportClient->EngineShowFlags.BSP)
	{
		UWorld* World = EditorViewportClient->GetWorld();
		check(World);
		// Add the persistent level always
		ModelsToCheck.Add(World->PersistentLevel->Model);
		// Add all streaming level models
		for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
		{
			// Only add streaming level models if the level is visible
			if (StreamingLevel && StreamingLevel->GetShouldBeVisibleInEditor())
			{
				if (ULevel* Level = StreamingLevel->GetLoadedLevel())
				{
					ModelsToCheck.Add(Level->Model);
				}
			}
		}
	}
}

void UBoxSelectInteraction::OnDrag(const FDragArgs& InDrag)
{
	if (!EditorViewportClientProxy)
	{
		return;
	}

	FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient();
	if (!EditorViewportClient)
	{
		return;
	}

	FEditorModeTools* const ModeTools = EditorViewportClient->GetModeTools();
	if (!ModeTools)
	{
		return;
	}

	FIntPoint MousePos;
	EditorViewportClient->Viewport->GetMousePos(MousePos);

	End = FVector(MousePos);

	const bool bUseHoverFeedback = GEditor && GetDefault<ULevelEditorViewportSettings>()->bEnableViewportHoverFeedback;

	if (bUseHoverFeedback)
	{
		const bool bStrictDragSelection = GetDefault<ULevelEditorViewportSettings>()->bStrictBoxSelection;

		UTypedElementSelectionSet* SelectionSet = ModeTools->GetEditorSelectionSet();

		UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>();
		const bool bGeometryMode = BrushSubsystem ? BrushSubsystem->IsGeometryEditorModeActive() : false;

		FWorldSelectionElementArgs SelectionArgs{ SelectionSet,
												  ETypedElementSelectionMethod::Primary,
												  FTypedElementSelectionOptions(),
												  &(EditorViewportClient->EngineShowFlags),
												  bStrictDragSelection,
												  bGeometryMode };

		// If we are using over feedback calculate a new box from the one being dragged
		FBox SelBBox;
		CalculateBox(SelBBox);

		// Check every actor to see if it intersects the frustum created by the box
		// If it does, the actor will be selected and should be given a hover cue
		UWorld* IteratorWorld = GWorld;
		for (FActorIterator It(IteratorWorld); It; ++It)
		{
			AActor& Actor = **It;
			const bool bActorHitByBox = !UE::EditorDragTools::Private::GetElementsIntersectingBox(
											 &Actor, SelBBox, EditorViewportClientProxy, SelectionArgs
			)
											 .IsEmpty();

			if (bActorHitByBox)
			{
				// Apply a hover effect to any actor that will be selected
				AddHoverEffect(Actor);
			}
			else
			{
				// Remove any hover effect on this actor as it no longer will be selected by the current box
				RemoveHoverEffect(Actor);
			}
		}

		// Check each model to see if it will be selected
		for (int32 ModelIndex = 0; ModelIndex < ModelsToCheck.Num(); ++ModelIndex)
		{
			UModel& Model = *ModelsToCheck[ModelIndex];
			for (int32 NodeIndex = 0; NodeIndex < Model.Nodes.Num(); NodeIndex++)
			{
				if (IntersectsBox(Model, NodeIndex, SelBBox, bStrictDragSelection))
				{
					// Apply a hover effect to any bsp surface that will be selected
					AddHoverEffect(Model, Model.Nodes[NodeIndex].iSurf);
				}
				else
				{
					// Remove any hover effect on this bsp surface as it no longer will be selected by the current box
					RemoveHoverEffect(Model, Model.Nodes[NodeIndex].iSurf);
				}
			}
		}
	}
}

void UBoxSelectInteraction::OnDragEnd(const FInputDeviceRay& InReleasePos)
{
	if (EditorViewportClientProxy)
	{
		if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
		{
			FEditorModeTools* const ModeTools = EditorViewportClient->GetModeTools();
			if (!ModeTools)
			{
				return;
			}

			// Workaround to use while ITF and legacy input coexist.
			// Editor Viewport Client processes ITF input before anything else.
			// This means that, while ITF and legacy input coexist, simple clicks will be "stolen" by ITF Box Select, which
			// is enabled by a simple Left Mouse Button click, with no modifiers. Because IE_Released events are used as
			// "Clicks" by the Viewport Client only while tracking, but tracking will be false after this OnClickRelease,
			// the Viewport Client will never use this IE_Release to trigger a click.
			// This workaround checks the distance traveled during this Drag, and if it's close to zero, forwards a IE_Released
			// event for left mouse button to the Viewport Client, bypassing the Internal_InputKey method.
			{
				UE::Math::TVector<double> Distance = End - Start;
				const bool bForwardAsSimpleClick = FMath::IsNearlyZero(Distance.Length());

				if (bForwardAsSimpleClick)
				{
					FSceneViewFamilyContext ViewFamily(
						FSceneViewFamily::ConstructionValues(
							EditorViewportClient->Viewport, EditorViewportClient->GetScene(), EditorViewportClient->EngineShowFlags
						)
							.SetRealtimeUpdate(EditorViewportClient->IsRealtime())
					);
					FSceneView* View = EditorViewportClient->CalcSceneView(&ViewFamily);

					FIntPoint MousePos;
					EditorViewportClient->Viewport->GetMousePos(MousePos);

					HHitProxy* const Proxy = EditorViewportClient->Viewport->GetHitProxy(MousePos.X, MousePos.Y);

					EditorViewportClient->ProcessClick(
						*View, Proxy, EKeys::LeftMouseButton, IE_Released, MousePos.X, MousePos.Y
					);

					return;
				}
			}

			UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>();
			const bool bGeometryMode = BrushSubsystem ? BrushSubsystem->IsGeometryEditorModeActive() : false;

			FScopedTransaction Transaction(NSLOCTEXT("ActorFrustumSelect", "MarqueeSelectTransaction", "Marquee Select"));

			bool bShouldSelect = true;
			FBox SelBBox;
			CalculateBox(SelBBox);

			// TODO: with ITF Gizmos indirect interactions, this is probably not going to be triggered
			const bool bControlDown = FInputDeviceState::IsCtrlKeyDown(InputState);
			const bool bShiftDown = FInputDeviceState::IsShiftKeyDown(InputState);

			if (bControlDown)
			{
				// If control is down remove from selection
				bShouldSelect = false;
			}
			else if (!bShiftDown)
			{
				// If the user is selecting, but isn't holding down SHIFT, give modes a chance to clear selection
				ModeTools->SelectNone();
			}

			constexpr bool bSelect = true;
			// Let the editor mode try to handle the box selection.
			const bool bEditorModeHandledBoxSelection = ModeTools->BoxSelect(SelBBox, bSelect);

			// Let the component visualizers try to handle the selection.
			const bool bComponentVisHandledSelection = !bEditorModeHandledBoxSelection
													&& GUnrealEd->ComponentVisManager.HandleBoxSelect(
														SelBBox, EditorViewportClient, EditorViewportClient->Viewport
													);

			// If the edit mode didn't handle the selection, try normal actor box selection.
			if (!bEditorModeHandledBoxSelection && !bComponentVisHandledSelection)
			{
				const bool bMustEncompassEntireElement = IsWindowSelection();

				UTypedElementSelectionSet* SelectionSet = ModeTools->GetEditorSelectionSet();
				FTypedElementSelectionOptions ElementSelectionOption;
				if (!bControlDown && !bShiftDown)
				{
					// If the user is selecting, but isn't holding down SHIFT, remove all current selections from the selection set
					SelectionSet->ClearSelection(ElementSelectionOption);
				}

				FWorldSelectionElementArgs SelectionArgs{
					SelectionSet,           ETypedElementSelectionMethod::Primary,
					ElementSelectionOption, &(EditorViewportClient->EngineShowFlags),
					bMustEncompassEntireElement,   bGeometryMode
				};

				// Select all element that are within the selection box area.  Be aware that certain modes do special processing below.
				bool bSelectionChanged = false;
				UWorld* IteratorWorld = GWorld;
				const TArray<FName>& HiddenLayers = EditorViewportClientProxy->GetHiddenLayers();
				TArray<FTypedElementHandle> Handles;

				for (FActorIterator It(IteratorWorld); It; ++It)
				{
					AActor* Actor = *It;

					bool bActorIsVisible = true;
					for (auto Layer : Actor->Layers)
					{
						// Check the actor isn't in one of the layers hidden from this viewport.
						if (HiddenLayers.Contains(Layer))
						{
							bActorIsVisible = false;
							break;
						}
					}

					// Select the actor or its child elements
					if (bActorIsVisible)
					{
						Handles.Append(
							UE::EditorDragTools::Private::GetElementsIntersectingBox(
								Actor, SelBBox, EditorViewportClientProxy, SelectionArgs
							)
						);
					}
				}

				if (bShouldSelect)
				{
					SelectionSet->SelectElements(Handles, ElementSelectionOption);
				}
				else
				{
					SelectionSet->DeselectElements(Handles, ElementSelectionOption);
				}

				// Check every model to see if its BSP surfaces should be selected
				for (int32 ModelIndex = 0; ModelIndex < ModelsToCheck.Num(); ++ModelIndex)
				{
					UModel& Model = *ModelsToCheck[ModelIndex];
					// Check every node in the model
					for (int32 NodeIndex = 0; NodeIndex < Model.Nodes.Num(); NodeIndex++)
					{
						if (IntersectsBox(Model, NodeIndex, SelBBox, bMustEncompassEntireElement))
						{
							// If the node intersected the frustum select the corresponding surface
							GEditor->SelectBSPSurf(&Model, Model.Nodes[NodeIndex].iSurf, bShouldSelect, false);
							bSelectionChanged = true;
						}
					}
				}

				if (bSelectionChanged)
				{
					// If any selections were made.  Notify that now.
					GEditor->NoteSelectionChange();
				}
			}

			// Clear any hovered objects that might have been created while dragging
			FLevelEditorViewportClient::ClearHoverFromObjects();
		}
	}

	UDragToolInteraction::OnDragEnd(InReleasePos);
}

TArray<FEditorModeID> UBoxSelectInteraction::GetUnsupportedModes() const
{
	const TArray<FEditorModeID> UnsupportedModes = { FBuiltinEditorModes::EM_Landscape, FBuiltinEditorModes::EM_Foliage };
	return UnsupportedModes;
}

void UBoxSelectInteraction::CalculateBox(FBox& OutBox) const
{
	FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient();
	if (!EditorViewportClient)
	{
		return;
	}

	FSceneViewFamilyContext ViewFamily(
		FSceneViewFamily::ConstructionValues(
			EditorViewportClient->Viewport, EditorViewportClient->GetScene(), EditorViewportClient->EngineShowFlags
		)
			.SetRealtimeUpdate(EditorViewportClient->IsRealtime())
	);

	FSceneView* View = EditorViewportClient->CalcSceneView(&ViewFamily);

	FVector3f StartFloat{ Start };
	FVector3f EndFloat{ End };

	FVector4 StartScreenPos = View->PixelToScreen(StartFloat.X, StartFloat.Y, 0);
	FVector4 EndScreenPos = View->PixelToScreen(EndFloat.X, EndFloat.Y, 0);

	FVector TransformedStart = View->ScreenToWorld(View->PixelToScreen(StartFloat.X, StartFloat.Y, 0.5f));
	FVector TransformedEnd = View->ScreenToWorld(View->PixelToScreen(EndFloat.X, EndFloat.Y, 0.5f));

	// Create a bounding box based on the start/end points (normalizes the points).
	OutBox.Init();
	OutBox += TransformedStart;
	OutBox += TransformedEnd;

	switch (EditorViewportClient->ViewportType)
	{
	case LVT_OrthoXY:
	case LVT_OrthoNegativeXY:
		OutBox.Min.Z = -WORLD_MAX;
		OutBox.Max.Z = WORLD_MAX;
		break;
	case LVT_OrthoXZ:
	case LVT_OrthoNegativeXZ:
		OutBox.Min.Y = -WORLD_MAX;
		OutBox.Max.Y = WORLD_MAX;
		break;
	case LVT_OrthoYZ:
	case LVT_OrthoNegativeYZ:
		OutBox.Min.X = -WORLD_MAX;
		OutBox.Max.X = WORLD_MAX;
		break;
	case LVT_OrthoFreelook:
	case LVT_Perspective:
		break;
	}
}

bool UBoxSelectInteraction::IntersectsBox(const UModel& InModel, int32 InNodeIndex, const FBox& InBox, bool bInUseStrictSelection)
{
	FBox NodeBB;
	InModel.GetNodeBoundingBox(InModel.Nodes[InNodeIndex], NodeBB);

	bool bFullyContained = false;
	bool bIntersects = false;
	if (!bInUseStrictSelection)
	{
		bIntersects = InBox.Intersect(NodeBB);
	}
	else
	{
		bIntersects = InBox.IsInside(NodeBB.Max) && InBox.IsInside(NodeBB.Min);
	}

	return bIntersects;
}

void UBoxSelectInteraction::AddHoverEffect(AActor& InActor)
{
	const FViewportHoverTarget HoverTarget(&InActor);
	UE::EditorDragTools::Private::AddHoverEffect(HoverTarget);
}

void UBoxSelectInteraction::AddHoverEffect(UModel& InModel, int32 InSurfIndex)
{
	FViewportHoverTarget HoverTarget(&InModel, InSurfIndex);
	UE::EditorDragTools::Private::AddHoverEffect(HoverTarget);
}

void UBoxSelectInteraction::RemoveHoverEffect(AActor& InActor)
{
	FViewportHoverTarget HoverTarget(&InActor);
	UE::EditorDragTools::Private::RemoveHoverEffect(HoverTarget);
}

void UBoxSelectInteraction::RemoveHoverEffect(UModel& InModel, int32 InSurfIndex)
{
	FViewportHoverTarget HoverTarget(&InModel, InSurfIndex);
	UE::EditorDragTools::Private::RemoveHoverEffect(HoverTarget);
}
