// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDragTools/DragToolInteraction.h"
#include "ContextObjectStore.h"
#include "Editor.h"
#include "EditorDragTools/EditorViewportClientProxy.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "InputCoreTypes.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

namespace UE::Editor::DragTools
{

// CVar initializer
static int32 UseITFTools = 0;
static FAutoConsoleVariableRef CVarEnableITFTools(
	TEXT("DragTools.EnableITFTools"),
	UseITFTools,
	TEXT("Is the ITF version of Drag Tools enabled?"),
	FConsoleVariableDelegate::CreateLambda(
		[](const IConsoleVariable* InVariable)
		{
			if (UseITFTools)
			{
				OnEditorDragToolsActivated().Broadcast();
			}
			else
			{
				OnEditorDragToolsDeactivated().Broadcast();
			}
		}
	)
);

// Used to group Viewport Change behaviors inside the BehaviorSet and remove them if needed
static const FString ViewportChangeBehaviorGroup = TEXT("ViewportChange");

// CVar initializer
static int32 UseViewportChangeTool = 0;
static FAutoConsoleVariableRef CVarEnableViewportChangeTool(
	TEXT("DragTools.EnableViewportChangeTool"),
	UseViewportChangeTool,
	TEXT("Is the ITF version of the viewport change tool enabled?"),
	FConsoleVariableDelegate::CreateLambda(
		[](const IConsoleVariable* InVariable)
		{
			if (UseViewportChangeTool)
			{
				OnViewportChangeToolActivated().Broadcast();
			}
			else
			{
				OnViewportChangeToolDeactivated().Broadcast();
			}
		}
	)
);

bool UseEditorDragTools()
{
	return UseITFTools == 1;
}

FOnEditorDragToolsToggleDelegate& OnEditorDragToolsActivated()
{
	static FOnEditorDragToolsToggleDelegate OnDragToolsActivated;
	return OnDragToolsActivated;
}

FOnEditorDragToolsToggleDelegate& OnEditorDragToolsDeactivated()
{
	static FOnEditorDragToolsToggleDelegate OnDragToolsDeactivated;
	return OnDragToolsDeactivated;
}


bool IsViewportChangeToolEnabled()
{
	return UE::Editor::DragTools::UseViewportChangeTool == 1;
}

FOnOnViewportChangeToolToggleDelegate& OnViewportChangeToolActivated()
{
	static FOnOnViewportChangeToolToggleDelegate OnViewportChangedToolActivated;
	return OnViewportChangedToolActivated;
}

FOnOnViewportChangeToolToggleDelegate& OnViewportChangeToolDeactivated()
{
	static FOnOnViewportChangeToolToggleDelegate OnViewportChangedToolDeactivated;
	return OnViewportChangedToolDeactivated;
}

} // namespace UE::Editor::DragTools

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// FEditorDragTools
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UDragToolInteraction::UDragToolInteraction()
	: bConvertDelta(false)
	, Start()
	, End()
	, bUseSnapping(false)
	, bIsDragging(false)
	, EditorViewportClientProxy(nullptr)
{
}

void UDragToolInteraction::Initialize(UViewportInteractionsBehaviorSource* const InViewportInteractionsBehaviorSource)
{
	UViewportInteraction::Initialize(InViewportInteractionsBehaviorSource);

	if (const FEditorModeTools* const ModeTools = GetModeTools())
	{
		EditorViewportClientProxy = IEditorViewportClientProxy::CreateViewportClientProxy(ModeTools->GetInteractiveToolsContext());
	}
}

void UDragToolInteraction::OnStateUpdated(const FInputDeviceState& InInputDeviceState)
{
	InputState = InInputDeviceState;
}

bool UDragToolInteraction::ShouldDraw(const EViewInteractionState InInteractionState)
{
	return IsDragging() && EnumHasAllFlags(InInteractionState, EViewInteractionState::Focused);
}

void UDragToolInteraction::OnDragEnd(const FInputDeviceRay& InReleasePos)
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->ClearMouseCursorOverride();
	}

	Start = End = FVector::ZeroVector;
	bIsDragging = false;
}

void UDragToolInteraction::OnEndCapture(EEndCaptureReason InReason)
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->ClearMouseCursorOverride();
	}

	Start = End = FVector::ZeroVector;
	bIsDragging = false;

	InputState.bShiftKeyDown = false;
	InputState.bCtrlKeyDown = false;
	InputState.bAltKeyDown = false;

	// Signal that this tool is no longer active
	OnDeactivateTool().Broadcast();

	if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
	{
		EditorViewportClient->Invalidate(true, false);
	}
}
