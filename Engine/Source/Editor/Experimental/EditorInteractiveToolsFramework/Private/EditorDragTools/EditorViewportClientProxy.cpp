// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDragTools/EditorViewportClientProxy.h"
#include "EditorModeManager.h"
#include "GameFramework/Volume.h"
#include "LevelEditorViewport.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "UnrealWidget.h"

IEditorViewportClientProxy* IEditorViewportClientProxy::CreateViewportClientProxy(FEditorViewportClient* InViewportClient)
{
	if (FEditorModeTools* ModeTool = InViewportClient->GetModeTools())
	{
		return CreateViewportClientProxy(ModeTool->GetInteractiveToolsContext());
	}
	return nullptr;
}

IEditorViewportClientProxy* IEditorViewportClientProxy::CreateViewportClientProxy(UModeManagerInteractiveToolsContext* InInteractiveToolsContext)
{
	if (InInteractiveToolsContext)
	{
		if (FEditorViewportClient* EditorViewportClient = InInteractiveToolsContext->GetFocusedViewportClient())
		{
			if (EditorViewportClient->IsLevelEditorClient())
			{
				return new FLevelEditorViewportClientProxy(InInteractiveToolsContext);
			}
		}
	}

	return new FEditorViewportClientProxy(InInteractiveToolsContext);
}

FEditorViewportClientProxy::FEditorViewportClientProxy(UModeManagerInteractiveToolsContext* InInteractiveToolsContext)
    : InteractiveToolsContextWeak(InInteractiveToolsContext)
{
}

bool FEditorViewportClientProxy::IsActorVisible(const AActor* InActor) const
{
	if (InActor)
	{
		return !InActor->IsA(AVolume::StaticClass());
	}

	return false;
}

const TArray<FName> FEditorViewportClientProxy::GetHiddenLayers() const
{
	return TArray<FName>();
}

FEditorViewportClient* FEditorViewportClientProxy::GetEditorViewportClient() const
{
	if (TStrongObjectPtr<UModeManagerInteractiveToolsContext> InteractiveToolsContext = InteractiveToolsContextWeak.Pin())
	{
		return InteractiveToolsContext->GetFocusedViewportClient();
	}

	return nullptr;
}

FLevelEditorViewportClientProxy::FLevelEditorViewportClientProxy(UModeManagerInteractiveToolsContext* InInteractiveToolsContext)
	: InteractiveToolsContextWeak(InInteractiveToolsContext)
{
}

bool FLevelEditorViewportClientProxy::IsActorVisible(const AActor* InActor) const
{
	if (InActor)
	{
		if (!InActor->IsA(AVolume::StaticClass()))
		{
			return true;
		}

		if (FLevelEditorViewportClient* LevelEditorViewportClient = GetLevelEditorViewportClient())
		{
			return !LevelEditorViewportClient->IsVolumeVisibleInViewport(*InActor);
		}
	}

	return false;
}

const TArray<FName> FLevelEditorViewportClientProxy::GetHiddenLayers() const
{
	if (FLevelEditorViewportClient* LevelEditorViewportClient = GetLevelEditorViewportClient())
	{
		return LevelEditorViewportClient->ViewHiddenLayers;
	}

	return TArray<FName>();
}

FEditorViewportClient* FLevelEditorViewportClientProxy::GetEditorViewportClient() const
{
	if (TStrongObjectPtr<UModeManagerInteractiveToolsContext> InteractiveToolsContext = InteractiveToolsContextWeak.Pin())
	{
		return InteractiveToolsContext->GetFocusedViewportClient();
	}

	return nullptr;
}

FLevelEditorViewportClient* FLevelEditorViewportClientProxy::GetLevelEditorViewportClient() const
{
	return static_cast<FLevelEditorViewportClient*>(GetEditorViewportClient());
}
