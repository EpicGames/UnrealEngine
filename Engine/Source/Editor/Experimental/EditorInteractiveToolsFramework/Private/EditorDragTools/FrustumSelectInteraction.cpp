// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDragTools/FrustumSelectInteraction.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/KeyAsModifierInputBehavior.h"
#include "Components/PrimitiveComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorDragTools/EditorViewportClientProxy.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "EngineUtils.h"
#include "GameFramework/Volume.h"
#include "LevelEditorViewport.h"
#include "Model.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "SnappingUtils.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "UnrealEdGlobals.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

namespace UE::EditorDragTools
{

namespace Private
{
	TArray<FTypedElementHandle> GetElementsIntersectingFrustum(
		const AActor* Actor,
		const FConvexVolume& InFrustum,
		const IEditorViewportClientProxy* InEditorViewportClientProxy,
		const FWorldSelectionElementArgs& SelectionArgs
	)
	{
		if (InEditorViewportClientProxy && InEditorViewportClientProxy->IsActorVisible(Actor))
		{
			if (FTypedElementHandle ActorHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor))
			{
				if (TTypedElement<ITypedElementWorldInterface> WorldElement =
						UTypedElementRegistry::GetInstance()->GetElement<ITypedElementWorldInterface>(ActorHandle))
				{
					return WorldElement.GetSelectionElementsInConvexVolume(InFrustum, SelectionArgs);
				}
			}
		}

		return {};
	}
}

FViewportFrustum::FViewportFrustum(const FEditorViewportClient& InViewportClient, const FSceneView& InView,
	const FVector& InStart, const FVector& InEnd, const bool InUseBoxFrustum)
	: ViewportClient(InViewportClient)
	, View(InView)
	, Start(InStart)
	, End(InEnd) 
	, bUseBoxFrustum(InUseBoxFrustum)
{}

void FViewportFrustum::Calculate(FConvexVolume& OutFrustum) const
{
	if (bUseBoxFrustum)
	{
		const FVector& CamPoint = ViewportClient.GetViewLocation();

		// extend the 2D box of 1 pixel if needed to avoid degenerated volume 
		const double DX = Start.X == End.X ? 0.5 : 0.0;
		const double Left = FMath::Min(Start.X, End.X) - DX;
		const double Right = FMath::Max(Start.X, End.X) + DX;
		const double DY = Start.Y == End.Y ? 0.5 : 0.0;
		const double Bottom = FMath::Min(Start.Y, End.Y) - DY;
		const double Top = FMath::Max(Start.Y, End.Y) + DY;

		// Deproject the four corners of the selection box
		const FVector2D Point1(Left, Bottom); // Upper Left Corner
		const FVector2D Point2(Right, Bottom); // Upper Right Corner
		const FVector2D Point3(Right, Top); // Lower Right Corner
		const FVector2D Point4(Left, Top); // Lower Left Corner

		FVector BoxPoint1, BoxPoint2, BoxPoint3, BoxPoint4;
		FVector WorldDir1, WorldDir2, WorldDir3, WorldDir4;
		View.DeprojectFVector2D(Point1, BoxPoint1, WorldDir1);
		View.DeprojectFVector2D(Point2, BoxPoint2, WorldDir2);
		View.DeprojectFVector2D(Point3, BoxPoint3, WorldDir3);
		View.DeprojectFVector2D(Point4, BoxPoint4, WorldDir4);

		// Use the camera position and the selection box to create the bounding planes
		auto MakePlane = [](const FVector& A, const FVector& B, const FVector& C)
		{
			// the default tolerance used in GetSafeNormal (UE_SMALL_NUMBER) is not enough to handle low near plane values
			// see TPlane<T>::TPlane(TVector<T> A, TVector<T> B, TVector<T> C) 
			static constexpr double SmallNumberCubed = UE_SMALL_NUMBER * UE_SMALL_NUMBER * UE_SMALL_NUMBER;
			const FVector Normal = (B - A ^ C - A).GetSafeNormal(SmallNumberCubed);
			const double W = A | Normal;
			FPlane Plane(Normal, W);
			return MoveTemp(Plane);
		};

		FPlane TopPlane = MakePlane(BoxPoint1, BoxPoint2, CamPoint); // Top Plane
		FPlane RightPlane = MakePlane(BoxPoint2, BoxPoint3, CamPoint); // Right Plane
		FPlane BottomPlane = MakePlane(BoxPoint3, BoxPoint4, CamPoint); // Bottom Plane
		FPlane LeftPlane = MakePlane(BoxPoint4, BoxPoint1, CamPoint); // Left Plane

		// Try to get all six planes to create a frustum.
		// The frustum is built with the first four planes corresponding to the sides of the frustum.
		OutFrustum.Planes.Empty();
		OutFrustum.Planes.Emplace(MoveTemp(TopPlane));
		OutFrustum.Planes.Emplace(MoveTemp(RightPlane));
		OutFrustum.Planes.Emplace(MoveTemp(BottomPlane));
		OutFrustum.Planes.Emplace(MoveTemp(LeftPlane));

		FPlane NearPlane;
		if ( View.ViewMatrices.GetWorldToClip().GetFrustumNearPlane(NearPlane) )
		{
			OutFrustum.Planes.Emplace(MoveTemp(NearPlane));
		}

		FPlane FarPlane;
		if ( View.ViewMatrices.GetWorldToClip().GetFrustumFarPlane(FarPlane) )
		{
			OutFrustum.Planes.Emplace(MoveTemp(FarPlane));
		}

		OutFrustum.Init();
	}
	else
	{
		OutFrustum = View.ViewFrustum;
		OutFrustum.Init();
	}
}

} // namespace UE::EditorDragTools

UFrustumSelectInteraction::UFrustumSelectInteraction()
{
	InteractionName = UE::Editor::ViewportInteractions::FrustumSelect;

	if (UViewportClickDragBehavior* ClickDragBehavior = ClickDragInputBehavior.Get())
	{
		ClickDragBehavior->SetBindings({
			UE::Editor::ViewportInteractions::FButtonBinding(EKeys::LeftMouseButton).TriggersStart(),
			EKeys::LeftControl,
			EKeys::LeftAlt
		});
	}
}

FInputRayHit UFrustumSelectInteraction::CanBeginClickDragSequence(const FInputDeviceRay& InPressPos)
{
	return FInputRayHit(TNumericLimits<float>::Max()); // bHit is true. Depth is max to lose the standard tiebreaker.
}

void UFrustumSelectInteraction::OnDragStart(const FInputDeviceRay& InPressPos)
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetMouseCursorOverride(EMouseCursor::Default);
	}

	bIsDragging = true;

	// Signal that this tool is now active
	OnActivateTool().Broadcast();

	// Remove any active hover objects
	FLevelEditorViewportClient::ClearHoverFromObjects();

	const FVector2D MouseDownPosition = ClickDragInputBehavior->GetMouseDownPosition();
	Start = FVector(MouseDownPosition.X, MouseDownPosition.Y, 0);
	End = Start;
}

void UFrustumSelectInteraction::OnDrag(const FDragArgs& InDrag)
{
	End = FVector(InDrag.Ray.ScreenPosition.X, InDrag.Ray.ScreenPosition.Y, 0);
}

void UFrustumSelectInteraction::OnDragEnd(const FInputDeviceRay& InReleasePos)
{
	End = FVector(InReleasePos.ScreenPosition.X, InReleasePos.ScreenPosition.Y, 0);

	if (!EditorViewportClientProxy)
	{
		return;
	}

	FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient();
	if (!EditorViewportClient)
	{
		return;
	}

	FViewport* Viewport = EditorViewportClient->Viewport;
	if (!Viewport)
	{
		return;
	}

	FEditorModeTools* ModeTools = EditorViewportClient->GetModeTools();
	if (!ModeTools)
	{
		return;
	}

	UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>();
	const bool bGeometryMode = BrushSubsystem ? BrushSubsystem->IsGeometryEditorModeActive() : false;

	FSceneViewFamilyContext ViewFamily(
		FSceneViewFamily::ConstructionValues(Viewport, EditorViewportClient->GetScene(), EditorViewportClient->EngineShowFlags)
	);
	FSceneView* SceneView = EditorViewportClient->CalcSceneView(&ViewFamily);

	// Generate a frustum out of the dragged box
	FConvexVolume Frustum;
	CalculateFrustum(SceneView, Frustum, true);

	FScopedTransaction Transaction(NSLOCTEXT("ActorFrustumSelect", "MarqueeSelectTransaction", "Marquee Select"));

	if (!FInputDeviceState::IsShiftKeyDown(InputState))
	{
		// If the user is selecting, but isn't hold down SHIFT, remove all current selections.
		ModeTools->SelectNone();
	}

	// Does an actor have to be fully contained in the box to be selected
	const bool bMustEncompassEntireElement = IsWindowSelection();

	constexpr bool bShouldSelect = true; // bLeftMouseButtonDown
	// Let the editor mode try to handle the selection.
	const bool bEditorModeHandledSelection = ModeTools->FrustumSelect(Frustum, EditorViewportClient, bShouldSelect);

	if (bEditorModeHandledSelection)
	{
		UE_LOGF(LogITFViewportInteractions, Verbose, "Mode tools handled frustum select");
	}

	// Let the component visualizers try to handle the selection.
	const bool bComponentVisHandledSelection =
		!bEditorModeHandledSelection
		&& GUnrealEd->ComponentVisManager.HandleFrustumSelect(Frustum, EditorViewportClient, Viewport);

	if (bComponentVisHandledSelection)
	{
		UE_LOGF(LogITFViewportInteractions, Verbose, "Component visualizer handled frustum select");
	}

	if (!bEditorModeHandledSelection && !bComponentVisHandledSelection)
	{
		UTypedElementSelectionSet* SelectionSet = ModeTools->GetEditorSelectionSet();
		SelectionSet->Modify();

		FWorldSelectionElementArgs SeletionArgs{ SelectionSet,
												 ETypedElementSelectionMethod::Primary,
												 FTypedElementSelectionOptions(),
												 &(EditorViewportClient->EngineShowFlags),
												 bMustEncompassEntireElement,
												 bGeometryMode };

		const int32 ViewportSizeX = Viewport->GetSizeXY().X;
		const int32 ViewportSizeY = Viewport->GetSizeXY().Y;

		TArray<FTypedElementHandle> ElementsToSelect;
		const bool bTransparentBoxSelection = GetDefault<ULevelEditorViewportSettings>()->bTransparentBoxSelection;
		if (bTransparentBoxSelection)
		{
			// Get a list of frustum-culled actors
			for (FActorIterator It(EditorViewportClient->GetWorld()); It; ++It)
			{
				AActor* Actor = *It;
				ElementsToSelect.Append(
					UE::EditorDragTools::Private::GetElementsIntersectingFrustum(
						Actor, Frustum, EditorViewportClientProxy, SeletionArgs
					)
				);
			}
		}
		else
		{
			// Extend the endpoint of the rect to get the actual line
			
			const FVector MinVector = FVector::Min(Start, End);
			const FVector MaxVector = FVector::Max(Start, End);

			const int32 MinX = UE::LWC::FloatToIntCastChecked<int32>(FMath::Max<double>(0.0, MinVector.X));
			const int32 MinY = UE::LWC::FloatToIntCastChecked<int32>(FMath::Max<double>(0.0, MinVector.Y));
			const int32 MaxX = FMath::Min(ViewportSizeX, FMath::TruncToInt32(MaxVector.X + 1.0));
			const int32 MaxY = FMath::Min(ViewportSizeY, FMath::TruncToInt32(MaxVector.Y + 1.0));

			const FIntPoint Min{ MinX, MinY };
			const FIntPoint Max{ MaxX, MaxY };
			const FIntRect BoxRect{ Min, Max };

			// Typed Element selection
			{
				FTypedElementListRef ElementList = UTypedElementRegistry::GetInstance()->CreateElementList();
				Viewport->GetElementHandlesInRect(BoxRect, ElementList);

				if (bMustEncompassEntireElement)
				{
					ElementList->ForEachElement<ITypedElementWorldInterface>(
						[bMustEncompassEntireElement, &Frustum, &SelectionSet, &ElementsToSelect](
							const TTypedElement<ITypedElementWorldInterface>& InElement
						)
						{
							if (InElement.IsElementInConvexVolume(Frustum, bMustEncompassEntireElement))
							{
								ElementsToSelect.Add(
									SelectionSet->GetSelectionElement(InElement, ETypedElementSelectionMethod::Primary)
								);
							}

							return true;
						}
					);
				}
				else
				{
					// Grab only the selectable handles (this remove the components from the selection and select the actor instead)
					ElementList->ForEachElementHandle(
						[&SelectionSet, &ElementsToSelect](const FTypedElementHandle& InHandle)
						{
							ElementsToSelect.Add(
								SelectionSet->GetSelectionElement(InHandle, ETypedElementSelectionMethod::Primary)
							);
							return true;
						}
					);
				}
			}

			// We need this old code to support the BSP
			TSet<AActor*> BSPActors;
			TSet<UModel*> HitModels;
			Viewport->GetActorsAndModelsInHitProxy(BoxRect, BSPActors, HitModels);
			BSPActors.Empty(HitModels.Num());

			if (HitModels.Num() > 0)
			{
				// Check every model to see if its BSP surfaces should be selected
				for (auto It = HitModels.CreateConstIterator(); It; ++It)
				{
					UModel& Model = **It;
					// Check every node in the model
					for (int32 NodeIndex = 0; NodeIndex < Model.Nodes.Num(); NodeIndex++)
					{
						if (IntersectsFrustum(Model, NodeIndex, Frustum, bMustEncompassEntireElement))
						{
							uint32 SurfaceIndex = Model.Nodes[NodeIndex].iSurf;
							FBspSurf& Surf = Model.Surfs[SurfaceIndex];
							BSPActors.Add(Surf.Actor);
						}
					}
				}
			}

			if (BSPActors.Num() > 0)
			{
				for (auto It = BSPActors.CreateConstIterator(); It; ++It)
				{
					AActor* Actor = *It;
					if (bMustEncompassEntireElement)
					{
						ElementsToSelect.Append(
							UE::EditorDragTools::Private::GetElementsIntersectingFrustum(
								Actor, Frustum, EditorViewportClientProxy, SeletionArgs
							)
						);
					}
					else
					{
						ElementsToSelect.Add(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor));
					}
				}
			}
		}

		FTypedElementSelectionOptions ElementSelectionOption;

		if (ElementsToSelect.Num() > 0)
		{
			UE_LOGF(LogITFViewportInteractions, Verbose, "Hit %i things", ElementsToSelect.Num());
		}

		if (!FInputDeviceState::IsShiftKeyDown(InputState))
		{
			// If the user is selecting, but isn't hold down SHIFT, remove the previous selections.
			SelectionSet->SetSelection(MoveTemp(ElementsToSelect), ElementSelectionOption);
		}
		else
		{
			SelectionSet->SelectElements(MoveTemp(ElementsToSelect), ElementSelectionOption);
		}
	}

	// Clear any hovered objects that might have been created while dragging
	FLevelEditorViewportClient::ClearHoverFromObjects();

	UDragToolInteraction::OnDragEnd(InReleasePos);
}

TArray<FEditorModeID> UFrustumSelectInteraction::GetUnsupportedModes() const
{
	// Animation Mode has its own Frustum Select for now (input bindings customizations might change this in the future)
	const FEditorModeID EM_ControlRig(TEXT("EditMode.ControlRig"));

	const TArray<FEditorModeID> UnsupportedModes = {
		FBuiltinEditorModes::EM_Landscape,
		FBuiltinEditorModes::EM_Foliage,
		EM_ControlRig
	};

	return UnsupportedModes;
}

bool UFrustumSelectInteraction::IntersectsFrustum(
	const UModel& InModel, int32 NodeIndex, const FConvexVolume& InFrustum, bool bUseStrictSelection
)
{
	FBox NodeBB;
	// Get a bounding box of the node being checked
	InModel.GetNodeBoundingBox(InModel.Nodes[NodeIndex], NodeBB);

	bool bFullyContained = false;

	// Does the box intersect the frustum
	bool bIntersects = InFrustum.IntersectBox(NodeBB.GetCenter(), NodeBB.GetExtent(), bFullyContained);

	return bIntersects && (!bUseStrictSelection || (bUseStrictSelection && bFullyContained));
}

void UFrustumSelectInteraction::CalculateFrustum(const FSceneView* InView, FConvexVolume& OutFrustum, bool bUseBoxFrustum) const
{
	if (!InView || !EditorViewportClientProxy)
	{
		return;
	}

	FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient();
	if (!EditorViewportClient)
	{
		return;
	}

	UE::EditorDragTools::FViewportFrustum ViewportFrustum(*EditorViewportClient, *InView, Start, End, bUseBoxFrustum);
	ViewportFrustum.Calculate(OutFrustum);
}
