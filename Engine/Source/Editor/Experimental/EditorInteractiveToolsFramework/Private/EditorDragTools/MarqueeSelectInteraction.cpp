// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDragTools/MarqueeSelectInteraction.h"

#include "BaseBehaviors/ClickDragBehavior.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Settings/LevelEditorViewportSettings.h"

UMarqueeSelectInteraction::UMarqueeSelectInteraction()
{
	const FName UniqueName = MakeUniqueObjectName(GetTransientPackageAsObject(), UViewportClickDragBehavior::StaticClass(), *FString::Printf(TEXT("%s_Behavior"), *GetClass()->GetName()));
	UViewportClickDragBehavior* ClickDragBehavior = NewObject<UViewportClickDragBehavior>(GetTransientPackageAsObject(), UniqueName);
	ClickDragBehavior->Initialize(this);

	ClickDragInputBehavior = ClickDragBehavior;

	RegisterInputBehavior(ClickDragBehavior);
}

void UMarqueeSelectInteraction::Draw(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI)
{
	if (!ensure(InRenderAPI) || !ensure(InCanvas))
	{
		return;
	}

	if (!ShouldDraw(InRenderAPI->GetViewInteractionState()))
	{
		return;
	}

	if (IsWindowSelection())
	{
		FCanvasBoxItem BoxItem(
			FVector2D(Start.X, Start.Y) / InCanvas->GetDPIScale(),
			FVector2D(End.X - Start.X, End.Y - Start.Y) / InCanvas->GetDPIScale()
		);

		BoxItem.SetColor(FLinearColor::White);
		InCanvas->DrawItem(BoxItem);
	}
	else
	{
		FVector LocalStart = Start;
		FVector LocalEnd = End;

		// Place the starting corner in the upper-left
		if (LocalStart.X > LocalEnd.X)
		{
			Swap(LocalStart.X, LocalEnd.X);
		}

		if (LocalStart.Y > LocalEnd.Y)
		{
			Swap(LocalStart.Y, LocalEnd.Y);
		}

		FCanvasDashedBoxItem BoxItem(
			FVector2D(LocalStart.X, LocalStart.Y) / InCanvas->GetDPIScale(),
			FVector2D(LocalEnd.X - LocalStart.X, LocalEnd.Y - LocalStart.Y) / InCanvas->GetDPIScale()
		);

		BoxItem.DashLength = 12.0f;
		BoxItem.DashGap = 6.0f;
		BoxItem.SetColor(FLinearColor::White);

		InCanvas->DrawItem(BoxItem);
	}
}

bool UMarqueeSelectInteraction::IsWindowSelection() const
{
	switch (GetDefault<ULevelEditorViewportSettings>()->MarqueeSelectionMode)
	{
	case EMarqueeSelectionMode::Crossing:
		return false;
	case EMarqueeSelectionMode::Window:
		return true;
	case EMarqueeSelectionMode::CrossLeft:
		return Start.X < End.X;
	case EMarqueeSelectionMode::CrossRight:
		return Start.X > End.X;
	default:
		return false;
	}
}
