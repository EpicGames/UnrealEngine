// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDragTools/MeasureToolInteraction.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Editor.h"
#include "EditorDragTools/EditorViewportClientProxy.h"
#include "EditorViewportClient.h"
#include "EngineGlobals.h"
#include "SceneView.h"
#include "Settings/EditorStyleSettings.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "SnappingUtils.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UMeasureToolInteraction::UMeasureToolInteraction()
	: PixelStart()
	, PixelEnd()
{
	InteractionName = UE::Editor::ViewportInteractions::Measure;

	bUseSnapping = true;
	bConvertDelta = false;

	UViewportClickDragBehavior* MouseBehavior = NewObject<UViewportClickDragBehavior>();
	MouseBehavior->Initialize(this);
	MouseBehavior->SetBindings({
		UE::Editor::ViewportInteractions::FButtonBinding(EKeys::MiddleMouseButton).TriggersStart()
	});

	RegisterInputBehavior(MouseBehavior);
}

void UMeasureToolInteraction::Draw(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI)
{
	if (!ensure(InRenderAPI) || !ensure(InCanvas))
	{
		return;
	}

	if (!ShouldDraw(InRenderAPI->GetViewInteractionState()))
	{
		return;
	}

	FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient();
	if (!EditorViewportClient)
	{
		return;
	}

	if (!EditorViewportClient->Viewport)
	{
		return;
	}

	const float OrthoUnitsPerPixel = EditorViewportClient->GetOrthoUnitsPerPixel(EditorViewportClient->Viewport);
	const float Length =
		FMath::RoundToFloat(FVector2f{ PixelEnd - PixelStart }.Size() * OrthoUnitsPerPixel * EditorViewportClient->GetDPIScale());

	if (InCanvas && Length >= 1.0f)
	{
		const FLinearColor ToolColor = GetDefault<UEditorStyleSettings>()->ViewportToolOverlayColor;
		FCanvasLineItem LineItem(PixelStart, PixelEnd);
		LineItem.SetColor(ToolColor);
		InCanvas->DrawItem(LineItem);

		const FVector2D PixelMid = FVector2D(PixelStart + ((PixelEnd - PixelStart) / 2));

		// Calculate number of decimal places to display, based on the current viewport zoom
		float Divisor = 1.0f;
		int32 DecimalPlaces = 0;
		const float OrderOfMagnitude = FMath::LogX(10.0f, OrthoUnitsPerPixel);

		switch (GetDefault<ULevelEditorViewportSettings>()->MeasuringToolUnits)
		{
		case MeasureUnits_Meters:
			Divisor = 100.0f;
			// Max one decimal place allowed for meters.
			DecimalPlaces = FMath::Clamp(FMath::FloorToInt(1.5f - OrderOfMagnitude), 0, 1);
			break;

		case MeasureUnits_Kilometers:
			Divisor = 100000.0f;
			// Max two decimal places allowed for kilometers.
			DecimalPlaces = FMath::Clamp(FMath::FloorToInt(4.5f - OrderOfMagnitude), 0, 2);
			break;
		}

		FNumberFormattingOptions Options;
		Options.UseGrouping = false;
		Options.MinimumFractionalDigits = DecimalPlaces;
		Options.MaximumFractionalDigits = DecimalPlaces;

		const FText LengthStr = FText::AsNumber(Length / Divisor, &Options);

		FCanvasTextItem TextItem(
			FVector2D(FMath::FloorToFloat(PixelMid.X), FMath::FloorToFloat(PixelMid.Y)), LengthStr, GEngine->GetSmallFont(), ToolColor
		);
		TextItem.bCentreX = true;
		InCanvas->DrawItem(TextItem);
	}
}

FInputRayHit UMeasureToolInteraction::CanBeginClickDragSequence(const FInputDeviceRay& InPressPos)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		// Measure Tool is for Orthographic viewports only
		if (EditorViewportClient->IsOrtho())
		{
			return FInputRayHit(TNumericLimits<float>::Max());
		}
	}

	return FInputRayHit();
}

void UMeasureToolInteraction::OnDragStart(const FInputDeviceRay& InPressPos)
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetMouseCursorOverride(EMouseCursor::Default);
	}

	FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient();
	if (EditorViewportClient && EditorViewportClient->Viewport)
	{
		OnActivateTool().Broadcast();

		bIsDragging = true;

		FIntPoint MousePos;
		EditorViewportClient->Viewport->GetMousePos(MousePos);

		// Take into account DPI scale for drawing lines properly when scale is not 1.0
		Start = FVector(MousePos.X, MousePos.Y, 0) / EditorViewportClient->GetDPIScale();

		// Snap to constraints.
		if (bUseSnapping)
		{
			const float GridSize = GEditor->GetGridSize();
			const FVector GridBase(GridSize, GridSize, GridSize);
			FSnappingUtils::SnapPointToGrid(Start, GridBase);
		}

		End = Start;

		PixelStart = GetSnappedPixelPos(FVector2D(InPressPos.ScreenPosition));
		PixelEnd = PixelStart;
	}
}

void UMeasureToolInteraction::OnDrag(const FDragArgs& InDrag)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		if (EditorViewportClient->Viewport)
		{
			FIntPoint MousePos;
			EditorViewportClient->Viewport->GetMousePos(MousePos);
			PixelEnd = GetSnappedPixelPos(FVector2D(MousePos));
		}
	}
}

FVector2D UMeasureToolInteraction::GetSnappedPixelPos(FVector2D InPixelPos) const
{
	FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient();
	if (!EditorViewportClient)
	{
		return FVector2D::Zero();
	}

	FSceneViewFamilyContext ViewFamily(
		FSceneViewFamily::ConstructionValues(EditorViewportClient->Viewport, EditorViewportClient->GetScene(), EditorViewportClient->EngineShowFlags)
			.SetRealtimeUpdate(EditorViewportClient->IsRealtime())
	);

	FSceneView* View = EditorViewportClient->CalcSceneView(&ViewFamily);

	// Put the mouse pos in world space
	FVector2f PixelPosFloat{ InPixelPos };
	FVector WorldPos = View->ScreenToWorld(View->PixelToScreen(PixelPosFloat.X, PixelPosFloat.Y, 0.5f));

	// Snap the world position
	const float GridSize = GEditor->GetGridSize();
	const FVector GridBase(GridSize, GridSize, GridSize);
	FSnappingUtils::SnapPointToGrid(WorldPos, GridBase);

	// And back into pixel-space (might fail, in which case we return the original)
	View->WorldToPixel(WorldPos, InPixelPos);

	// The canvas we're going to render to factors in dpi scale to final position.
	// Since we're basing our position off mouse coordinates it will already be pixel accurate and therefore we must back out the scale the canvas will apply
	InPixelPos /= EditorViewportClient->GetDPIScale();

	return InPixelPos;
}
