// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDragTools/ViewportChangeInteraction.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "EditorViewportClient.h"
#include "Settings/EditorStyleSettings.h"
#include "SnappingUtils.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportChangeInteraction::UViewportChangeInteraction()
{
	using namespace UE::Editor::ViewportInteractions;

	InteractionName = UE::Editor::ViewportInteractions::ViewportChange;

	ViewOptionOffset = FVector2D::ZeroVector;
	bConvertDelta = false;

	UViewportClickDragBehavior* MouseBehavior = NewObject<UViewportClickDragBehavior>();
	MouseBehavior->Initialize(this);
	MouseBehavior->SetBindings({ 
		FButtonBinding(EKeys::LeftControl).RequiredToContinue(false),
		FButtonBinding(EKeys::LeftAlt).RequiredToContinue(false),
		FButtonBinding(EKeys::MiddleMouseButton).TriggersStart() 
	}
	);

	RegisterInputBehavior(MouseBehavior);
}

void UViewportChangeInteraction::Draw(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI)
{
	if (!ensure(InRenderAPI) || !ensure(InCanvas))
	{
		return;
	}

	if (!ShouldDraw(InRenderAPI->GetViewInteractionState()))
	{
		return;
	}

	FCanvasLineItem LineItem(Start, End);
	InCanvas->DrawItem(LineItem);

	const FLinearColor ToolColor = GetDefault<UEditorStyleSettings>()->ViewportToolOverlayColor;
	FCanvasTextItem TextItem(
		FVector2D(FMath::FloorToFloat(End.X), FMath::FloorToFloat(End.Y) + 20),
		GetDesiredViewportTypeText(),
		GEngine->GetMediumFont(),
		ToolColor
	);
	TextItem.bCentreX = true;
	InCanvas->DrawItem(TextItem);
}

FInputRayHit UViewportChangeInteraction::CanBeginClickDragSequence(const FInputDeviceRay& InPressPos)
{
	return FInputRayHit(TNumericLimits<float>::Max()); // bHit is true. Depth is max to lose the standard tiebreaker.
}

void UViewportChangeInteraction::OnBeginCapture(const FInputDeviceRay& InClickPressPos)
{
	if (UViewportInteractionsBehaviorSource* Source = GetViewportInteractionsBehaviorSource())
	{
		Source->SetMouseCursorOverride(EMouseCursor::Default);
	}
}

void UViewportChangeInteraction::OnDragStart(const FInputDeviceRay& InPressPos)
{
	FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient();
	if (EditorViewportClient && EditorViewportClient->Viewport)
	{
		OnActivateTool().Broadcast();

		bIsDragging = true;

		FIntPoint MousePos;
		EditorViewportClient->Viewport->GetMousePos(MousePos);

		// Take into account DPI scale for drawing lines properly when scale is not 1.0
		End = Start = FVector(MousePos.X, MousePos.Y, 0) / EditorViewportClient->GetDPIScale();
	}
}

void UViewportChangeInteraction::OnDrag(const FDragArgs& InDrag)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		FIntPoint MousePos;
		EditorViewportClient->Viewport->GetMousePos(MousePos);

		// Take into account DPI scale for drawing lines properly when scale is not 1.0
		End = FVector(MousePos) / EditorViewportClient->GetDPIScale();

		ViewOptionOffset.X = End.X - Start.X;
		ViewOptionOffset.Y = End.Y - Start.Y;
	}
}

void UViewportChangeInteraction::OnDragEnd(const FInputDeviceRay& InReleasePos)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		EditorViewportClient->SetViewportType(GetDesiredViewportType());
	}

	UDragToolInteraction::OnDragEnd(InReleasePos);
}

ELevelViewportType UViewportChangeInteraction::GetDesiredViewportType() const
{
	ELevelViewportType ViewOption = LVT_Perspective;

	if (ViewOptionOffset.Y == 0)
	{
		if (ViewOptionOffset.X == 0)
		{
			ViewOption = LVT_Perspective;
		}
		else if (ViewOptionOffset.X > 0)
		{
			ViewOption = LVT_OrthoRight; // Right
		}
		else
		{
			ViewOption = LVT_OrthoLeft; // Left
		}
	}
	else
	{
		double OffsetRatio = ViewOptionOffset.X / ViewOptionOffset.Y;
		double DragAngle = FMath::RadiansToDegrees(FMath::Atan(OffsetRatio));

		if (ViewOptionOffset.Y >= 0)
		{
			if (DragAngle >= -15.f && DragAngle <= 15.f)
			{
				ViewOption = LVT_OrthoBottom; // Bottom
			}
			else if (DragAngle > 75.f)
			{
				ViewOption = LVT_OrthoRight; // Right
			}
			else if (DragAngle < -75.f)
			{
				ViewOption = LVT_OrthoLeft; // Left
			}
		}
		else
		{
			if (DragAngle >= -15.f && DragAngle < 15.f)
			{
				ViewOption = LVT_OrthoTop; // Top
			}
			else if (DragAngle >= 15.f && DragAngle < 75.f)
			{
				ViewOption = LVT_OrthoFront; // Front
			}
			else if (DragAngle >= -75.f && DragAngle < -15.f)
			{
				ViewOption = LVT_OrthoBack; // Back
			}
			else if (DragAngle >= 75.f)
			{
				ViewOption = LVT_OrthoLeft; // Left
			}
			else if (DragAngle <= -75.f)
			{
				ViewOption = LVT_OrthoRight; // Right
			}
		}
	}

	return ViewOption;
}

FText UViewportChangeInteraction::GetDesiredViewportTypeText() const
{
	switch (GetDesiredViewportType())
	{
	case LVT_Perspective:
		return FText::FromString("Perspective");
	case LVT_OrthoFreelook:
		return FText::FromString("Free Look");
	case LVT_OrthoTop:
		return FText::FromString("Top");
	case LVT_OrthoLeft:
		return FText::FromString("Left");
	case LVT_OrthoFront:
		return FText::FromString("Front");
	case LVT_OrthoBack:
		return FText::FromString("Back");
	case LVT_OrthoBottom:
		return FText::FromString("Bottom");
	case LVT_OrthoRight:
		return FText::FromString("Right");
	default:;
	}

	return FText();
}
