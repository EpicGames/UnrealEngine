// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoDebugBase.h"

#include "BaseGizmos/GizmoElementGroup.h"
#include "BaseGizmos/GizmoUtil.h"
#include "GizmoDebugProvider.h"

UGizmoDebugBase::UGizmoDebugBase()
{
	Material = static_cast<UMaterialInterface*>(StaticLoadObject(
		UMaterialInterface::StaticClass(),
		nullptr,
		TEXT("/Engine/EditorMaterials/WidgetVertexColorMaterial.WidgetVertexColorMaterial"),
		nullptr,
		LOAD_None,
		nullptr));	
}

void UGizmoDebugBase::Draw(
	const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI,
	const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const
{
}

void UGizmoDebugBase::DrawCanvas(
	const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI,
	const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const
{
}

bool UGizmoElementDebugBase::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	if (!ensure(InElement)
		|| !ensure(InRenderAPI))
	{
		return false;
	}

	constexpr FGizmoElementAccessor Accessor;
	return Accessor.UpdateRenderState(*const_cast<UGizmoElementBase*>(InElement), InRenderAPI, InLocalCenter, InOutRenderState);
}

FLinearColor UGizmoElementDebugBase::GetElementColor(const UGizmoElementBase* InElement) const
{
	if (!ensure(InElement))
	{
		return FLinearColor::White;
	}

	switch (InElement->GetElementInteractionState())
	{
	case EGizmoElementInteractionState::None:
		return InElement->GetVertexColor();

	case EGizmoElementInteractionState::Hovering:
		return InElement->GetHoverVertexColor();

	case EGizmoElementInteractionState::Interacting:
		return InElement->GetInteractVertexColor();

	case EGizmoElementInteractionState::Selected:
		return InElement->GetSelectVertexColor();

	case EGizmoElementInteractionState::Subdued:
		return InElement->GetSubdueVertexColor();

	case EGizmoElementInteractionState::Max:
	default:
		return FLinearColor::White;
	}
}

TObjectPtr<const UMaterialInterface> UGizmoDebugBase::GetMaterial() const
{
	return Material;
}

void UGizmoElementDebugBase::Draw(
	const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI,
	const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const
{
	const UGizmoElementBase* Element = UE::Editor::InteractiveToolsFramework::Internal::GetVariantAsGizmoElement<UGizmoElementBase>(InObject);
	if (!ensure(Element))
	{
		return;
	}

	DrawElement(Element, InDebugProvider, InRenderAPI, InRenderState, InSettings);
}

void UGizmoElementDebugBase::DrawHitGeometry(
	const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI,
	const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	const UGizmoElementBase* Element = UE::Editor::InteractiveToolsFramework::Internal::GetVariantAsGizmoElement<UGizmoElementBase>(InObject);
	if (!ensure(Element))
	{
		return;
	}

	DrawElementHitGeometry(Element, InDebugProvider, InRenderAPI, InRenderState, InColor);
}

void UGizmoElementDebugBase::DrawElement(
	const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState,
	const FGizmoDebugSettings& InSettings) const
{
}

TSubclassOf<UObject> UGizmoElementGroupDebug::GetSupportedClass() const
{
	return UGizmoElementGroupBase::StaticClass();
}

void UGizmoElementGroupDebug::DrawElement(
	const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI,
	const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const
{
	if (!ensure(InElement)
		|| !ensure(InDebugProvider)
		|| !ensure(InRenderAPI))
	{
		return;
	}

	const UGizmoElementGroupBase* GroupElement = Cast<UGizmoElementGroupBase>(InElement);
	if (!ensure(GroupElement))
	{
		return;
	}

	UGizmoElementBase::FRenderTraversalState CurrentRenderState(InRenderState);
	if (UpdateRenderState(GroupElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState))
	{
		constexpr FGizmoElementAccessor Accessor;
		for (const TObjectPtr<UGizmoElementBase>& Element : Accessor.GetSubElements(*GroupElement))
		{
			if (Element && Element->GetEnabled() && Element->GetHittableState())
			{
				InDebugProvider->Draw(FGizmoDebugObjectVariant(TInPlaceType<const UGizmoElementBase*>(), Element), InRenderAPI, CurrentRenderState, InSettings);
			}
		}
	}
}

void UGizmoElementGroupDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	if (!ensure(InElement)
		|| !ensure(InDebugProvider)
		|| !ensure(InRenderAPI))
	{
		return;
	}

	const UGizmoElementGroupBase* GroupElement = Cast<UGizmoElementGroupBase>(InElement);
	if (!ensure(GroupElement))
	{
		return;
	}

	UGizmoElementBase::FRenderTraversalState CurrentRenderState(InRenderState);
	if (UpdateRenderState(GroupElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState))
	{
		constexpr FGizmoElementAccessor Accessor;
		for (const TObjectPtr<UGizmoElementBase>& Element : Accessor.GetSubElements(*GroupElement))
		{
			if (Element && Element->GetEnabled() && Element->GetHittableState())
			{
				InDebugProvider->DrawHitGeometry(FGizmoDebugObjectVariant(TInPlaceType<const UGizmoElementBase*>(), Element), InRenderAPI, CurrentRenderState, InColor);
			}
		}
	}
}

bool UGizmoElementGroupDebug::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	const UGizmoElementGroupBase* GroupElement = Cast<UGizmoElementGroupBase>(InElement);
	if (!ensure(GroupElement))
	{
		return false;
	}
	
	constexpr FGizmoElementAccessor Accessor;
	const bool bResult = Accessor.UpdateRenderState(*const_cast<UGizmoElementGroupBase*>(GroupElement), InRenderAPI, InLocalCenter, InOutRenderState);

	// Copy/Paste from UGizmoElementGroupBase::ApplyUniformConstantScaleToTransform
	auto ScaleTransform = [](double PixelToWorldScale, bool bConstantScale, FTransform& InOutLocalToWorldTransform)
	{
		double Scale = InOutLocalToWorldTransform.GetScale3D().X;
		if (bConstantScale)
		{
			Scale *= PixelToWorldScale;
		}
		InOutLocalToWorldTransform.SetScale3D(FVector(Scale, Scale, Scale));
	};

	ScaleTransform(InOutRenderState.PixelToWorldScale, GroupElement->GetConstantScale(), InOutRenderState.LocalToWorldTransform);

	return bResult;
}
