// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoElementBase.h"
#include "CoreTypes.h"
#include "EditorGizmos/EditorGizmoElementShared.h"
#include "UObject/Object.h"

#include "GizmoDebugBase.generated.h"

class UGizmoDebugProvider;
class UGizmoElementGroupBase;
class UGizmoViewContext;

/** Variant type that can hold either a gizmo element or an interactive gizmo, used as input to debug draw routines. */
using FGizmoDebugObjectVariant = TVariant<const UGizmoElementBase*, const UInteractiveGizmo*>;

namespace UE::Editor::InteractiveToolsFramework
{
	namespace Private
	{
		/**
		 * Safely extracts a derived UObject pointer from an FGizmoDebugObjectVariant.
		 * @tparam StoredType The base type stored in the variant (UGizmoElementBase or UInteractiveGizmo).
		 * @tparam DerivedType The specific derived type to cast to.
		 * @return The cast pointer, or nullptr if the variant does not hold the expected type.
		 */
		template <class StoredType, class DerivedType
		UE_REQUIRES(
			std::is_base_of_v<UObject, std::decay_t<StoredType>>
			&& std::is_base_of_v<std::decay_t<StoredType>, std::decay_t<DerivedType>>)>
		const DerivedType* GetVariantAs(const FGizmoDebugObjectVariant& InObject)
		{
			using DecayedStoredType = std::decay_t<StoredType>;
			using DecayedDerivedType = std::decay_t<DerivedType>;

			auto StoredValue = InObject.TryGet<const DecayedStoredType*>();
			if (!StoredValue || !*StoredValue)
			{
				UE_LOGF(LogTemp, Error, "GizmoDebugObjectVariant is not a %ls", *DecayedStoredType::StaticClass()->GetName());
				return nullptr;
			}

			auto TypedValue = Cast<DecayedDerivedType>(*StoredValue);
			if (!TypedValue)
			{
				UE_LOGF(LogTemp, Error, "GizmoDebugObjectVariant is not a %ls", *DecayedDerivedType::StaticClass()->GetName());
				return nullptr;
			}

			return TypedValue;
		}
	}

	namespace Internal
	{
		/** Extracts a gizmo element of the specified type from the variant. Returns nullptr on type mismatch. */
		template <class GizmoElementType UE_REQUIRES(std::is_base_of_v<UGizmoElementBase, GizmoElementType>)>
		const GizmoElementType* GetVariantAsGizmoElement(const FGizmoDebugObjectVariant& InObject)
		{
			return Private::GetVariantAs<UGizmoElementBase, GizmoElementType>(InObject);
		}

		/** Extracts an interactive gizmo of the specified type from the variant. Returns nullptr on type mismatch. */
		template <class GizmoType UE_REQUIRES(std::is_base_of_v<UInteractiveGizmo, GizmoType>)>
		const GizmoType* GetVariantAsGizmo(const FGizmoDebugObjectVariant& InObject)
		{
			return Private::GetVariantAs<UInteractiveGizmo, GizmoType>(InObject);
		}
	}
}

/**
 * This Debug object provides various Debug features for an FGizmoDebugObjectVariant, as specified by GetSupportedClass,
 */
UCLASS(Transient, Abstract, MinimalAPI)
class UGizmoDebugBase : public UObject
{
	GENERATED_BODY()

public:
	UGizmoDebugBase();

	/** The supported Class (specified in FGizmoDebugObjectVariant). */
	virtual TSubclassOf<UObject> GetSupportedClass() const PURE_VIRTUAL(UGizmoElementDebugBase::GetSupportedClass, return { };);

	/** General Draw functionality. Override specialized functions where present (ie. DrawHitGeometry). */
	virtual void Draw(const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const;

	/** General Draw functionality. Override specialized functions where present (ie. DrawHitGeometry). */
	virtual void DrawCanvas(const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const;

	/** Draws hit-testing geometry for the given gizmo object. Must be implemented by subclasses. */
	virtual void DrawHitGeometry(const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5f)) const PURE_VIRTUAL(UGizmoElementDebugBase::DrawHitGeometry, { });

protected:
	/** Returns the translucent material used for debug geometry rendering. */
	TObjectPtr<const UMaterialInterface> GetMaterial() const;

private:
	/** Translucent material instance used for rendering debug overlays. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material;
};

/** Extends UGizmoDebugBase with UGizmoElement specific functionality. */
UCLASS(Transient, Abstract, MinimalAPI)
class UGizmoElementDebugBase : public UGizmoDebugBase
{
	GENERATED_BODY()

public:
	/** General Draw functionality. Iterates element-specific debug drawing for this gizmo element. */
	virtual void Draw(const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const override;

	/** Draws hit-testing geometry for this gizmo element. */
	virtual void DrawHitGeometry(const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5f)) const override;
	
protected:
	/** Draws debug visualization for a single gizmo element. */
	virtual void DrawElement(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const;

	/** Draws hit-testing geometry for a single gizmo element. Must be implemented by subclasses. */
	virtual void DrawElementHitGeometry(const UGizmoElementBase* InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5f)) const PURE_VIRTUAL(UGizmoElementDebugBase, { });

	/** Updates the render traversal state for the given element, applying its local transform. Returns true if the element should be rendered. */
	virtual bool UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const;

	/** Returns the color for the element, based on its current state. */
	virtual FLinearColor GetElementColor(const UGizmoElementBase* InElement) const;
};

/** Debug visualizer for UGizmoElementGroupBase elements. Iterates child elements and delegates drawing to their respective debug objects. */
UCLASS(Transient, MinimalAPI)
class UGizmoElementGroupDebug
	: public UGizmoElementDebugBase
{
	GENERATED_BODY()

public:
	/** Returns UGizmoElementGroupBase as the supported class for this debug visualizer. */
	virtual TSubclassOf<UObject> GetSupportedClass() const override;

protected:
	/** Draws debug visualization for a single gizmo element. Iterates child elements and delegates to their debug objects. */
	virtual void DrawElement(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const override;

	/** Draws hit-testing geometry for a single gizmo element. Iterates child elements and delegates to their debug objects. */
	virtual void DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5f)) const override;

	/** Updates the render traversal state for the given group element. Applies group-level transform before child rendering. */
	virtual bool UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const override;
};
