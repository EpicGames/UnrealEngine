// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoDebugProvider.h"

#include "BaseGizmos/GizmoUtil.h"
#include "GizmoDebugBase.h"
#include "UObject/UObjectIterator.h"

void UGizmoDebugProvider::Setup()
{
	for (const UClass* const Class : TObjectRange<UClass>())
	{
		if (!Class->IsChildOf(UGizmoDebugBase::StaticClass())
			|| Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			continue;
		}

		if (const UGizmoDebugBase* DebugObject = Class->GetDefaultObject<UGizmoDebugBase>())
		{
			const TSubclassOf<UObject> SupportedClassForDebugObject = DebugObject->GetSupportedClass();
			if (ensure(SupportedClassForDebugObject))
			{
				DebugObjects.Emplace(SupportedClassForDebugObject, DebugObject);	
			}
		}
	}
}

void UGizmoDebugProvider::Draw(const FGizmoDebugObjectVariant& InObject, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const
{
	TObjectPtr<const UGizmoDebugBase> DebugObject;
	if (GetDebugObjectFor(InObject, DebugObject))
	{
		DebugObject->Draw(InObject, this, InRenderAPI, InRenderState, InSettings);
	}

	if (InSettings.bDrawHitTarget)
	{
		DrawHitGeometry(InObject, InRenderAPI, InRenderState);
	}
}

void UGizmoDebugProvider::DrawCanvas(
	const FGizmoDebugObjectVariant& InObject, FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState,
	const FGizmoDebugSettings& InSettings) const
{
	TObjectPtr<const UGizmoDebugBase> DebugObject;
	if (GetDebugObjectFor(InObject, DebugObject))
	{
		DebugObject->DrawCanvas(InObject, this, InCanvas, InRenderAPI, InRenderState, InSettings);
	}
}

void UGizmoDebugProvider::DrawHitGeometry(const FGizmoDebugObjectVariant& InObject, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	TObjectPtr<const UGizmoDebugBase> DebugObject;
	if (GetDebugObjectFor(InObject, DebugObject))
	{
		FLinearColor Color = FLinearColor::White;
		Color.A = HitGeometryOpacity;
		DebugObject->DrawHitGeometry(InObject, this, InRenderAPI, InRenderState, Color);
	}
}

TSubclassOf<UObject> UGizmoDebugProvider::GetDebugObjectSuperClass(const FGizmoDebugObjectVariant& InObject) const
{
	if (InObject.IsType<const UGizmoElementBase*>())
	{
		return UGizmoElementBase::StaticClass();
	}
	else if (InObject.IsType<const UInteractiveGizmo*>())
	{
		return UInteractiveGizmo::StaticClass();
	}

	return nullptr;
}

TSubclassOf<UObject> UGizmoDebugProvider::GetDebugObjectClass(const FGizmoDebugObjectVariant& InObject) const
{
	if (InObject.IsType<const UGizmoElementBase*>())
	{
		if (const UGizmoElementBase* const* GizmoElement = InObject.TryGet<const UGizmoElementBase*>();
			GizmoElement && *GizmoElement)
		{
			return (*GizmoElement)->GetClass();	
		}
	}
	else if (InObject.IsType<const UInteractiveGizmo*>())
	{
		if (const UInteractiveGizmo* const* Gizmo = InObject.TryGet<const UInteractiveGizmo*>();
			Gizmo && *Gizmo)
		{
			return (*Gizmo)->GetClass();	
		}
	}

	return nullptr;
}

bool UGizmoDebugProvider::GetDebugObjectFor(const FGizmoDebugObjectVariant& InObject, TObjectPtr<const UGizmoDebugBase>& OutDebugObject) const
{
	UClass* ObjectClass = GetDebugObjectClass(InObject);
	if (!ensure(ObjectClass))
	{
		return false;
	}

	if (const TObjectPtr<const UGizmoDebugBase>* FoundDebugObject = DebugObjects.Find(ObjectClass))
	{
		OutDebugObject = *FoundDebugObject;
		return true;
	}

	TArray<TObjectPtr<const UGizmoDebugBase>> CandidateDebugObjects;
	CandidateDebugObjects.Reserve(DebugObjects.Num());

	// If not found, find candidates that supports a parent element class
	for (const TPair<TSubclassOf<UObject>, TObjectPtr<const UGizmoDebugBase>>& DebugObjectPair : DebugObjects)
	{
		if (ObjectClass->IsChildOf(DebugObjectPair.Key))
		{
			CandidateDebugObjects.Emplace(DebugObjectPair.Value);
		}
	}

	if (CandidateDebugObjects.IsEmpty())
	{
		return false;
	}

	const UClass* SuperClass = GetDebugObjectSuperClass(InObject);
	if (!ensure(SuperClass))
	{
		return false;
	}

	// If we have candidates, use the closest match (ie. for class hierarchy A->B->C, for C, if there's a debug object for A and B, this will use B)
	TObjectPtr<const UGizmoDebugBase> BestCandidate = nullptr;
	const UClass* ParentClass = ObjectClass->GetSuperClass();
	while (BestCandidate == nullptr && ParentClass && ParentClass->IsChildOf(SuperClass))
	{
		for (const TObjectPtr<const UGizmoDebugBase>& CandidateDebugObject : CandidateDebugObjects)
		{
			if (CandidateDebugObject->GetSupportedClass() == ParentClass)
			{
				BestCandidate = CandidateDebugObject;
				break;
			}
		}

		ParentClass = ParentClass->GetSuperClass();
	}

	if (ParentClass && BestCandidate)
	{
		UE_LOGF(LogTemp, Warning, "No debug object found for Class %ls, using %ls instead", *ObjectClass->GetName(), *BestCandidate->GetClass()->GetName());

		DebugObjects.Emplace(ObjectClass, BestCandidate);
		OutDebugObject = BestCandidate;

		return true;
	}

	return false;
}
