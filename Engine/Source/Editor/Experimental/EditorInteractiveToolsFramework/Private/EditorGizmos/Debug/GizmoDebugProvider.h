// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoElementBase.h"
#include "CoreTypes.h"
#include "EditorGizmos/EditorTRSGizmo.h"
#include "GizmoDebugBase.h"
#include "UObject/Object.h"

#include "GizmoDebugProvider.generated.h"

class UGizmoDebugBase;

/**
 * Provides various Debug functionality for Gizmo Elements & Gizmos
 */
UCLASS(Transient, MinimalAPI)
class UGizmoDebugProvider
	: public UObject
{
	GENERATED_BODY()

	friend class UGizmoDebugBase;

public:
	/** Discovers and registers all available UGizmoDebugBase subclasses. Must be called before any Draw methods. */
	void Setup();

	/** Draws debug visualizations for the given gizmo object using the appropriate registered debug handler. */
	void Draw(const FGizmoDebugObjectVariant& InObject, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const;

	/** Draws canvas-based (2D) debug visualizations for the given gizmo object. */
	void DrawCanvas(const FGizmoDebugObjectVariant& InObject, FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const;

	/** Draws hit-testing geometry visualization for the given gizmo object. */
	void DrawHitGeometry(const FGizmoDebugObjectVariant& InObject, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5f)) const;

private:
	/** Returns the base UObject class stored in the variant (UGizmoElementBase or UInteractiveGizmo). */
	TSubclassOf<UObject> GetDebugObjectSuperClass(const FGizmoDebugObjectVariant& InObject) const;

	/** Returns the actual UObject class of the object stored in the variant. */
	TSubclassOf<UObject> GetDebugObjectClass(const FGizmoDebugObjectVariant& InObject) const;

	/** Looks up or creates the appropriate debug handler for the given variant. Returns true if found. */
	bool GetDebugObjectFor(const FGizmoDebugObjectVariant& InObject, TObjectPtr<const UGizmoDebugBase>& OutDebugObject) const;

private:
	/** Cache mapping UObject classes to their corresponding debug handler instances. Populated lazily. */
	UPROPERTY()
	mutable TMap<TSubclassOf<UObject>, TObjectPtr<const UGizmoDebugBase>> DebugObjects;

	/** Default opacity used when rendering hit-testing geometry overlays. */
	static constexpr float HitGeometryOpacity = 0.15f;
};
