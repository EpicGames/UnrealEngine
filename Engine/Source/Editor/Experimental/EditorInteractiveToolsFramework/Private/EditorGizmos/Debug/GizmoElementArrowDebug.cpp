// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementArrowDebug.h"

#include "BaseGizmos/GizmoElementArrow.h"

TSubclassOf<UObject> UGizmoElementArrowDebug::GetSupportedClass() const
{
	return UGizmoElementArrow::StaticClass();
}

void UGizmoElementArrowDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	if (!ensure(InRenderAPI)
		|| !ensure(InElement))
	{
		return;
	}

	if (!InElement->GetHittableState() || !InElement->GetEnabled())
	{
		return;
	}

	const UGizmoElementArrow* ArrowElement = Cast<UGizmoElementArrow>(InElement);
	if (!ensure(ArrowElement))
	{
		return;
	}

	const UMaterialInterface* Mtl = GetMaterial();
	if (!ensure(Mtl))
	{
		return;
	}

	UGizmoElementBase::FRenderTraversalState CurrentRenderState(InRenderState);
	UpdateRenderState(InElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState);

	Super::DrawElementHitGeometry(InElement, InDebugProvider, InRenderAPI, CurrentRenderState, InColor);
}

bool UGizmoElementArrowDebug::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	const UGizmoElementArrow* ArrowElement = Cast<UGizmoElementArrow>(InElement);
	if (!ensure(ArrowElement))
	{
		return false;
	}

	const FVector LocalCenter = ArrowElement->GetBase();

	return Super::UpdateRenderState(ArrowElement, InRenderAPI, LocalCenter, InOutRenderState);
}
