// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementArrowHeadDebug.h"

#include "BaseGizmos/GizmoElementArrowHead.h"

TSubclassOf<UObject> UGizmoElementArrowHeadDebug::GetSupportedClass() const
{
	return UGizmoElementArrowHead::StaticClass();
}

void UGizmoElementArrowHeadDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{	
	if (!ensure(InRenderAPI)
		|| !ensure(InElement))
	{
		return;
	}

	if (!InElement->GetHittableState() || !InElement->GetEnabled())
	{
		return;
	}

	const UGizmoElementArrowHead* ArrowHeadElement = Cast<UGizmoElementArrowHead>(InElement);
	if (!ensure(ArrowHeadElement))
	{
		return;
	}

	const UMaterialInterface* Mtl = GetMaterial();
	if (!ensure(Mtl))
	{
		return;
	}

	UGizmoElementBase::FRenderTraversalState CurrentRenderState(InRenderState);
	UpdateRenderState(ArrowHeadElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState);

	Super::DrawElementHitGeometry(InElement, InDebugProvider, InRenderAPI, CurrentRenderState, InColor);
}

bool UGizmoElementArrowHeadDebug::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	const UGizmoElementArrowHead* ArrowHeadElement = Cast<UGizmoElementArrowHead>(InElement);
	if (!ensure(ArrowHeadElement))
	{
		return false;
	}

	const FVector LocalCenter = ArrowHeadElement->GetCenter();
	return Super::UpdateRenderState(InElement, InRenderAPI, LocalCenter, InOutRenderState);
}
