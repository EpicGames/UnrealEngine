// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementBoxDebug.h"

#include "BaseGizmos/GizmoElementBox.h"
#include "BaseGizmos/GizmoUtil.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneManagement.h"

TSubclassOf<UObject> UGizmoElementBoxDebug::GetSupportedClass() const
{
	return UGizmoElementBox::StaticClass();
}

void UGizmoElementBoxDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	if (!ensure(InRenderAPI)
		|| !ensure(InElement))
	{
		return;
	}

	if (!InElement->GetHittableState() || !InElement->GetEnabled())
	{
		return;
	}

	const UGizmoElementBox* BoxElement = Cast<UGizmoElementBox>(InElement);
	if (!ensure(BoxElement))
	{
		return;
	}

	const UMaterialInterface* Mtl = GetMaterial();
	if (!ensure(Mtl))
	{
		return;
	}

	UGizmoElementBase::	FRenderTraversalState CurrentRenderState(InRenderState);
	UpdateRenderState(InElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState);

	const float PixelHitDistanceThreshold = BoxElement->GetPixelHitDistanceThreshold();
	const FVector YAxis = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(BoxElement->GetSideDirection());
	const FVector ZAxis = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(BoxElement->GetUpDirection());
	const FVector XAxis = FVector::CrossProduct(YAxis, ZAxis);
	const FVector WorldCenter = CurrentRenderState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector);
	const double Scale = CurrentRenderState.LocalToWorldTransform.GetScale3D().X;
	const double PixelHitThresholdAdjust = CurrentRenderState.PixelToWorldScale * PixelHitDistanceThreshold;
	const FVector WorldExtent = BoxElement->GetDimensions() * Scale * 0.5 + FVector(PixelHitThresholdAdjust);

	FQuat LocalRotation = FRotationMatrix::MakeFromYZ(BoxElement->GetSideDirection(), BoxElement->GetUpDirection()).ToQuat();
	FTransform RenderLocalToWorldTransform = FTransform(LocalRotation) * CurrentRenderState.LocalToWorldTransform;

	const FLinearColor Color = GetElementColor(BoxElement).CopyWithNewOpacity(InColor.A);

	const FBox Box(-WorldExtent, WorldExtent);

	DrawWireBox(
		InRenderAPI->GetPrimitiveDrawInterface(),
		RenderLocalToWorldTransform.ToMatrixNoScale(),
		Box,
		Color,
		SDPG_Foreground,
		0);

	// DrawBox(
	// 	InRenderAPI->GetPrimitiveDrawInterface(),
	// 	CurrentRenderState.LocalToWorldTransform.ToMatrixNoScale(),
	// 	WorldExtent,
	// 	Mtl->GetRenderProxy(),
	// 	SDPG_Foreground);
}

bool UGizmoElementBoxDebug::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	const UGizmoElementBox* BoxElement = Cast<UGizmoElementBox>(InElement);
	if (!ensure(BoxElement))
	{
		return false;
	}

	const FVector LocalCenter = BoxElement->GetCenter();

	constexpr FGizmoElementAccessor Accessor;
	return Accessor.UpdateRenderState(*const_cast<UGizmoElementBox*>(BoxElement), InRenderAPI, LocalCenter, InOutRenderState);
}
