// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementCircleDebug.h"

#include "BaseGizmos/GizmoElementCircle.h"
#include "BaseGizmos/GizmoUtil.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneManagement.h"

TSubclassOf<UObject> UGizmoElementCircleDebug::GetSupportedClass() const
{
	return UGizmoElementCircle::StaticClass();
}

void UGizmoElementCircleDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	if (!ensure(InRenderAPI)
		|| !ensure(InElement))
	{
		return;
	}

	if (!InElement->GetHittableState() || !InElement->GetEnabled())
	{
		return;
	}

	const UGizmoElementCircle* CircleElement = Cast<UGizmoElementCircle>(InElement);
	if (!ensure(CircleElement))
	{
		return;
	}

	const UMaterialInterface* Mtl = GetMaterial();
	if (!ensure(Mtl))
	{
		return;
	}

	UGizmoElementBase::	FRenderTraversalState CurrentRenderState(InRenderState);
	UpdateRenderState(InElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState);

	const float PixelHitDistanceThreshold = CircleElement->GetPixelHitDistanceThreshold();
	const double Radius = CircleElement->GetRadius();
	const FVector Axis0 = CircleElement->GetAxisBitangent();
	const FVector Axis1 = CircleElement->GetAxisTangent();

	const FVector WorldCenter = CurrentRenderState.LocalToWorldTransform.GetLocation();
	const FVector WorldAxis0 = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Axis0);
	const FVector WorldAxis1 = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Axis1);
	const FVector WorldUp = WorldAxis1 ^ WorldAxis0;
	const double PixelHitThresholdAdjust = CurrentRenderState.PixelToWorldScale * PixelHitDistanceThreshold;
	double WorldRadius = CurrentRenderState.LocalToWorldTransform.GetScale3D().X * Radius;

	const FColor Color = GetElementColor(CircleElement).CopyWithNewOpacity(InColor.A).ToFColor(true);

	if (CircleElement->GetHitMesh())
	{
		// DrawCircle(
		// 	InRenderAPI->GetPrimitiveDrawInterface(),
		// 	WorldCenter,
		// 	WorldAxis0,
		// 	WorldAxis1,
		// 	Color,
		// 	WorldRadius,
		// 	CircleElement->GetNumSegments(),
		// 	SDPG_Foreground,
		// 	static_cast<float>(PixelHitThresholdAdjust) * 2.0f);
		DrawWireChoppedCone(
			InRenderAPI->GetPrimitiveDrawInterface(),
			WorldCenter,
			WorldAxis0,
			WorldAxis1,
			WorldUp,
			Color,
			WorldRadius + PixelHitThresholdAdjust,
			0.0,
			1.0,
			CircleElement->GetNumSegments(),
			SDPG_Foreground);
	}
	else if (CircleElement->GetHitLine())
	{
		// DrawCircle(
		// InRenderAPI->GetPrimitiveDrawInterface(),
		// WorldCenter,
		// WorldAxis0,
		// WorldAxis1,
		// Color,
		// WorldRadius,
		// CircleElement->GetNumSegments(),
		// SDPG_World,
		// static_cast<float>(PixelHitThresholdAdjust) * 2.0f);
	}
}

bool UGizmoElementCircleDebug::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	const UGizmoElementCircle* CircleElement = Cast<UGizmoElementCircle>(InElement);
	if (!ensure(CircleElement))
	{
		return false;
	}

	const FVector LocalCenter = CircleElement->GetCenter();

	constexpr FGizmoElementAccessor Accessor;
	return Accessor.UpdateRenderState(*const_cast<UGizmoElementCircle*>(CircleElement), InRenderAPI, LocalCenter, InOutRenderState);
}
