// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GizmoDebugBase.h"

#include "GizmoElementCircleDebug.generated.h"

class UGizmoElementCircle;

/**
 * 
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementCircleDebug 
	: public UGizmoElementDebugBase
{
	GENERATED_BODY()

public:
	virtual TSubclassOf<UObject> GetSupportedClass() const override;

protected:
	virtual void DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5f)) const override;
	virtual bool UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const override;
};
