// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementConeDebug.h"

#include "BaseGizmos/GizmoElementCone.h"
#include "BaseGizmos/GizmoUtil.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneManagement.h"

TSubclassOf<UObject> UGizmoElementConeDebug::GetSupportedClass() const
{
	return UGizmoElementCone::StaticClass();
}

void UGizmoElementConeDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	if (!ensure(InRenderAPI)
		|| !ensure(InElement))
	{
		return;
	}

	if (!InElement->GetHittableState() || !InElement->GetEnabled())
	{
		return;
	}

	const UGizmoElementCone* ConeElement = Cast<UGizmoElementCone>(InElement);
	if (!ensure(ConeElement))
	{
		return;
	}

	// @todo: generalize to a "IsGeometryValid"?
	if (FMath::IsNearlyZero(ConeElement->GetRadius())
		|| FMath::IsNearlyZero(ConeElement->GetHeight()))
	{
		return;	
	}

	const UMaterialInterface* Mtl = GetMaterial();
	if (!ensure(Mtl))
	{
		return;
	}

	UGizmoElementBase::	FRenderTraversalState CurrentRenderState(InRenderState);
	UpdateRenderState(InElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState);

	// Rotate LocalToWorldTransform by the direction of the cone
	CurrentRenderState.LocalToWorldTransform.SetRotation(CurrentRenderState.LocalToWorldTransform.GetRotation() * FQuat(ConeElement->GetDirection().GetSafeNormal(), UE_PI));

	const double PixelHitThresholdAdjust = CurrentRenderState.PixelToWorldScale * InElement->GetPixelHitDistanceThreshold();
	const double ConeSide = FMath::Sqrt(ConeElement->GetHeight() * ConeElement->GetHeight() + ConeElement->GetRadius() * ConeElement->GetRadius());
	const double CosAngle = ConeElement->GetHeight() / ConeSide;
	const FVector WorldDirection = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(ConeElement->GetDirection());
	const FVector WorldOrigin = CurrentRenderState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector) - WorldDirection * PixelHitThresholdAdjust;
	const FVector XAxis = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(FVector::RightVector);
	const FVector YAxis = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(FVector::UpVector);

	const FLinearColor Color = GetElementColor(ConeElement).CopyWithNewOpacity(InColor.A);

	// TArray<FVector> Unused;
	// DrawWireCone(
	// 	InRenderAPI->GetPrimitiveDrawInterface(),
	// 	Unused,
	// 	CurrentRenderState.LocalToWorldTransform.ToMatrixNoScale(),
	// 	ConeElement->GetHeight() * CurrentRenderState.LocalToWorldTransform.GetScale3D().X + PixelHitThresholdAdjust * 2.0,
	// 	(ConeElement->GetRadius() + PixelHitThresholdAdjust * 2.0) * CosAngle,
	// 	8,
	// 	Color,
	// 	SDPG_Foreground);
	// DrawDisc(
	// 	InRenderAPI->GetPrimitiveDrawInterface(),
	// 	WorldOrigin,
	// 	XAxis,
	// 	YAxis,
	// 	Color,
	// 	ConeElement->GetRadius() * CosAngle * CurrentRenderState.LocalToWorldTransform.GetScale3D().X + PixelHitThresholdAdjust * 2.0,
	// 	ConeElement->GetNumSides(),
	// 	Mtl->GetRenderProxy(),
	// 	SDPG_World);
}

bool UGizmoElementConeDebug::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	const UGizmoElementCone* ConeElement = Cast<UGizmoElementCone>(InElement);
	if (!ensure(ConeElement))
	{
		return false;
	}

	const FVector LocalCenter = ConeElement->GetOrigin();

	constexpr FGizmoElementAccessor Accessor;
	return Accessor.UpdateRenderState(*const_cast<UGizmoElementCone*>(ConeElement), InRenderAPI, LocalCenter, InOutRenderState);
}
