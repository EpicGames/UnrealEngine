// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementCylinderDebug.h"

#include "Algo/ForEach.h"
#include "BaseGizmos/GizmoElementCylinder.h"
#include "BaseGizmos/GizmoUtil.h"
#include "DynamicMeshBuilder.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneManagement.h"

namespace UE::Editor::InteractiveToolsFramework::Private
{
	namespace GizmoElementCylinderDebugLocals
	{
		// Copy/Paste, with added Vertex Color, @see: PrimitiveDrawingUtil.h, DrawCylinder(...);
		void DrawCylinder(
			class FPrimitiveDrawInterface* PDI,
			const FVector& Start, const FVector& End, double Radius, int32 Sides, const FMaterialRenderProxy* MaterialInstance, uint8 DepthPriority,
			const FColor& InColor)
		{
			FVector Dir = End - Start;
			double Length = Dir.Size();

			if (Length > UE_SMALL_NUMBER)
			{
				FVector Z = Dir.GetUnsafeNormal();
				FVector X, Y;
				Z.GetUnsafeNormal().FindBestAxisVectors(X, Y);

				FVector Base = Z * Length*0.5 + Start;

				TArray<FDynamicMeshVertex> MeshVerts;
				TArray<uint32> MeshIndices;
				BuildCylinderVerts(Base, X, Y, Z, Radius, Length * 0.5f, Sides, MeshVerts, MeshIndices);

				Algo::ForEach(MeshVerts, [InColor](FDynamicMeshVertex& InVertex)
				{
					InVertex.Color = InColor;
				});

				FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
				MeshBuilder.AddVertices(MeshVerts);
				MeshBuilder.AddTriangles(MeshIndices);

				MeshBuilder.Draw(PDI, FMatrix::Identity, MaterialInstance, DepthPriority,0.f);
			}
		}
	}
}

TSubclassOf<UObject> UGizmoElementCylinderDebug::GetSupportedClass() const
{
	return UGizmoElementCylinder::StaticClass();
}

void UGizmoElementCylinderDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	if (!ensure(InRenderAPI)
		|| !ensure(InElement))
	{
		return;
	}

	if (!InElement->GetHittableState() || !InElement->GetEnabled())
	{
		return;
	}

	const UGizmoElementCylinder* CylinderElement = Cast<UGizmoElementCylinder>(InElement);
	if (!ensure(CylinderElement))
	{
		return;
	}

	const UMaterialInterface* Mtl = GetMaterial();
	if (!ensure(Mtl))
	{
		return;
	}

	UGizmoElementBase::	FRenderTraversalState CurrentRenderState(InRenderState);
	UpdateRenderState(InElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState);

	const float PixelHitDistanceThreshold = CylinderElement->GetPixelHitDistanceThreshold();
	const FVector Direction = CylinderElement->GetDirection();
	const float Height = CylinderElement->GetHeight();
	const float Radius = CylinderElement->GetRadius();

	const double PixelHitThresholdAdjust = CurrentRenderState.PixelToWorldScale * PixelHitDistanceThreshold;
	const double WorldHeight = Height * CurrentRenderState.LocalToWorldTransform.GetScale3D().X + PixelHitThresholdAdjust * 2.0;
	const double HalfWorldHeight = WorldHeight * 0.5;
	const double WorldRadius = Radius * CurrentRenderState.LocalToWorldTransform.GetScale3D().X + PixelHitThresholdAdjust;
	const FVector WorldDirection = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Direction);
	const FVector LocalCenter = Direction * Height * 0.5;
	const FVector WorldCenter = CurrentRenderState.LocalToWorldTransform.TransformPosition(LocalCenter);

	FVector Z = WorldDirection;
	FVector X, Y;
	Z.GetUnsafeNormal().FindBestAxisVectors(X, Y);

	const FLinearColor ColorL = GetElementColor(CylinderElement).CopyWithNewOpacity(InColor.A);
	const FColor Color = ColorL.ToFColor(true);

	DrawWireCylinder(
		InRenderAPI->GetPrimitiveDrawInterface(),
		WorldCenter,
		X,
		Y,
		Z,
		ColorL,
		WorldRadius,
		HalfWorldHeight,
		8,
		SDPG_Foreground);

	// UE::Editor::InteractiveToolsFrames::Private::GizmoElementCylinderDebugLocals::DrawCylinder(
	// 	InRenderAPI->GetPrimitiveDrawInterface(),
	// 	WorldCenter - (WorldDirection * HalfWorldHeight),
	// 	WorldCenter + (WorldDirection * HalfWorldHeight),
	// 	WorldRadius,
	// 	CylinderElement->GetNumSides(),
	// 	Mtl->GetRenderProxy(),
	// 	SDPG_World,
	// 	Color);
}

bool UGizmoElementCylinderDebug::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	const UGizmoElementCylinder* CylinderElement = Cast<UGizmoElementCylinder>(InElement);
	if (!ensure(CylinderElement))
	{
		return false;
	}

	const FVector LocalCenter = CylinderElement->GetBase();

	constexpr FGizmoElementAccessor Accessor;
	return Accessor.UpdateRenderState(*const_cast<UGizmoElementCylinder*>(CylinderElement), InRenderAPI, LocalCenter, InOutRenderState);
}
