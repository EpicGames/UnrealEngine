// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementGimbalDebug.h"

#include "BaseGizmos/GizmoUtil.h"
#include "EditorGizmos/GizmoRotationUtil.h"
#include "EditorGizmos/GizmoElementGimbal.h"
#include "GizmoDebugProvider.h"

TSubclassOf<UObject> UGizmoElementGimbalDebug::GetSupportedClass() const
{
	return UGizmoElementGimbal::StaticClass();
}

void UGizmoElementGimbalDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	if (!ensure(InRenderAPI)
		|| !ensure(InElement)
		|| !ensure(InDebugProvider))
	{
		return;
	}

	if (!InElement->GetHittableState() || !InElement->GetEnabled())
	{
		return;
	}

	const UGizmoElementGimbal* GimbalElement = Cast<UGizmoElementGimbal>(InElement);
	if (!ensure(GimbalElement))
	{
		return;
	}

	const UMaterialInterface* Mtl = GetMaterial();
	if (!ensure(Mtl))
	{
		return;
	}

	UGizmoElementBase::FRenderTraversalState CurrentRenderState(InRenderState);
	if (Super::UpdateRenderState(GimbalElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState))
	{
		using namespace UE::GizmoRotationUtil;

		// decompose rotations
		FRotationDecomposition Decomposition;
		DecomposeRotations(CurrentRenderState.LocalToWorldTransform, GimbalElement->RotationContext, Decomposition);
		
		constexpr FGizmoElementAccessor Accessor;
		int32 Index = 0;
		for (const TObjectPtr<UGizmoElementBase>& Element : Accessor.GetSubElements(*InElement))
		{
			if (Element && Element->GetEnabled() && Element->GetHittableState() && Index < UE_ARRAY_COUNT(Decomposition.R))
			{
				CurrentRenderState.LocalToWorldTransform.SetRotation(Decomposition.R[Index++]);
				InDebugProvider->DrawHitGeometry(FGizmoDebugObjectVariant(TInPlaceType<const UGizmoElementBase*>(), Element), InRenderAPI, CurrentRenderState, InColor);
			}
		}
	}
}
