// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoElementBase.h"
#include "CoreTypes.h"
#include "GizmoDebugBase.h"
#include "UObject/Object.h"

#include "GizmoElementGimbalDebug.generated.h"

class UGizmoDebugProvider;
class UGizmoElementGroupBase;
class UGizmoViewContext;

/**
 * 
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementGimbalDebug 
	: public UGizmoElementGroupDebug
{
	GENERATED_BODY()

public:
	virtual TSubclassOf<UObject> GetSupportedClass() const override;

protected:
	virtual void DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5f)) const override;
};
