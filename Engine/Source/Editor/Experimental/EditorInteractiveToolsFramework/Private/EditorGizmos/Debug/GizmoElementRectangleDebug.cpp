// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementRectangleDebug.h"

#include "BaseGizmos/GizmoElementRectangle.h"
#include "BaseGizmos/GizmoUtil.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneManagement.h"

TSubclassOf<UObject> UGizmoElementRectangleDebug::GetSupportedClass() const
{
	return UGizmoElementRectangle::StaticClass();
}

void UGizmoElementRectangleDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	if (!ensure(InRenderAPI)
		|| !ensure(InElement))
	{
		return;
	}

	if (!InElement->GetHittableState() || !InElement->GetEnabled())
	{
		return;
	}

	const UGizmoElementRectangle* RectangleElement = Cast<UGizmoElementRectangle>(InElement);
	if (!ensure(RectangleElement))
	{
		return;
	}

	const UMaterialInterface* Mtl = GetMaterial();
	if (!ensure(Mtl))
	{
		return;
	}

	UGizmoElementBase::	FRenderTraversalState CurrentRenderState(InRenderState);
	UpdateRenderState(InElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState);

	const float PixelHitDistanceThreshold = RectangleElement->GetPixelHitDistanceThreshold();
	const double Width = RectangleElement->GetWidth();
	const double Height = RectangleElement->GetHeight();
	const FVector Up = RectangleElement->GetUpDirection();
	const FVector Side = RectangleElement->GetSideDirection();
	const FVector Forward = FVector::CrossProduct(Up, Side);

	const FVector WorldCenter = CurrentRenderState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector);
	const FVector WorldUp = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Up);
	const FVector WorldSide = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Side);
	const FVector WorldForward = FVector::CrossProduct(WorldUp, WorldSide);
	const double Scale = CurrentRenderState.LocalToWorldTransform.GetScale3D().X;
	const double PixelHitThresholdAdjust = CurrentRenderState.PixelToWorldScale * PixelHitDistanceThreshold;
	const float WorldWidth = static_cast<float>(Scale * Width + PixelHitThresholdAdjust * 2.0);
	const float WorldHeight = static_cast<float>(Scale * Height + PixelHitThresholdAdjust * 2.0);
	const FVector Base = WorldCenter - WorldUp * WorldHeight * 0.5 - WorldSide * WorldWidth * 0.5;

	const FColor Color = GetElementColor(RectangleElement).CopyWithNewOpacity(InColor.A).ToFColor(true);

	if (RectangleElement->GetHitMesh())
	{
		DrawRectangle(
			InRenderAPI->GetPrimitiveDrawInterface(),
			WorldCenter,
			WorldUp,
			WorldSide,
			Color,
			WorldWidth,
			WorldHeight,
			SDPG_Foreground);
		// DrawDisc(
		// InRenderAPI->GetPrimitiveDrawInterface(),
		// WorldCenter,
		// WorldAxis0,
		// WorldAxis1,
		// Color,
		// WorldRadius,
		// RectangleElement->GetNumSegments(),
		// Mtl->GetRenderProxy(),
		// SDPG_World);
	}
	else if (RectangleElement->GetHitLine())
	{
		// DrawRectangle(
		// InRenderAPI->GetPrimitiveDrawInterface(),
		// WorldCenter,
		// WorldAxis0,
		// WorldAxis1,
		// Color,
		// WorldRadius,
		// RectangleElement->GetNumSegments(),
		// SDPG_World,
		// static_cast<float>(PixelHitThresholdAdjust) * 2.0f);
	}
}

bool UGizmoElementRectangleDebug::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	const UGizmoElementRectangle* RectangleElement = Cast<UGizmoElementRectangle>(InElement);
	if (!ensure(RectangleElement))
	{
		return false;
	}

	const FVector LocalCenter = RectangleElement->GetCenter();

	constexpr FGizmoElementAccessor Accessor;
	return Accessor.UpdateRenderState(*const_cast<UGizmoElementRectangle*>(RectangleElement), InRenderAPI, LocalCenter, InOutRenderState);
}
