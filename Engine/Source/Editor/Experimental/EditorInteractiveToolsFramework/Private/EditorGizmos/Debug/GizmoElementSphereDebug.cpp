// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementSphereDebug.h"

#include "BaseGizmos/GizmoElementSphere.h"
#include "BaseGizmos/GizmoUtil.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneManagement.h"

TSubclassOf<UObject> UGizmoElementSphereDebug::GetSupportedClass() const
{
	return UGizmoElementSphere::StaticClass();
}

void UGizmoElementSphereDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	if (!ensure(InRenderAPI)
		|| !ensure(InElement))
	{
		return;
	}

	if (!InElement->GetHittableState() || !InElement->GetEnabled())
	{
		return;
	}

	const UGizmoElementSphere* SphereElement = Cast<UGizmoElementSphere>(InElement);
	if (!ensure(SphereElement))
	{
		return;
	}

	const UMaterialInterface* Mtl = GetMaterial();
	if (!ensure(Mtl))
	{
		return;
	}

	UGizmoElementBase::	FRenderTraversalState CurrentRenderState(InRenderState);
	UpdateRenderState(InElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState);

	const FVector WorldCenter = CurrentRenderState.LocalToWorldTransform.GetTranslation();
	const double PixelHitThresholdAdjust = CurrentRenderState.PixelToWorldScale * SphereElement->GetPixelHitDistanceThreshold();
	const double Scale = CurrentRenderState.LocalToWorldTransform.GetScale3D().X;
	const double WorldExtent = Scale * SphereElement->GetRadius() + PixelHitThresholdAdjust;

	DrawSphere(
		InRenderAPI->GetPrimitiveDrawInterface(),
		WorldCenter,
		CurrentRenderState.LocalToWorldTransform.Rotator(),
		FVector(WorldExtent),
		SphereElement->GetNumSides(),
		SphereElement->GetNumSides(),
		Mtl->GetRenderProxy(),
		SDPG_World);
}

bool UGizmoElementSphereDebug::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	const UGizmoElementSphere* SphereElement = Cast<UGizmoElementSphere>(InElement);
	if (!ensure(SphereElement))
	{
		return false;
	}

	const FVector LocalCenter = SphereElement->GetCenter();

	constexpr FGizmoElementAccessor Accessor;
	return Accessor.UpdateRenderState(*const_cast<UGizmoElementSphere*>(SphereElement), InRenderAPI, LocalCenter, InOutRenderState);
}
