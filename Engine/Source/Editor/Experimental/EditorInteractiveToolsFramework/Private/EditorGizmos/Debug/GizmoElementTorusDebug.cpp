// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementTorusDebug.h"

#include "BaseGizmos/GizmoElementTorus.h"
#include "BaseGizmos/GizmoUtil.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneManagement.h"

TSubclassOf<UObject> UGizmoElementTorusDebug::GetSupportedClass() const
{
	return UGizmoElementTorus::StaticClass();
}

void UGizmoElementTorusDebug::DrawElementHitGeometry(const UGizmoElementBase* InElement, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	if (!ensure(InRenderAPI)
		|| !ensure(InElement))
	{
		return;
	}

	if (!InElement->GetHittableState() || !InElement->GetEnabled())
	{
		return;
	}

	const UGizmoElementTorus* TorusElement = Cast<UGizmoElementTorus>(InElement);
	if (!ensure(TorusElement))
	{
		return;
	}

	const UMaterialInterface* Mtl = GetMaterial();
	if (!ensure(Mtl))
	{
		return;
	}

	UGizmoElementBase::FRenderTraversalState CurrentRenderState(InRenderState);
	UpdateRenderState(InElement, InRenderAPI, FVector::ZeroVector, CurrentRenderState);

	const double Radius = TorusElement->GetRadius() * CurrentRenderState.LocalToWorldTransform.GetScale3D().X;
	const double InnerRadius = TorusElement->GetInnerRadius() * CurrentRenderState.LocalToWorldTransform.GetScale3D().X;
	const FVector Axis0 = TorusElement->GetAxisBitangent();
	const FVector Axis1 = TorusElement->GetAxisTangent();
	const FVector Normal = Axis0 ^ Axis1;
	const double PartialStartAngle = TorusElement->GetPartialStartAngle();
	const double PartialEndAngle = TorusElement->GetPartialEndAngle();

	constexpr FGizmoElementAccessor Accessor;
	const bool bIsPartial = Accessor.IsPartial(
		*const_cast<UGizmoElementTorus*>(TorusElement),
		CurrentRenderState.LocalToWorldTransform.GetLocation(),
		CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Normal),
		InRenderAPI->GetCameraState().Position,
		InRenderAPI->GetCameraState().Forward(),
		InRenderAPI->GetSceneView()->IsPerspectiveProjection());

	const FVector BeginAxis = bIsPartial ? Axis0.RotateAngleAxis(PartialStartAngle, Normal).GetSafeNormal() : Axis0;

	float PartialAngle = static_cast<float>(PartialEndAngle - PartialStartAngle);
	if (PartialAngle <= 0.0f)
	{
		return;
	}

	const FVector SideAxis = (Normal ^ BeginAxis).GetSafeNormal();

	const double PixelHitThresholdAdjust = CurrentRenderState.PixelToWorldScale * TorusElement->GetPixelHitDistanceThreshold();

	const FColor Color = CurrentRenderState.GetCurrentVertexColor().CopyWithNewOpacity(InColor.A).ToFColor(true);

	DrawTorus(
		InRenderAPI->GetPrimitiveDrawInterface(),
		CurrentRenderState.LocalToWorldTransform.ToMatrixNoScale(),
		BeginAxis,
		SideAxis,
		Color,
		Radius,
		InnerRadius + PixelHitThresholdAdjust,
		TorusElement->GetNumSegments(),
		TorusElement->GetNumInnerSlices(),  
		Mtl->GetRenderProxy(),
		SDPG_Foreground,
		bIsPartial,
		PartialAngle,
		true);
}

bool UGizmoElementTorusDebug::UpdateRenderState(const UGizmoElementBase* InElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	const UGizmoElementTorus* TorusElement = Cast<UGizmoElementTorus>(InElement);
	if (!ensure(TorusElement))
	{
		return false;
	}

	const FVector LocalCenter = TorusElement->GetCenter();

	constexpr FGizmoElementAccessor Accessor;
	return Accessor.UpdateRenderState(*const_cast<UGizmoElementTorus*>(TorusElement), InRenderAPI, LocalCenter, InOutRenderState);
}
