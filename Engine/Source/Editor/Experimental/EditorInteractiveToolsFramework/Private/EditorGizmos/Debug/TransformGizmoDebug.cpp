// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CanvasItem.h: Unreal canvas item definitions
=============================================================================*/

#include "TransformGizmoDebug.h"

#include "BaseGizmos/GizmoMath.h"
#include "CanvasTypes.h"
#include "EditorGizmos/EditorGizmoMath.h"
#include "EditorGizmos/EditorTRSGizmo.h"
#include "EditorGizmos/GizmoElementGimbal.h"
#include "EditorGizmos/GizmoElementRotateAxis.h"
#include "EditorGizmos/GizmoElementRotateAxisSet.h"
#include "EditorGizmos/GizmoElementScaleGroup.h"
#include "EditorGizmos/GizmoElementTranslateGroup.h"
#include "EditorGizmos/TransformGizmo.h"
#include "EditorToolDataVisualizer.h"
#include "Engine/Engine.h"
#include "GizmoDebugProvider.h"
#include "GlobalRenderResources.h"
#include "ToolDataVisualizer.h"

TSubclassOf<UObject> UTransformGizmoDebug::GetSupportedClass() const
{
	return UTransformGizmo::StaticClass();
}

void UTransformGizmoDebug::Draw(
	const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI,
	const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const
{
	const UEditorTRSGizmo* TransformGizmo = UE::Editor::InteractiveToolsFramework::Internal::GetVariantAsGizmo<UEditorTRSGizmo>(InObject);
	if (!ensure(TransformGizmo)
		|| !ensure(InDebugProvider)
		|| !ensure(InRenderAPI))
	{
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::ToolDataVisualizer;

	static constexpr float ScreenSpaceHandleRadius = 5.0f;

	const FVector CameraFacingNormal = -InRenderAPI->GetSceneView()->GetViewDirection();

	static const FLinearColor ActorStartColor = FLinearColor::Blue.CopyWithNewOpacity(0.5f);
	static const FLinearColor ActorCurrentColor = FLinearColor(0.0f, 1.0f, 1.0f).CopyWithNewOpacity(0.5f);

	static const FLinearColor InteractionStartColor = FLinearColor::Red.CopyWithNewOpacity(0.5f);
	static const FLinearColor InteractionCurrentColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5f);

	if (InRenderAPI)
	{
		const uint32 HitPart = static_cast<uint32>(TransformGizmo->LastHitPart);
		ETransformGizmoPartIdentifier ModeHitPartId = TransformGizmo->GetCurrentModeLastHitPart();

		switch (TransformGizmo->CurrentMode)
		{
			case EGizmoTransformMode::Translate:
			{
				if (TransformGizmo->TranslateGroupElement)
				{
					TransformGizmo->TranslateGroupElement->DrawDebug(InRenderAPI, InSettings, HitPart);
				}
			}
			break;

		case EGizmoTransformMode::Rotate:
			{
				if (TransformGizmo->RotateAxisSetElement)
				{
					TransformGizmo->RotateAxisSetElement->DrawDebug(InRenderAPI, InSettings, HitPart);
				}

				if (TransformGizmo->RotateGimbalElement && TransformGizmo->bGimbalRotationMode)
				{
					TransformGizmo->RotateGimbalElement->GetAxisElement<UGizmoElementRotateAxis>(EAxis::X)->DrawDebug(InRenderAPI, InSettings);
					TransformGizmo->RotateGimbalElement->GetAxisElement<UGizmoElementRotateAxis>(EAxis::Y)->DrawDebug(InRenderAPI, InSettings);
					TransformGizmo->RotateGimbalElement->GetAxisElement<UGizmoElementRotateAxis>(EAxis::Z)->DrawDebug(InRenderAPI, InSettings);
				}

				if (TransformGizmo->RotateScreenSpaceElement2 && ModeHitPartId == ETransformGizmoPartIdentifier::RotateScreenSpace)
				{
					TransformGizmo->RotateScreenSpaceElement2->DrawDebug(InRenderAPI, InSettings);
				}
			}
			break;

		case EGizmoTransformMode::Scale:
			{
				if (TransformGizmo->ScaleGroupElement)
				{
					TransformGizmo->ScaleGroupElement->DrawDebug(InRenderAPI, InSettings, HitPart);
				}
			}
			break;

		case EGizmoTransformMode::Max:
		case EGizmoTransformMode::None:
		default:
			break;
		}

		// Draw.DrawGrid(
		// 	InteractionPlanarOrigin,
		// 	InteractionPlanarNormal,
		// 	FVector::OneVector * 100,
		// 	FLinearColor::Gray,
		// 	1, false);

		FToolDataVisualizer Draw;
		Draw.BeginFrame(InRenderAPI);
		Draw.bDepthTested = false;
		Draw.PointSize = 8.0f;

		// Homography Matrix
		#if (0)
		{
			const double Length = 100.0;
			const FVector PlanePointBottomLeft = TransformGizmo->AlignedInteractionPlane.Origin;
			const FVector PlanePointBottomRight = TransformGizmo->AlignedInteractionPlane.Origin + TransformGizmo->AlignedInteractionPlane.GetAxis(1) * Length;
			const FVector PlanePointTopLeft = TransformGizmo->AlignedInteractionPlane.Origin + TransformGizmo->AlignedInteractionPlane.GetAxis(2) * Length;
			const FVector PlanePointTopRight = TransformGizmo->AlignedInteractionPlane.Origin + TransformGizmo->AlignedInteractionPlane.GetAxis(1) * Length + TransformGizmo->AlignedInteractionPlane.GetAxis(2) * Length;

			Draw.DrawPoint(PlanePointBottomLeft, FLinearColor::White);
			Draw.DrawPoint(PlanePointTopLeft, FLinearColor::Red);
			Draw.DrawPoint(PlanePointTopRight, FLinearColor::Green);
			Draw.DrawPoint(PlanePointBottomRight, FLinearColor::Blue);
		}
		#endif

		if (InSettings.bDrawCursorRay)
		{
			//Draw.DrawPoint(TransformGizmo->DebugData.InteractionRay.Origin + TransformGizmo->DebugData.InteractionRay.Direction * 100.0f, InteractionCurrentColor, 8.0f, false);
		}

		auto DrawAxis = [&](const FVector& InDirection, const FLinearColor& InColor)
		{
			Draw.DrawDirectionalArrow(
				TransformGizmo->InteractionPlanarOrigin,
				TransformGizmo->InteractionPlanarOrigin + (InDirection * 100),
				InDirection,
				InColor,
				25.0f,
				0.0f,
				false);
		};

		// Can be used to scale other visualizations to encompass the cursor
		const double OriginCursorDistance = FVector::Distance(TransformGizmo->InteractionPlanarOrigin, TransformGizmo->InteractionPlanarCurrPoint);

		// Screen Plane
		if (InSettings.bDrawScreenPlane)
		{
			FVector Normal = TransformGizmo->ScreenPlane.GetAxis(0);
			FVector Side = TransformGizmo->ScreenPlane.GetAxis(1);
			FVector Up = TransformGizmo->ScreenPlane.GetAxis(2);
			
			// Plane Normal
			DrawAxis(Normal, FLinearColor::Red);

			// Plane Side
			DrawAxis(Side, FLinearColor::Green);

			// Plane Up
			DrawAxis(Up, FLinearColor::Blue);

			Draw.DrawGrid(
				TransformGizmo->InteractionPlanarOrigin,
				Normal,
				Side,
				Up,
				FVector::OneVector * 2.5 * OriginCursorDistance,
				FLinearColor::Green,
				1, false);
		}

		// Interaction Plane
		if (InSettings.bDrawInteractionPlane)
		{
			// Plane Normal
			DrawAxis(TransformGizmo->InteractionPlanarNormal, FLinearColor::Red);

			// Plane Side
			DrawAxis(TransformGizmo->InteractionPlanarAxisX, FLinearColor::Green);

			// Plane Up
			DrawAxis(TransformGizmo->InteractionPlanarAxisY, FLinearColor::Blue);

			Draw.DrawGrid(
				TransformGizmo->InteractionPlanarOrigin,
				TransformGizmo->InteractionPlanarNormal,
				TransformGizmo->InteractionPlanarAxisX,
				TransformGizmo->InteractionPlanarAxisY,
				FVector::OneVector * 2.5 * OriginCursorDistance,
				TransformGizmo->Style.InteractColor,
				1, false);
		}

		// Aligned Interaction Plane
		if (InSettings.bDrawAlignedInteractionPlane)
		{
			FVector Normal = TransformGizmo->AlignedInteractionPlane.GetAxis(0);
			FVector Side = TransformGizmo->AlignedInteractionPlane.GetAxis(1);
			FVector Up = TransformGizmo->AlignedInteractionPlane.GetAxis(2);

			// Plane Normal
			DrawAxis(Normal, FLinearColor::Red);

			// Plane Side
			DrawAxis(Side, FLinearColor::Green);

			// Plane Up
			DrawAxis(Up, FLinearColor::Blue);

			Draw.DrawGrid(
				TransformGizmo->InteractionPlanarOrigin,
				Normal,
				Side,
				Up,
				FVector::OneVector * 2.5 * OriginCursorDistance,
				FLinearColor::Gray,
				1, false);
		}

		// Current Transform
		if (InSettings.bDrawLocalTransform)
		{
			DrawAxis(TransformGizmo->CurrentTransform.GetUnitAxis(EAxis::X), FLinearColor::Red);
			DrawAxis(TransformGizmo->CurrentTransform.GetUnitAxis(EAxis::Y), FLinearColor::Green);
			DrawAxis(TransformGizmo->CurrentTransform.GetUnitAxis(EAxis::Z), FLinearColor::Blue);
		}

		// Element Transform
		#if (0)
		{
			DrawAxis(TransformGizmo->AlignedInteractionPlane.GetAxis(0), FLinearColor::Red);
			DrawAxis(TransformGizmo->AlignedInteractionPlane.GetAxis(1), FLinearColor::Green);
			DrawAxis(TransformGizmo->AlignedInteractionPlane.GetAxis(2), FLinearColor::Blue);
		}
		#endif

		// Cursor
		if (InSettings.bDrawCursor)
		{
			Draw.DrawPoint(TransformGizmo->DebugData.Test3, FLinearColor::Blue, 8.0f, false);
		}

		// Cursor
		if (InSettings.bDrawCursorRay)
		{
			Draw.DrawPoint(TransformGizmo->InteractionPlanarCurrPoint, InteractionCurrentColor, 8.0f, false);
			Draw.DrawCircle(TransformGizmo->DebugData.InteractionStart.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionStartColor, 0.0f, false);
			// Draw.DrawCircle(InteractionStartOvershoot, CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionStartColor, 0.0f, false);
			// Draw.DrawCircle(TransformGizmo->DebugData.InteractionCurrent.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionCurrentColor, 0.0f, false);
		}

		switch (TransformGizmo->CurrentMode)
		{
			case EGizmoTransformMode::Translate:
			{
				// Cursor
				if (InSettings.bDrawCursor)
				{
					Draw.DrawCircle(TransformGizmo->DebugData.InteractionStart.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionStartColor, 0.0f, false);
                	Draw.DrawCircle(TransformGizmo->DebugData.InteractionCurrent.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionCurrentColor, 0.0f, false);
                	Draw.DrawLine(TransformGizmo->DebugData.InteractionStart.GetLocation(), TransformGizmo->DebugData.InteractionCurrent.GetLocation(), InteractionCurrentColor, 0.0f, false);		
				}

				// Actual Object
				if (InSettings.bDrawInputDelta)
				{
					Draw.DrawCircle(TransformGizmo->DebugData.TransformStart.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, ActorStartColor, 0.0f, false);
					Draw.DrawCircle(TransformGizmo->CurrentTransform.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, ActorCurrentColor, 0.0f, false);
					Draw.DrawLine(TransformGizmo->DebugData.TransformStart.GetLocation(), TransformGizmo->CurrentTransform.GetLocation(), ActorCurrentColor, 0.0f, false);
				}

				// Snapping
				if (InSettings.bDrawSnapping)
				{
					const FToolContextSnappingConfiguration SnappingSettings = TransformGizmo->GetGizmoManager()->GetContextQueriesAPI()->GetCurrentSnappingSettings();
					if (SnappingSettings.IsRotationGridSnappingActive())
					{
						FVector Normal = TransformGizmo->InteractionPlanarNormal;
						FVector Side = TransformGizmo->InteractionPlanarAxisX;
						FVector Up = TransformGizmo->InteractionPlanarAxisY;

						if (TransformGizmo->GetCurrentPartCoordinateSystem() == EToolContextCoordinateSystem::Screen)
						{
							Normal = -TransformGizmo->GizmoViewContext->GetViewDirection();
							Side = TransformGizmo->GizmoViewContext->GetViewRight();
							Up = TransformGizmo->GizmoViewContext->GetViewUp();
						}

						const double StartAngleRad = TransformGizmo->InteractionStartAngle;
						const double RadialSpacingRad = FMath::DegreesToRadians(SnappingSettings.RotationGridAngles.Pitch);
						const double Radius = InRenderState.PixelToWorldScale * TransformGizmo->Style.RotateStyle.AxisStyle.Radius * TransformGizmo->Style.RotateStyle.AxisStyle.RadiusMultiplier * TransformGizmo->GetSizeCoefficient();;

						const int32 NumSpokePoints = FMath::FloorToInt32(UE_DOUBLE_TWO_PI / RadialSpacingRad) + 1;

						TArray<FVector> SpokePoints;
						SpokePoints.Reserve(NumSpokePoints);

						for (int32 SpokeIndex = 0; SpokeIndex < NumSpokePoints; ++SpokeIndex)
						{
							const FVector SpokeOffset =
								(Side * FMath::Cos(StartAngleRad + (RadialSpacingRad * SpokeIndex))
								+ Up * FMath::Sin(StartAngleRad + (RadialSpacingRad * SpokeIndex)));

							const FVector SpokeStartPoint = TransformGizmo->InteractionPlanarOrigin + (SpokeOffset * Radius * 0.75f);
							const FVector SpokeEndPoint = TransformGizmo->InteractionPlanarOrigin + SpokeOffset * Radius;

							Draw.DrawLine(
								SpokeStartPoint,
								SpokeEndPoint,
								FLinearColor::White,
								0.0f,
								false);
						}
					}
				}
			}
			break;

			case EGizmoTransformMode::Scale:
			{
				const FVector TransformStartLocation = TransformGizmo->DebugData.TransformStart.GetLocation();
					
				// const float InteractionRadius = static_cast<float>(FVector::Distance(TransformStartLocation, TransformGizmo->DebugData.InteractionCurrent.GetLocation()));
				const float InteractionRadius = static_cast<float>(FVector::Distance(TransformStartLocation, TransformGizmo->DebugData.InteractionCurrent.GetLocation()));

				const float StartRadius = static_cast<float>(FMath::Max(
				UE_DOUBLE_KINDA_SMALL_NUMBER,
				FVector::Distance(TransformGizmo->InteractionPlanarStartPoint, TransformGizmo->InteractionPlanarOrigin)));

				const float CurrentRadius = static_cast<float>(FMath::Max(
					UE_DOUBLE_KINDA_SMALL_NUMBER,
					FVector::Distance(TransformGizmo->InteractionPlanarCurrPoint, TransformGizmo->InteractionPlanarOrigin)));

				FVector StartNormal = (TransformGizmo->DebugData.InteractionStart.GetLocation() - TransformStartLocation).GetSafeNormal();
				FVector InteractionStartOvershoot = TransformStartLocation + (StartNormal * InteractionRadius);

				// Draw.DrawLine(TransformStartLocation, )

				// Cursor
				if (InSettings.bDrawCursor)
				{
					Draw.DrawCircle(TransformGizmo->DebugData.InteractionStart.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionStartColor, 0.0f, false);
					// Draw.DrawCircle(InteractionStartOvershoot, CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionStartColor, 0.0f, false);
					Draw.DrawCircle(TransformGizmo->DebugData.InteractionCurrent.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionCurrentColor, 0.0f, false);

					Draw.DrawPoint(TransformGizmo->DebugData.Test, FLinearColor::Green, 40.0f, false);
				}

				if (InSettings.bDrawInputDelta)
				{
					// Draw.DrawDashedCircle()
					Draw.DrawCircle(TransformStartLocation, TransformGizmo->DebugData.InteractionPlaneNormal, StartRadius, 128, InteractionStartColor, 0.0f, false);
					Draw.DrawCircle(TransformStartLocation, TransformGizmo->DebugData.InteractionPlaneNormal, CurrentRadius, 128, InteractionCurrentColor, 0.0f, false);

					// Draw.DrawLine(TransformStartLocation, TransformStartLocation + (TransformGizmo->DebugData.Test * StartRadius), InteractionStartColor, 4.0f, false);

					// Draw.DrawLine(TransformGizmo->DebugData.ReferencePoint, InteractionPlanarCurrPoint, FLinearColor::Green, 4.0f, false);
					Draw.DrawLine(TransformStartLocation, TransformStartLocation + (TransformGizmo->InteractionBidirection * StartRadius), FLinearColor::Green, 2.0f, false);

					if (TransformGizmo->Interaction.ScaleInteraction.DistanceType == EGizmoElementScaleDistanceType::Directional)
					{
						// GizmoMath::NearestPointOnLine(TransformStartLocation)
						// Draw.DrawDirectionalArrow(TransformStartLocation, TransformGizmo->DebugData.InteractionCurrent.GetLocation(), CameraFacingNormal, InteractionCurrentColor, 24.0f, 0.0f, false);
						// Draw.DrawCircle(TransformStartLocation, TransformGizmo->DebugData.InteractionPlaneNormal, InteractionRadius, 128, InteractionCurrentColor, 0.0f, false);
					}
					else
					{
						// Draw.DrawCircle(TransformStartLocation, TransformGizmo->DebugData.InteractionPlaneNormal, InteractionRadius * 2.0f, 128, InteractionCurrentColor, 0.0f, false);
					}

					// Draw.DrawLine(TransformStartLocation, TransformGizmo->DebugData.InteractionCurrent.GetLocation(), InteractionStartColor.CopyWithNewOpacity(0.5f), 0.0f, false);
					// Draw.DrawDirectionalArrow(TransformStartLocation, TransformGizmo->DebugData.InteractionCurrent.GetLocation(), CameraFacingNormal, InteractionCurrentColor, 24.0f, 0.0f, false);
				}

				if (InSettings.bDrawInputCorrespondence)
				{
					// Draw 50%, 200% radius
					Draw.DrawCircle(TransformStartLocation, TransformGizmo->DebugData.InteractionPlaneNormal, StartRadius * 0.5f, 128, FLinearColor::White, 0.0f, false);
					Draw.DrawCircle(TransformStartLocation, TransformGizmo->DebugData.InteractionPlaneNormal, CurrentRadius * 2.0f, 128, FLinearColor::White, 0.0f, false);

					Draw.DrawPoint(TransformGizmo->InteractionPlanarStartPoint, FLinearColor::Blue, 20.0f, false);
					Draw.DrawPoint(TransformGizmo->DebugData.ReferencePoint, FLinearColor::Green, 20.0f, false);
				}

				// Snapping
				if (InSettings.bDrawSnapping)
				{
					const FToolContextSnappingConfiguration SnappingSettings = TransformGizmo->GetGizmoManager()->GetContextQueriesAPI()->GetCurrentSnappingSettings();
					if (SnappingSettings.IsPositionGridSnappingActive())
					{
						FVector Normal = TransformGizmo->InteractionPlanarNormal;
						FVector Side = TransformGizmo->InteractionPlanarAxisX;
						FVector Up = TransformGizmo->InteractionPlanarAxisY;

						if (TransformGizmo->GetCurrentPartCoordinateSystem() == EToolContextCoordinateSystem::Screen)
						{
							Normal = -TransformGizmo->GizmoViewContext->GetViewDirection();
							Side = TransformGizmo->GizmoViewContext->GetViewRight();
							Up = TransformGizmo->GizmoViewContext->GetViewUp();
						}

						constexpr int32 GridDivisions = 16; // matches FToolDataVisualizer
						const FVector ObjectSize = TransformGizmo->StartTransform.GetScale3D();
						const double NormalizeLength = TransformGizmo->InteractionDeltaDivisor;
						const double GizmoSize = InRenderState.PixelToWorldScale;
						const FVector GridSize = GizmoSize * NormalizeLength * ObjectSize * static_cast<double>(SnappingSettings.ScaleGridSize) * GridDivisions;

						Draw.DrawGrid(
							TransformGizmo->InteractionPlanarStartPoint,
							Normal,
							Up,
							Side,
							GridSize,
							FLinearColor::White,
							1,
							false);
					}
				}
			}
			break;

			case EGizmoTransformMode::Max:
			case EGizmoTransformMode::None:
			default:
				break;
		}

		Draw.EndFrame();
	}
}

void UTransformGizmoDebug::DrawCanvas(
	const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI,
	const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const
{
	const UEditorTRSGizmo* TransformGizmo = UE::Editor::InteractiveToolsFramework::Internal::GetVariantAsGizmo<UEditorTRSGizmo>(InObject);
	if (!ensure(TransformGizmo)
		|| !ensure(InCanvas)
		|| !ensure(InDebugProvider)
		|| !ensure(InRenderAPI))
	{
		return;
	}
	
	using namespace UE::Editor::InteractiveToolsFramework::ToolDataVisualizer;

	static const FLinearColor InteractionStartColor = FLinearColor::Red.CopyWithNewOpacity(0.5f);
	static const FLinearColor InteractionCurrentColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5f);

	FIntRect ViewRectI = InCanvas->GetViewRect();
	FBox2D ViewRect = FBox2D(FVector2D(ViewRectI.Min), FVector2D(ViewRectI.Max));
	FVector2D ViewSize = ViewRect.Max - ViewRect.Min;

	constexpr float LabelPadding = 8.0f;
	constexpr float LabelFontHeight = 20.0f;

	int32 RowCount = 1;
	if (InSettings.bDrawTiming)
	{
		RowCount += 10;
	}

	auto DrawText = [&ViewSize, InCanvas](const FString& InString, const int32 RowOffset = 0, const FLinearColor& InColor = FLinearColor::Yellow)
	{
		FCanvasTextItem TextItem(
			FVector2D(LabelPadding * 2.0f, ViewSize.Y - ((LabelPadding + LabelFontHeight) * static_cast<float>(RowOffset + 1))),
			FText::FromString(InString),
			GEngine->GetMediumFont(),
			InColor);

		TextItem.EnableShadow(FLinearColor::Black, FVector2D(1.0f, 1.0f));
		TextItem.Draw(InCanvas);
	};

	constexpr float TextBackgroundBoxWidth = 300.0f;
	const float TextBackgroundBoxHeight = (LabelFontHeight + LabelPadding) * (LabelPadding + static_cast<float>(RowCount));

	FCanvasTileItem TextBackgroundBox(
		FVector2D(LabelPadding, ViewSize.Y - TextBackgroundBoxHeight - LabelPadding),
		GWhiteTexture,
		FVector2D(TextBackgroundBoxWidth, TextBackgroundBoxHeight),
		FLinearColor::Black.CopyWithNewOpacity(0.3f));

	TextBackgroundBox.BlendMode = SE_BLEND_Translucent;
	TextBackgroundBox.Draw(InCanvas);

	if (!TransformGizmo->DebugData.DebugString.IsEmpty())
	{
		DrawText(TransformGizmo->DebugData.DebugString, 0);
	}

	if (InSettings.bDrawCursor)
	{
		constexpr bool bDrawCursorCrosshairs = false;
		if (bDrawCursorCrosshairs)
		{
			// X
			{
				FVector2D Start(TransformGizmo->DebugData.InteractionScreenPos.X, 0.0f);
				FVector2D End(TransformGizmo->DebugData.InteractionScreenPos.X, 1.0f * ViewSize.Y);

				FCanvasLineItem LineItem(Start, End);
				LineItem.SetColor(InteractionCurrentColor);
				LineItem.LineThickness = 1.0f;

				InCanvas->DrawItem(LineItem);
			}

			// Y
			{
				FVector2D Start(0.0f, TransformGizmo->DebugData.InteractionScreenPos.Y);
				FVector2D End(1.0f * ViewSize.X,TransformGizmo->DebugData.InteractionScreenPos.Y);

				FCanvasLineItem LineItem(Start, End);
				LineItem.SetColor(InteractionCurrentColor);
				LineItem.LineThickness = 1.0f;

				InCanvas->DrawItem(LineItem);
			}
		}

		// Rotation Direction
		constexpr bool bDrawRotationDirection = true;
		if (bDrawRotationDirection)
		{				
			FVector2D Start(TransformGizmo->DebugData.InteractionScreenPos);
			FVector2D End = TransformGizmo->DebugData.InteractionScreenPos + TransformGizmo->DebugData.CursorDirectionSS * 50.0f;

			FCanvasLineItem LineItem(Start, End);
			LineItem.SetColor(FLinearColor::Red);
			LineItem.LineThickness = 1.0f;

			InCanvas->DrawItem(LineItem);
		}
	}

	if (InSettings.bDrawInputCorrespondence)
	{
		constexpr float CircleRadius = 100.0f;
		const FVector2D CircleCenter = FVector2D(LabelPadding + CircleRadius, LabelPadding + CircleRadius);

		{
			FCanvasNGonItem CircleBackground(
				CircleCenter,
				FVector2D(CircleRadius, CircleRadius),
				128,
				FLinearColor::Black.CopyWithNewOpacity(0.3f));

			CircleBackground.BlendMode = SE_BLEND_Translucent;
			CircleBackground.Draw(InCanvas);
		}

		// Drag Dir
		{
			{
				FCanvasLineItem LineItem(CircleCenter, CircleCenter + TransformGizmo->InteractionScreenAxisDirection * CircleRadius);
				LineItem.SetColor(FLinearColor::Yellow);

				LineItem.Draw(InCanvas);	
			}
			
			{
				FCanvasLineItem LineItem = FCanvasLineItem(CircleCenter, CircleCenter + TransformGizmo->NormalProjectionToRemove * CircleRadius);
				LineItem.SetColor(FLinearColor::Green);

				LineItem.Draw(InCanvas);
			}
		}
	}

	switch (TransformGizmo->CurrentMode)
	{
	case EGizmoTransformMode::Translate:
		break;

	case EGizmoTransformMode::Rotate:
		{
			constexpr float Size = 8.0f;
			FCanvasNGonItem Pt1(TransformGizmo->DebugData.Test2D1, FVector2D::One() * Size, 5, FLinearColor::White);
			FCanvasNGonItem Pt2(TransformGizmo->DebugData.Test2D2, FVector2D::One() * Size, 5, FLinearColor::Red);
			FCanvasNGonItem Pt3(TransformGizmo->DebugData.Test2D3, FVector2D::One() * Size, 5, FLinearColor::Green);
			
			Pt1.Draw(InCanvas);
			Pt2.Draw(InCanvas);
			Pt3.Draw(InCanvas);
		}
		break;

	case EGizmoTransformMode::Scale:
	{
		FVector BidirectionStartPoint;
		FVector BidirectionEndPoint;

		if (InSettings.bDrawCursorRay || InSettings.bDrawInputDelta)
		{
			// FCanvasNGonItem Item(TransformGizmo->DebugData.Test2, FVector2D::One() * 10.0f, 32, FLinearColor::Red);
			// Item.Draw(InCanvas);
			
			constexpr double AxisLength = 500.0;
			const bool bDraw = InSettings.bDrawCursorRay;

			FVector2D Start =
				TransformGizmo->Interaction.ScaleInteraction.DistanceSource == EGizmoElementScaleDistanceSource::FromStart
				? TransformGizmo->InteractionPlanarStartPoint2D
				: TransformGizmo->InteractionScreenObjectPos2D;

			Start += TransformGizmo->InteractionScreenAxisDirection * (TransformGizmo->InteractionReferencePointOffsetDistance * TransformGizmo->InteractionStartSign);

			FVector2D End = Start + (TransformGizmo->InteractionScreenAxisDirection * AxisLength);

			if (bDraw)
			{
				FCanvasLineItem LineItem(Start, End);
				LineItem.SetColor(InteractionStartColor);
				LineItem.LineThickness = 1.0f;

				InCanvas->DrawItem(LineItem);
			}

			// Binormal
			const FVector2D Bidirection(-TransformGizmo->InteractionScreenAxisDirection.Y, TransformGizmo->InteractionScreenAxisDirection.X);
			End = Start + (Bidirection * AxisLength);
			Start -= (Bidirection * AxisLength);

			BidirectionStartPoint = FVector(Start, 0.0);
			BidirectionEndPoint = FVector(End, 0.0);

			if (bDraw)
			{
				FCanvasLineItem LineItem = FCanvasLineItem(Start, End);
				LineItem.SetColor(InteractionCurrentColor);
				LineItem.LineThickness = 1.5f;

				InCanvas->DrawItem(LineItem);
			}
		}

		if (InSettings.bDrawInputDelta)
		{
			const FVector ScreenPos3D = FVector(TransformGizmo->InteractionScreenCurrPos, 0.0);

			if (TransformGizmo->Interaction.ScaleInteraction.DistanceType == EGizmoElementScaleDistanceType::Directional)
			{
				const FVector MirrorLineDirection = (BidirectionEndPoint - BidirectionStartPoint).GetSafeNormal();
				if (!MirrorLineDirection.IsNearlyZero())
				{
					FVector NearestPoint;
					float DistanceToNearestPoint;
					GizmoMath::NearestPointOnLine(BidirectionStartPoint, MirrorLineDirection, ScreenPos3D, NearestPoint, DistanceToNearestPoint);

					FVector2D NearestPoint2D = FVector2D(NearestPoint.X, NearestPoint.Y);

					FCanvasDashedLineItem LineItem(NearestPoint2D, TransformGizmo->InteractionScreenCurrPos);
					LineItem.SetColor(InteractionStartColor);
					LineItem.LineThickness = 1.5f;
					LineItem.DashLength = 3.0f;
					LineItem.DashGap = 3.0f;

					InCanvas->DrawItem(LineItem);
				}
			}
			else
			{
				FCanvasDashedLineItem LineItem( TransformGizmo->InteractionScreenObjectPos2D, TransformGizmo->InteractionScreenCurrPos);
				LineItem.SetColor(InteractionStartColor);
				LineItem.LineThickness = 1.5f;
				LineItem.DashLength = 3.0f;
				LineItem.DashGap = 3.0f;

				InCanvas->DrawItem(LineItem);
			}
		}
	}
	break;

	case EGizmoTransformMode::Max:
	case EGizmoTransformMode::None:
	default:
		break;
	}
}

void UTransformGizmoDebug::DrawHitGeometry(
	const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI,
	const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor) const
{
	const UEditorTRSGizmo* TransformGizmo = UE::Editor::InteractiveToolsFramework::Internal::GetVariantAsGizmo<UEditorTRSGizmo>(InObject);
	if (!ensure(TransformGizmo)
		|| !ensure(InDebugProvider)
		|| !ensure(InRenderAPI))
	{
		return;
	}

	InDebugProvider->DrawHitGeometry(FGizmoDebugObjectVariant(TInPlaceType<const UGizmoElementBase*>(), TransformGizmo->GizmoElementRoot), InRenderAPI, InRenderState, InColor);
}
