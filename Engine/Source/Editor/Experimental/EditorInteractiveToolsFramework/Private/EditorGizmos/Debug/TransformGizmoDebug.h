// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CanvasItem.h: Unreal canvas item definitions
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "GizmoDebugBase.h"
#include "UObject/Object.h"

#include "TransformGizmoDebug.generated.h"

/**
 * Provides Debug visualization for an owning TransformGizmo
 */
UCLASS(Transient, MinimalAPI)
class UTransformGizmoDebug : public UGizmoDebugBase
{
	GENERATED_BODY()

public:
	virtual TSubclassOf<UObject> GetSupportedClass() const override;
	virtual void Draw(const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const override;
	virtual void DrawCanvas(const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FGizmoDebugSettings& InSettings) const override;
	virtual void DrawHitGeometry(const FGizmoDebugObjectVariant& InObject, const UGizmoDebugProvider* InDebugProvider, IToolsContextRenderAPI* InRenderAPI, const UGizmoElementBase::FRenderTraversalState& InRenderState, const FLinearColor& InColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5)) const override;
};
