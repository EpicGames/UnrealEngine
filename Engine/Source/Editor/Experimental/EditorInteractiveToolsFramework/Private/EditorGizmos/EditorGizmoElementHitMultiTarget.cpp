// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmoElementHitMultiTarget.h"

#include "BaseGizmos/GizmoElementBase.h"
#include "BaseGizmos/GizmoElementShared.h"
#include "BaseGizmos/ViewBasedTransformAdjusters.h"

UEditorGizmoElementHitMultiTarget::UEditorGizmoElementHitMultiTarget()
{
	// @note: The order should match the order of the enum EGizmoElementInteractionState
	StateRules = 
	{
        FStateRule
        {
            EGizmoElementInteractionState::None,
            false,
            0,
            {
                EGizmoElementInteractionState::Hovering,
                EGizmoElementInteractionState::Subdued,
                EGizmoElementInteractionState::Selected, // When switching modes, we can immediately select the "default parts" for that mode
            }
        },
        FStateRule
        {
            EGizmoElementInteractionState::Hovering,
            true,
            3,
            {
                EGizmoElementInteractionState::None,
                EGizmoElementInteractionState::Interacting,
                EGizmoElementInteractionState::Selected
            }
        },
        FStateRule
        {
            EGizmoElementInteractionState::Interacting,
            true,
            4,
            {
                EGizmoElementInteractionState::None,
                EGizmoElementInteractionState::Hovering,
                EGizmoElementInteractionState::Selected
            },
            {
            },
            {
                EGizmoElementInteractionState::Selected
            },
            false
        },
        FStateRule
        {
            EGizmoElementInteractionState::Selected,
            true,
            2,
            {
                EGizmoElementInteractionState::None,
                EGizmoElementInteractionState::Hovering,
                EGizmoElementInteractionState::Interacting
            }
        },
        FStateRule
        {
            EGizmoElementInteractionState::Subdued,
            false,
            1,
            {
                EGizmoElementInteractionState::None
            },
            {
                // A Subdued part should never be selected
                EGizmoElementInteractionState::Selected
            },
        },
     };
}

FInputRayHit UEditorGizmoElementHitMultiTarget::IsHit(const FInputDeviceRay& ClickPos) const
{
	// @note: We override this purely because the base implementation uses the TransformProxy's GetTransform, where the scale is not always 1.0
	if (GizmoElement && GizmoViewContext && GizmoTransformProxy && (!Condition || Condition(ClickPos)))
	{
		const FTransform LineTraceTransform =
			TransformAdjuster.IsValid()
			? TransformAdjuster->GetAdjustedComponentToWorld(*GizmoViewContext, GizmoTransformProxy->GetTransform())
			: GizmoTransformProxy->GetTransform();

		UGizmoElementBase::FLineTraceTraversalState LineTraceState;
		LineTraceState.Initialize(GizmoViewContext, LineTraceTransform);
		LineTraceState.HitSortType = EGizmoElementHitSortType::PriorityThenSurfaceThenClosest;

		UGizmoElementBase::FLineTraceOutput LineTraceOutput;
		return GizmoElement->LineTrace(
			GizmoViewContext,
			LineTraceState,
			ClickPos.WorldRay.Origin,
			ClickPos.WorldRay.Direction,
			LineTraceOutput);
	}

	return FInputRayHit();
}

void UEditorGizmoElementHitMultiTarget::UpdateHoverState(bool bInIsHovering, uint32 PartIdentifier)
{
	if (!GizmoElement)
	{
		return;
	}

	if (bInIsHovering)
	{
		GizmoElement->UpdatePartInteractionState(EGizmoElementInteractionState::Hovering, PartIdentifier, bAllowMultiElementParts);
	}
	else
	{
		TOptional<EGizmoElementInteractionState> CurrentState = GizmoElement->GetPartInteractionState(PartIdentifier);
		if (CurrentState.IsSet() && CurrentState.GetValue() == EGizmoElementInteractionState::Interacting)
		{
			// If the part is currently interacting, no state switch is required
			return;
		}

		const bool bPartIsSelected = IsPartSelected(PartIdentifier);
		const EGizmoElementInteractionState NextState = bPartIsSelected ? EGizmoElementInteractionState::Selected : EGizmoElementInteractionState::None;

		GizmoElement->UpdatePartInteractionState(NextState, PartIdentifier, bAllowMultiElementParts);
	}
}

void UEditorGizmoElementHitMultiTarget::UpdateInteractingState(bool bInteracting, uint32 InPartIdentifier)
{
	if (!GizmoElement)
	{
		return;
	}

	if (bInteracting)
	{
		GizmoElement->UpdatePartInteractionState(EGizmoElementInteractionState::Interacting, InPartIdentifier, bAllowMultiElementParts);
	}
	else
	{
		const bool bPartIsSelected = IsPartSelected(InPartIdentifier);
		const EGizmoElementInteractionState NextState = bPartIsSelected ? EGizmoElementInteractionState::Selected : EGizmoElementInteractionState::None;
		GizmoElement->UpdatePartInteractionState(NextState, InPartIdentifier, bAllowMultiElementParts);
	}
}

void UEditorGizmoElementHitMultiTarget::UpdateSelectedState(bool bSelected, uint32 InPartIdentifier)
{
	if (!GizmoElement)
	{
		return;
	} 

	const EGizmoElementInteractionState NextState = bSelected ? EGizmoElementInteractionState::Selected : EGizmoElementInteractionState::None;
	SetPartSelected(InPartIdentifier, bSelected);

	if (bSelected)
	{
		// Just because we're selected, it doesn't mean we should switch states - only switch if a higher priority state isn't active
		TOptional<EGizmoElementInteractionState> CurrentState = GizmoElement->GetPartInteractionState(InPartIdentifier);
		if (CurrentState.IsSet())
		{
			const uint32 CurrentStatePriority = StateRules[static_cast<uint32>(CurrentState.GetValue())].Priority;
			const uint32 SelectedStatePriority = StateRules[static_cast<uint32>(EGizmoElementInteractionState::Selected)].Priority;

			if (CurrentStatePriority > SelectedStatePriority)
			{
				return; // Don't switch states if the current state has a higher priority
			}
		}
	}

	GizmoElement->UpdatePartInteractionState(NextState, InPartIdentifier, bAllowMultiElementParts);
}

void UEditorGizmoElementHitMultiTarget::UpdateSubdueState(bool bSubdued, uint32 InPartIdentifier)
{
	if (!GizmoElement)
	{
		return;
	}

	const EGizmoElementInteractionState NextState = bSubdued ? EGizmoElementInteractionState::Subdued : EGizmoElementInteractionState::None;

	// Subdue forcibly deselects the part
	if (bSubdued)
	{
		SetPartSelected(InPartIdentifier, false);
	}

	GizmoElement->UpdatePartInteractionState(NextState, InPartIdentifier, bAllowMultiElementParts);
}

void UEditorGizmoElementHitMultiTarget::SetUsePriorityHitTesting(const bool bInUsePriorityHitTesting)
{
	bUsePriorityHitTesting = bInUsePriorityHitTesting;
}

bool UEditorGizmoElementHitMultiTarget::GetUsePriorityHitTesting() const
{
	return bUsePriorityHitTesting;
}

UEditorGizmoElementHitMultiTarget* UEditorGizmoElementHitMultiTarget::Construct(UGizmoElementBase* InGizmoElement, UGizmoViewContext* InGizmoViewContext, UObject* Outer)
{
	UEditorGizmoElementHitMultiTarget* NewHitTarget = NewObject<UEditorGizmoElementHitMultiTarget>(Outer);
	NewHitTarget->GizmoElement = InGizmoElement;
	NewHitTarget->GizmoViewContext = InGizmoViewContext;
	return NewHitTarget;
}

bool UEditorGizmoElementHitMultiTarget::IsPartSelected(uint32 InPartIdentifier) const
{
	return PartSelectionState.IsValidIndex(InPartIdentifier) && PartSelectionState[InPartIdentifier];
}

void UEditorGizmoElementHitMultiTarget::SetPartSelected(uint32 InPartIdentifier, const bool bInSelected)
{
	if (!PartSelectionState.IsValidIndex(InPartIdentifier))
	{
		PartSelectionState.SetNum(InPartIdentifier + 1, false);
	}

	PartSelectionState[InPartIdentifier] = bInSelected;
}
