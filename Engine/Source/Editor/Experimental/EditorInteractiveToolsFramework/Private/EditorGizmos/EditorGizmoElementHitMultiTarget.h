// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "BaseGizmos/GizmoElementHitTargets.h"
#include "BaseGizmos/GizmoElementShared.h"

#include "EditorGizmoElementHitMultiTarget.generated.h"

/**
 * An implementation of IGizmoClickMultiTarget with additional rules to determine valid interaction state transitions. 
 */
UCLASS(MinimalAPI)
class UEditorGizmoElementHitMultiTarget : public UGizmoElementHitMultiTarget
{
	GENERATED_BODY()

public:
	UEditorGizmoElementHitMultiTarget();

	/** Performs a ray hit test against the gizmo element hierarchy, respecting priority hit testing when enabled. */
	virtual FInputRayHit IsHit(const FInputDeviceRay& ClickPos) const override;

	/** Updates the hover state for the given part, applying state transition rules. */
	virtual void UpdateHoverState(bool bHovering, uint32 InPartIdentifier) override;

	/** Updates the interacting (active drag) state for the given part, applying state transition rules. */
	virtual void UpdateInteractingState(bool bInteracting, uint32 InPartIdentifier) override;

	/** Updates the selected state for the given part, applying state transition rules. */
	virtual void UpdateSelectedState(bool bSelected, uint32 InPartIdentifier) override;

	/** Updates the subdued (dimmed) state for the given part, applying state transition rules. */
	virtual void UpdateSubdueState(bool bSubdued, uint32 InPartIdentifier) override;

	/** Sets whether priority-based hit testing is used (closer/more specific elements take precedence). */
	void SetUsePriorityHitTesting(const bool bInUsePriorityHitTesting);

	/** Returns whether priority-based hit testing is enabled. */
	bool GetUsePriorityHitTesting() const;

public:
	/**
	 * Factory method to create and initialize a hit multi-target for the given gizmo element.
	 * @param InGizmoElement The root gizmo element to test hits against.
	 * @param InGizmoViewContext The view context providing camera/viewport information.
	 * @param Outer The outer object for the new instance.
	 */
	static UEditorGizmoElementHitMultiTarget* Construct(
		UGizmoElementBase* InGizmoElement,
		UGizmoViewContext* InGizmoViewContext,
		UObject* Outer = (UObject*)GetTransientPackage());

private:
	/** Returns whether the given part is currently in the selected state. */
	bool IsPartSelected(uint32 InPartIdentifier) const;

	/** Sets the selected state for the given part, updating the PartSelectionState bit array. */
	void SetPartSelected(uint32 InPartIdentifier, const bool bInSelected);

	/** When true, uses priority-based hit testing where closer or more specific elements take precedence. */
	UPROPERTY(Getter = "GetUsePriorityHitTesting", Setter = "SetUsePriorityHitTesting")
	bool bUsePriorityHitTesting = true;

private:
	/** Whether parts can be composed of multiple gizmo elements (always true for this implementation). */
	static constexpr bool bAllowMultiElementParts = true;

	/** Defines a state transition rule for a single interaction state. */
	struct FStateRule
	{
		/** The interaction state this rule applies to. */
		EGizmoElementInteractionState State;

		/** If true, only the primary and secondary parts can hold this state simultaneously. */
		bool bIsExclusive = false;

		/** Priority rank used to resolve conflicts when multiple states are available. */
		uint8 Priority = 0;

		/** Valid states that can follow this state in a transition. */
		TArray<EGizmoElementInteractionState, TFixedAllocator<static_cast<uint32>(EGizmoElementInteractionState::Max)>> NextValidStates;

		/** States removed from the same part when this state is applied (prevents priority-based state loss). */
		TArray<EGizmoElementInteractionState, TFixedAllocator<static_cast<uint32>(EGizmoElementInteractionState::Max)>> RemoveStates;

		/** States removed from all other parts when this state is applied. */
		TArray<EGizmoElementInteractionState, TFixedAllocator<static_cast<uint32>(EGizmoElementInteractionState::Max)>> RemoveStatesOnOthers;

		/** If set, overrides the selection flag of all other parts when this state is applied. */
		TOptional<bool> bSelectFlagOverrideOnOthers;
	};

	/** Rules governing valid state transitions for each interaction state. */
	TArray<FStateRule, TFixedAllocator<static_cast<uint32>(EGizmoElementInteractionState::Max)>> StateRules;

	/** Per-part selection state, allowing interaction states to overlay (e.g. Hover can revert to Selected). */
	TBitArray<> PartSelectionState;
};
