// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmoElementInterfaces.h"

namespace UE::Editor::InteractiveToolsFramework
{
	Geometry::FFrame3d IPlaneProvider::MakePlane(
		const FTransform& InTransform,
		const UGizmoViewContext* InViewContext,
		const EToolContextCoordinateSystem InCoordinateSystem,
		const FRotationContext& InRotationContext,
		const EAxisList::Type InAxisList) const
	{
		// Default to calling the non-rotation context overload for implementers that don't require it
		return MakePlane(InTransform, InViewContext, InCoordinateSystem, InAxisList);
	}
}
