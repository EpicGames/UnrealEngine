// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorGizmos/TransformGizmoInterfaces.h"
#include "FrameTypes.h"
#include "ToolContextInterfaces.h"

class UGizmoViewContext;

namespace UE::Editor::InteractiveToolsFramework
{
	/**
	 * Interface for gizmo elements that can construct an interaction plane.
	 * Used by axis and group elements to define the plane on which drag interactions are projected.
	 */
	class IPlaneProvider
	{
	public:
		virtual ~IPlaneProvider() = default;

		/**
		 * Creates a plane from the given parameters, transformed into the provided coordinate space.
		 * The implementer *may* dictate and thus ignore the given AxisList. Implementers that require it should handle cases where an invalid value is passed to it.
		 */
		virtual Geometry::FFrame3d MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const EAxisList::Type InAxisList = EAxisList::None) const = 0;

		/**
		 * Creates a plane from the given parameters, transformed into the provided coordinate space using the rotation context.
		 * The implementer *may* dictate and thus ignore the given AxisList. Implementers that require it should handle cases where an invalid value is passed to it.
		 */
		virtual Geometry::FFrame3d MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const FRotationContext& InRotationContext, const EAxisList::Type InAxisList = EAxisList::None) const;
	};
}
