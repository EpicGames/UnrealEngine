// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/EditorGizmoElementShared.h"

#include "BaseGizmos/GizmoElementBase.h"
#include "BaseGizmos/GizmoElementLineBase.h"

namespace UE::Editor::InteractiveToolsFramework
{
	namespace Private::EditorGizmoElementSharedLocals
	{
		/** Shared functionality to reduce boilerplate! Copied from @see: GizmoElementShared.cpp */
		template <typename StructType, typename ValueType>
		static const ValueType& GetValueForState(const StructType& InPerStateValue, const EGizmoElementInteractionState InState)
		{
			switch (InState)
			{
			case EGizmoElementInteractionState::Hovering:
				return InPerStateValue.GetHoverValue();

			case EGizmoElementInteractionState::Interacting:
				return InPerStateValue.GetInteractValue();

			case EGizmoElementInteractionState::Selected:
				return InPerStateValue.GetSelectValue();

			case EGizmoElementInteractionState::Subdued:
				return InPerStateValue.GetSubdueValue();

			case EGizmoElementInteractionState::None:
			default:
				return InPerStateValue.GetDefaultValue();
			}
		}
	}

	void ApplyMaterialsToElement(UGizmoElementBase* InElement, const FGizmoPerStateValueMaterialVariant& InMaterials)
	{
		if (!ensure(InElement))
		{
			return;
		}

		// Returns either the Solid or Translucent material, whichever is available.
		auto GetFirstValidMaterial = [&](const FGizmoMaterialVariant& InMaterialVariants) -> UMaterialInterface*
		{
			if (InMaterialVariants.Solid)
			{
				return InMaterialVariants.Solid.Get();
			}

			if (InMaterialVariants.Translucent)
			{
				return InMaterialVariants.Translucent.Get();
			}

			return nullptr;
		};

		if (InMaterials.Default.IsSet())
		{
			InElement->SetMaterial(GetFirstValidMaterial(InMaterials.Default.GetValue()));	
		}

		if (InMaterials.Hover.IsSet())
		{
			InElement->SetHoverMaterial(GetFirstValidMaterial(InMaterials.Hover.GetValue()));	
		}

		if (InMaterials.Select.IsSet())
		{
			InElement->SetSelectMaterial(GetFirstValidMaterial(InMaterials.Select.GetValue()));	
		}

		if (InMaterials.Interact.IsSet())
		{
			InElement->SetInteractMaterial(GetFirstValidMaterial(InMaterials.Interact.GetValue()));
		}

		if (InMaterials.Subdue.IsSet())
		{
			InElement->SetSubdueMaterial(GetFirstValidMaterial(InMaterials.Subdue.GetValue()));
		}
	}

	void ApplyMaterialOverrides(FGizmoMaterialVariant& InMaterialToOverride, const FGizmoMaterialVariant& InMaterialToOverrideFrom)
	{
		InMaterialToOverride.Solid = InMaterialToOverrideFrom.Solid ? InMaterialToOverrideFrom.Solid : InMaterialToOverride.Solid;
		InMaterialToOverride.Translucent = InMaterialToOverrideFrom.Translucent ? InMaterialToOverrideFrom.Translucent : InMaterialToOverride.Translucent;
	}

	void ApplyMaterialOverrides(FGizmoPerStateValueMaterialVariant& InMaterialsToOverride, const FGizmoPerStateValueMaterialVariant& InMaterialsToOverrideFrom)
	{
		auto ApplyOverride = [&](TOptional<FGizmoMaterialVariant>& InMaterialToOverride, const TOptional<FGizmoMaterialVariant>& InMaterialToOverrideFrom)
		{
			if (InMaterialToOverrideFrom.IsSet())
			{
				if (!InMaterialToOverride.IsSet())
				{
					InMaterialToOverride.Emplace();
				}

				ApplyMaterialOverrides(InMaterialToOverride.GetValue(), InMaterialToOverrideFrom.GetValue());
			}
		};

		ApplyOverride(InMaterialsToOverride.Default, InMaterialsToOverrideFrom.Default);
		ApplyOverride(InMaterialsToOverride.Hover, InMaterialsToOverrideFrom.Hover);
		ApplyOverride(InMaterialsToOverride.Select, InMaterialsToOverrideFrom.Select);
		ApplyOverride(InMaterialsToOverride.Interact, InMaterialsToOverrideFrom.Interact);
		ApplyOverride(InMaterialsToOverride.Subdue, InMaterialsToOverrideFrom.Subdue);
	}

	void ApplyColorsToElement(UGizmoElementBase* InElement, const FGizmoPerStateValueLinearColor& InColors)
	{
		if (!ensure(InElement))
		{
			return;
		}

		if (InColors.Default.IsSet())
		{
			InElement->SetVertexColor(InColors.Default.GetValue());
		}

		if (InColors.Hover.IsSet())
		{
			InElement->SetHoverVertexColor(InColors.Hover.GetValue());
		}

		if (InColors.Select.IsSet())
		{
			InElement->SetSelectVertexColor(InColors.Select.GetValue());
		}

		if (InColors.Interact.IsSet())
		{
			InElement->SetInteractVertexColor(InColors.Interact.GetValue());
		}

		if (InColors.Subdue.IsSet())
		{
			InElement->SetSubdueVertexColor(InColors.Subdue.GetValue());
		}
	}

	void ApplyColorsToElement(UGizmoElementLineBase* InElement, const FGizmoPerStateValueLinearColor& InColors)
	{
		if (!ensure(InElement))
		{
			return;
		}

		if (InColors.Default.IsSet())
		{
			InElement->SetLineColor(InColors.Default.GetValue());	
		}

		if (InColors.Hover.IsSet())
		{
			InElement->SetHoverLineColor(InColors.Hover.GetValue());
		}

		if (InColors.Select.IsSet())
		{
			InElement->SetSelectLineColor(InColors.Select.GetValue());
		}

		if (InColors.Interact.IsSet())
		{
			InElement->SetInteractLineColor(InColors.Interact.GetValue());
		}

		if (InColors.Subdue.IsSet())
		{
			InElement->SetSubdueLineColor(InColors.Subdue.GetValue());
		}
	}

	void ApplyColorOverrides(FGizmoPerStateValueLinearColor& InColorsToOverride, const FGizmoPerStateValueLinearColor& InColorsToOverrideFrom)
	{
		auto ApplyOverride = [&](TOptional<FLinearColor>& InColorToOverride, const TOptional<FLinearColor>& InColorToOverrideFrom)
		{
			if (InColorToOverrideFrom.IsSet())
			{
				if (!InColorToOverride.IsSet())
				{
					InColorToOverride.Emplace();
				}

				InColorToOverride.Emplace(InColorToOverrideFrom.GetValue());
			}
		};

		ApplyOverride(InColorsToOverride.Default, InColorsToOverrideFrom.Default);
		ApplyOverride(InColorsToOverride.Hover, InColorsToOverrideFrom.Hover);
		ApplyOverride(InColorsToOverride.Select, InColorsToOverrideFrom.Select);
		ApplyOverride(InColorsToOverride.Interact, InColorsToOverrideFrom.Interact);
		ApplyOverride(InColorsToOverride.Subdue, InColorsToOverrideFrom.Subdue);
	}
}

const FGizmoPerStateValueMaterialVariant FGizmoStyleBase::DefaultMaterials;
const FGizmoPerStateValueLinearColor FGizmoStyleBase::DefaultColors;
const FGizmoPerStateValueMaterialVariant FGizmoStyleBase::DefaultLineMaterials;
const FGizmoPerStateValueLinearColor FGizmoStyleBase::DefaultLineColors;

const FGizmoMaterialVariant& FGizmoPerStateValueMaterialVariant::GetValueForState(const EGizmoElementInteractionState InState) const
{
	return UE::Editor::InteractiveToolsFramework::Private::EditorGizmoElementSharedLocals::GetValueForState<FGizmoPerStateValueMaterialVariant, const FGizmoMaterialVariant&>(*this, InState);
}

const FGizmoMaterialVariant& FGizmoPerStateValueMaterialVariant::GetDefaultValue() const
{
	static FGizmoMaterialVariant DefaultValue;
	return Default.Get(DefaultValue);
}
