// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/EditorGizmoElementSharedInternal.h"

#include "BaseGizmos/GizmoElementBox.h"
#include "EditorGizmoElementInterfaces.h"
#include "BaseGizmos/GizmoElementShared.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"
#include "FrameTypes.h"

namespace UE::Editor::InteractiveToolsFramework::Internal
{
	FTransformGizmoScaleAdjuster::FTransformGizmoScaleAdjuster(
		const IToolsContextQueriesAPI* InQueriesAPI,
		const TWeakInterfacePtr<ITransformGizmoSource>& InTransformGizmoSource)
		: QueriesAPI(InQueriesAPI)
		, TransformGizmoSource(InTransformGizmoSource)
	{
		ensureAlways(InQueriesAPI);

		// @note: TransformGizmoSource isn't always expected to be valid
	}

	FTransform FTransformGizmoScaleAdjuster::GetAdjustedComponentToWorld(const GizmoRenderingUtil::ISceneViewInterface& InView, const FTransform& InCurrentComponentToWorld)
	{
		const float Scale = TransformGizmoSource.IsValid() ? TransformGizmoSource->GetGizmoScale() : 1.0f;
	
		auto GetCoordinateSystem = [&]()
		{
			if (TransformGizmoSource.IsValid())
			{
				return TransformGizmoSource->GetGizmoCoordSystemSpace(); 
			}

			return QueriesAPI ? QueriesAPI->GetCurrentCoordinateSystem() : EToolContextCoordinateSystem::World;
		};

		const bool bIsLocalCoordinateSpace = GetCoordinateSystem() == EToolContextCoordinateSystem::Local;

		// Construct only with the translation component
		FTransform AdjustedTransform(InCurrentComponentToWorld.GetTranslation());
		if (bIsLocalCoordinateSpace)
		{
			AdjustedTransform.SetRotation(InCurrentComponentToWorld.GetRotation());
		}

		AdjustedTransform.SetScale3D(FVector(Scale));

		return AdjustedTransform;
	}

	int32 GetAxisIndex(const EAxisList::Type InAxisList)
	{
		switch (InAxisList)
		{
		case EAxisList::X:
		case EAxisList::Left:
			return 0;

		case EAxisList::Y:
		case EAxisList::Up:
			return 1;

		case EAxisList::Z:
		case EAxisList::Forward:
			return 2;

		default:
			return INDEX_NONE;
		}
	}

	Geometry::FFrame3d MakeScreenAlignedPlane(const FVector& InPlaneOrigin, const UGizmoViewContext* InViewContext)
	{
		// View "Up" and "Right" are different to Plane Up/Side - see GetAxisBasis

		const FVector PlaneNormal = -InViewContext->GetViewDirection(); // Face view, not away from view
		const FVector PlaneUp = InViewContext->GetViewUp();
		const FVector PlaneSide = -InViewContext->GetViewRight(); // Make left-handed

		using namespace UE::InteractiveToolsFramework;
		return MakePlaneFrame(InPlaneOrigin, PlaneNormal, PlaneUp, PlaneSide);
	}

	Geometry::FFrame3d MakeTransformedPlane(const FTransform& InTransform, const EToolContextCoordinateSystem InCoordinateSystem, const FVector& InForwardDirection, const FVector& InUpDirection, const FVector& InSideDirection)
	{
		const FVector PlaneOrigin = InTransform.GetLocation();

		FVector PlaneNormal = InForwardDirection;
		FVector PlaneUp = InUpDirection;
		FVector PlaneSide = InSideDirection;

		const bool bIsLocal = InCoordinateSystem == EToolContextCoordinateSystem::Local;
		if (bIsLocal)
		{
			PlaneNormal = InTransform.TransformVectorNoScale(PlaneNormal);
			PlaneUp = InTransform.TransformVectorNoScale(PlaneUp);
			PlaneSide = InTransform.TransformVectorNoScale(PlaneSide);
		}

		using namespace UE::InteractiveToolsFramework;
		return MakePlaneFrame(PlaneOrigin, PlaneNormal, PlaneUp, PlaneSide);
	}

	Geometry::FFrame3d MakeTransformedPlaneForAxisList(
		const FTransform& InTransform,
		const UGizmoViewContext* InViewContext,
		const EToolContextCoordinateSystem InCoordinateSystem,
		const EAxisList::Type InAxisList,
		const TFunctionRef<const UGizmoElementBox*(const EAxisList::Type InAxisList)>& InGetPlanarElementFunc,
		const TFunctionRef<const UE::Editor::InteractiveToolsFramework::IPlaneProvider*(const EAxisList::Type InAxisList)>& InGetAxisPlaneProvider)
	{
		if (!InViewContext)
		{
			return Geometry::FFrame3d();
		}

		using namespace UE::InteractiveToolsFramework;
		using namespace UE::Editor::InteractiveToolsFramework;

		const FVector PlaneOrigin = InTransform.GetLocation();

		FVector PlaneNormal = FVector::ForwardVector;
		FVector PlaneUp = FVector::UpVector;
		FVector PlaneSide = FVector::RightVector;

		const bool bIsLocal = InCoordinateSystem == EToolContextCoordinateSystem::Local;
		auto TransformAxis = [&](const FVector& InAxis)
		{
			if (bIsLocal)
			{
				return InTransform.TransformVectorNoScale(InAxis);	
			}

			return InAxis;
		};

		if (IsAxisPlanar(InAxisList))
		{
			if (const UGizmoElementBox* PlanarElement = InGetPlanarElementFunc(InAxisList))
			{
				PlaneNormal = PlanarElement->GetViewDependentAxis();
				PlaneUp = PlanarElement->GetUpDirection();
				PlaneSide = PlanarElement->GetSideDirection();

				PlaneNormal = TransformAxis(PlaneNormal);
				PlaneUp = TransformAxis(PlaneUp);
				PlaneSide = TransformAxis(PlaneSide);
			}
		}
		else if (IsAxisSingular(InAxisList))
		{
			if (const IPlaneProvider* AxisPlaneProvider = InGetAxisPlaneProvider(InAxisList))
			{
				return AxisPlaneProvider->MakePlane(InTransform, InViewContext, InCoordinateSystem, InAxisList);
			}
		}
		else
		{
			return MakeScreenAlignedPlane(PlaneOrigin, InViewContext);
		}

		return MakePlaneFrame(PlaneOrigin, PlaneNormal, PlaneUp, PlaneSide);
	}

	Geometry::FFrame3d ModifyPlaneForView(
		const Geometry::FFrame3d& InPlane,
		const UGizmoViewContext* InViewContext,
		FVector& OutDirectionToRemove)
	{
		if (!InViewContext)
		{
			return InPlane;
		}

		using namespace UE::InteractiveToolsFramework;

		FVector PlaneNormal = FVector::ForwardVector;
		FVector PlaneUp = FVector::UpVector;
		FVector PlaneSide = FVector::RightVector;
		BreakPlaneFrame(InPlane, PlaneNormal, PlaneUp, PlaneSide);

		const FVector ViewDirection = InViewContext->GetViewDirection();
		const FVector::FReal UpDot = FMath::Abs(FVector::DotProduct(ViewDirection, PlaneUp));
		const FVector::FReal SideDot = FMath::Abs(FVector::DotProduct(ViewDirection, PlaneSide));

		const FVector PrimaryDirection = (UpDot > SideDot) ? PlaneUp : PlaneSide;
		OutDirectionToRemove = (UpDot > SideDot) ? PlaneSide : PlaneUp;

		Geometry::FFrame3d ModifiedPlane = InPlane;
		ModifiedPlane.AlignAxis(0, PrimaryDirection); // Align the plane normal to the primary direction

		return ModifiedPlane;
	}

	int8 GetClockwiseAngleSignForAxis(const EAxis::Type InAxis)
	{
		// Z is correct, others aren't
		return InAxis == EAxis::Z ? 1 : -1;
	}
}
