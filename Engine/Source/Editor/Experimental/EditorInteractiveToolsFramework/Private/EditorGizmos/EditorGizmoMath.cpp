// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/EditorGizmoMath.h"

#include "BaseGizmos/GizmoElementShared.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "CircleTypes.h"
#include "Misc/AxisDisplayInfo.h"

namespace UE::Editor::GizmoMath
{
	/** Pushes the given rectangle to touch the outside of the given circle. */
	FBox2d PlaceRectOutsideCircle(const Geometry::FCircle2d& InCircle, const FBox2d& InRect)
	{
		const FVector2d RectCenter = InRect.GetCenter();		
		FVector2d CircleToRect = RectCenter - InCircle.Center;

		if (CircleToRect.IsNearlyZero())
		{
			// Center's overlap
			return InRect;
		}

		CircleToRect = CircleToRect.GetSafeNormal();

		const FVector2d RectExtent = InRect.GetExtent();
		const FVector2d NewRectCenter(
			FMath::Sign(CircleToRect.X) * RectExtent.X,
			FMath::Sign(CircleToRect.Y) * RectExtent.Y);

		const FVector2d ClosestPointOnRect = RectCenter - NewRectCenter;
		const FVector2d::FReal NewDistance = (ClosestPointOnRect - InCircle.Center).Size();

		const FVector2d::FReal Delta = InCircle.Radius - NewDistance;

		return InRect.ShiftBy(CircleToRect * Delta);
	}

	bool ClipLineToBox(const FVector2d& InLineStart, const FVector2d& InLineEnd, const FBox2d& InBox, FVector::FReal& OutClippedStart, FVector::FReal& OutClippedEnd)
	{
		OutClippedStart = 0.0;
		OutClippedEnd = 1.0;

		const FVector2d LineDelta = InLineEnd - InLineStart;

		FVector2d::FReal ClippedStart = 0.0;
		FVector2d::FReal ClippedEnd = 1.0;

		auto Clip = [](const FVector2d::FReal& InDelta, const FVector2d::FReal& InOrigin, FVector2d::FReal& InOutStart, FVector2d::FReal& InOutEnd) -> bool
		{
			if (FMath::Abs(InDelta) <= UE_DOUBLE_SMALL_NUMBER)
			{
				// Parallel case, so accept if origin is inside
				return InOrigin >= 0.0;
			}

			const FVector2d::FReal CandidateParameter = InOrigin / InDelta;

			// Entering boundary, do we go beyond it??
			if (InDelta < 0.0)
			{
				// Nope
				if (CandidateParameter > InOutEnd)
				{
					return false;
				}

				// Inside boundary/box, continue
				if (CandidateParameter > InOutStart)
				{
					InOutStart = CandidateParameter;
				}
			}
			// We're leaving the boundary, but did we start before it?
			else
			{
				// Nope
				if (CandidateParameter < InOutStart)
				{
					return false;
				}

				// Inside boundary/box, continue
				if (CandidateParameter < InOutEnd)
				{
					InOutEnd = CandidateParameter;
				}
			}

			// One or more intersections found
			return true;
		};

		if (Clip(-LineDelta.X, InLineStart.X - InBox.Min.X, ClippedStart, ClippedEnd)
			&& Clip(+LineDelta.X, InBox.Max.X - InLineStart.X, ClippedStart, ClippedEnd)
			&& Clip(-LineDelta.Y, InLineStart.Y - InBox.Min.Y, ClippedStart, ClippedEnd)
			&& Clip(+LineDelta.Y, InBox.Max.Y - InLineStart.Y, ClippedStart, ClippedEnd))
		{
			// Result was entirely outside of the box
			if (ClippedStart > ClippedEnd)
			{
				return false;
			}

			OutClippedStart = ClippedStart;
			OutClippedEnd = ClippedEnd;

			return true;
		}

		return false;
	}

	/** Compute (World) Plane -> Screen matrix. Use inverse for Screen -> Plane. */
	bool ComputePlaneToScreenMatrix(
		const UGizmoViewContext* InView,
		const UE::Geometry::FFrame3d& InPlane,
		Geometry::FMatrix3d& OutPlaneToScreen)
	{
		using namespace UE::Geometry;

		const FVector PlaneOrigin = InPlane.Origin;
		FVector UnusedPlaneNormal;
		FVector PlaneUp;
		FVector PlaneSide;
		UE::InteractiveToolsFramework::BreakPlaneFrame(InPlane, UnusedPlaneNormal, PlaneUp, PlaneSide);

		// Make unit square from plane
		const FVector PlanePointBottomLeft = PlaneOrigin;
		const FVector PlanePointBottomRight = PlaneOrigin + PlaneSide;
		const FVector PlanePointTopLeft = PlaneOrigin + PlaneUp;
		const FVector PlanePointTopRight = PlaneOrigin + PlaneSide + PlaneUp;

		FVector2d PlanePointBottomLeft2D = InView->WorldToPixel(PlanePointBottomLeft);
		FVector2d PlanePointBottomRight2D = InView->WorldToPixel(PlanePointBottomRight);
		FVector2d PlanePointTopLeft2D = InView->WorldToPixel(PlanePointTopLeft);
		FVector2d PlanePointTopRight2D = InView->WorldToPixel(PlanePointTopRight);

		// Create the homography matrix (PlanePointRight, PlanePointTop, 0 -> Screen)
		{
			// Based on Graphics Gems / Paul Heckbert
			const FVector2d RightEdgeAxis2D = PlanePointBottomRight2D - PlanePointTopRight2D;
			const FVector2d TopEdgeAxis2D = PlanePointTopLeft2D - PlanePointTopRight2D;

			// For perspective distortion vs. parallelogram
			const FVector2d DiagonalDifference2D = PlanePointBottomLeft2D - PlanePointBottomRight2D - PlanePointTopLeft2D + PlanePointTopRight2D;

			FVector2d PerspectiveWarp2D = FVector2d::ZeroVector;
			const double Denominator = FVector2d::DotProduct(RightEdgeAxis2D, TopEdgeAxis2D);
			if (!FMath::IsNearlyZero(Denominator))
			{
				PerspectiveWarp2D.X = FVector2d::DotProduct(DiagonalDifference2D, TopEdgeAxis2D) / Denominator;
				PerspectiveWarp2D.Y = FVector2d::DotProduct(RightEdgeAxis2D, DiagonalDifference2D) / Denominator;
			}

			const FVector2d BottomRightToBottomLeft2D = (PlanePointBottomRight2D - PlanePointBottomLeft2D) + PerspectiveWarp2D.X * PlanePointBottomRight2D;
			const FVector2d TopLeftToBottomLeft2D = (PlanePointTopLeft2D - PlanePointBottomLeft2D) + PerspectiveWarp2D.Y * PlanePointTopLeft2D;

			// Accounts for colinear points
			const double SignedArea = BottomRightToBottomLeft2D.X * TopLeftToBottomLeft2D.Y - BottomRightToBottomLeft2D.Y * TopLeftToBottomLeft2D.X;

			if (FMath::IsNearlyZero(SignedArea) || BottomRightToBottomLeft2D.IsNearlyZero() || TopLeftToBottomLeft2D.IsNearlyZero())
			{
				return false; // Degenerate projection, plane is edge-on
			}

			OutPlaneToScreen = FMatrix3d(
				BottomRightToBottomLeft2D.X, TopLeftToBottomLeft2D.X, PlanePointBottomLeft2D.X,
				BottomRightToBottomLeft2D.Y, TopLeftToBottomLeft2D.Y, PlanePointBottomLeft2D.Y,
				PerspectiveWarp2D.X, PerspectiveWarp2D.Y, 1.0);
		}

		return true;
	}

	/** Contains results shared between multiple internal calls. */
	struct FScreenToPlaneLocalResult
	{
		FVector PlaneOrigin = FVector::ZeroVector;
		FVector PlaneUp = FVector::UpVector;
		FVector PlaneSide = FVector::RightVector;

		/** This is relative to the plane origin. Note that it is not normalized, and includes the distance from the plane origin. */
		FVector DirectionToProjectedPoint = FVector::OneVector;
	};

	static bool ComputeScreenPointToPlane(
		const UGizmoViewContext* InView,
		const FVector2D& InScreenPoint,
		const UE::Geometry::FFrame3d& InPlane,
		FScreenToPlaneLocalResult& OutResult)
	{
		using namespace UE::Geometry;

		FMatrix3d PlaneToScreen;
		if (!ComputePlaneToScreenMatrix(InView, InPlane, PlaneToScreen))
		{
			return false; // Failed to compute the screen to plane matrix
		}

		const FVector PointOnPlane = PlaneToScreen.Inverse() * FVector(InScreenPoint.X, InScreenPoint.Y, 1.0);
		if (PointOnPlane.IsNearlyZero())
		{
			return false;
		}

		OutResult.PlaneOrigin = InPlane.Origin;

		FVector UnusedPlaneNormal;
		UE::InteractiveToolsFramework::BreakPlaneFrame(InPlane, UnusedPlaneNormal, OutResult.PlaneUp, OutResult.PlaneSide);

		OutResult.DirectionToProjectedPoint = (PointOnPlane.X * OutResult.PlaneSide) + (PointOnPlane.Y * OutResult.PlaneUp);
		if (OutResult.DirectionToProjectedPoint.IsNearlyZero())
		{
			return false;
		}

		return true;
	}

	bool ComputeProjectedPointOnPlaneFromScreen(
		const UGizmoViewContext* InView,
		const FVector2d& InScreenPoint,
		const UE::Geometry::FFrame3d& InPlane,
		FVector& OutPoint)
	{
		FScreenToPlaneLocalResult ScreenToPlaneResult;
		if (!ComputeScreenPointToPlane(InView, InScreenPoint, InPlane, ScreenToPlaneResult))
		{
			return false;
		}

		// Now add the plane origin to the point to get the world space position
		OutPoint = ScreenToPlaneResult.DirectionToProjectedPoint + ScreenToPlaneResult.PlaneOrigin;

		return true;
	}

	bool ComputeAngleInPlaneFromScreen(
		const UGizmoViewContext* InView,
		const FVector2d& InScreenPoint,
		const UE::Geometry::FFrame3d& InPlane,
		FVector& OutPoint,
		double& OutAngle)
	{
		FScreenToPlaneLocalResult ScreenToPlaneResult;
		if (!ComputeScreenPointToPlane(InView, InScreenPoint, InPlane, ScreenToPlaneResult))
		{
			return false;
		}

		OutPoint = ScreenToPlaneResult.DirectionToProjectedPoint;

		const FVector::FReal PointUp = FVector::DotProduct(OutPoint, ScreenToPlaneResult.PlaneUp);
		const FVector::FReal PointSide = FVector::DotProduct(OutPoint, ScreenToPlaneResult.PlaneSide);

		// Angle is relative to Up, and we flip the "Side" from the cartesian LHS convention, to UE RHS (when viewed towards the plane)
		OutAngle = FMath::Atan2(-PointSide, PointUp);

		// Now add the plane origin to the point to get the world space position
		OutPoint += ScreenToPlaneResult.PlaneOrigin;

		return true;
	}

	bool ComputePointOnPlaneFromScreen(
		const UGizmoViewContext* InView, 
		const FRay& InRay,
		const FVector2d& InScreenPoint, 
		const UE::Geometry::FFrame3d& InPlane, 
		FVector& OutPoint)
	{
		FScreenToPlaneLocalResult ScreenToPlaneResult;
		if (!ComputeScreenPointToPlane(InView, InScreenPoint, InPlane, ScreenToPlaneResult))
		{
			return false;
		}
		
		OutPoint = ScreenToPlaneResult.DirectionToProjectedPoint;

		FVector HitPoint = OutPoint;
		if (InPlane.RayPlaneIntersection(InRay.Origin, InRay.Direction, 0, HitPoint))
		{
			OutPoint = HitPoint;
		}
		else
		{
			// Ray is parallel or going beyond the horizon
			
			// Whether to place the point on the horizon, or use the projected point
			constexpr bool bUseHorizon = true;
			if constexpr (bUseHorizon)
			{
				constexpr double HorizonDistance = UE_OLD_HALF_WORLD_MAX1;
				OutPoint = ScreenToPlaneResult.PlaneOrigin + (ScreenToPlaneResult.DirectionToProjectedPoint.GetSafeNormal() * HorizonDistance);
			}
			else
			{
				// Make projected point relative to the plane origin
				OutPoint += ScreenToPlaneResult.PlaneOrigin;
			}
		}

		return true;
	}

	bool ComputePointOnCircleFromScreen(
		const UGizmoViewContext* InView,
		const FVector2d& InScreenPoint,
		const double& InCircleRadius,
		const UE::Geometry::FFrame3d& InPlane,
		FVector& OutPointOnCircle)
	{
		using namespace UE::Geometry;

		FMatrix3d PlaneToScreen; 
		if (!ComputePlaneToScreenMatrix(InView, InPlane, PlaneToScreen))
		{
			return false; // Failed to compute the screen to plane matrix
		}

		const FVector PointOnPlane = PlaneToScreen.Inverse() * FVector(InScreenPoint.X, InScreenPoint.Y, 1.0f);
		if (PointOnPlane.IsNearlyZero())
		{
			return false;
		}

		const FVector2d::FReal PointU = PointOnPlane.Y;
		const FVector2d::FReal PointV = PointOnPlane.X;

		const double DistanceToPoint = FMath::Sqrt(PointU * PointU + PointV * PointV);
		if (FMath::IsNearlyZero(DistanceToPoint))
		{
			return false;
		}

		const double DistanceU = PointU / DistanceToPoint;
		const double DistanceV = PointV / DistanceToPoint;

		const FVector PlaneOrigin = InPlane.Origin;
		FVector UnusedPlaneNormal;
		FVector PlaneUp;
		FVector PlaneSide;
		UE::InteractiveToolsFramework::BreakPlaneFrame(InPlane, UnusedPlaneNormal, PlaneUp, PlaneSide);
		

		// Angle is relative to Up, and we flip the "Side" from the cartesian LHS convention, to UE RHS (when viewed towards the plane)
		OutPointOnCircle = PlaneOrigin + (InCircleRadius * DistanceV) * PlaneSide + (InCircleRadius * DistanceU) * PlaneUp;

		return true;
	}

	float ComputePositivePlaneSignForView(const UGizmoViewContext* InView, const UE::Geometry::FFrame3d& InPlane, const float InDefaultSign)
	{
		// Compute a positive rotation around the plane normal to check CW or CCW from view.
		// This then suffices for both angle and translation manipulations.
		constexpr double SampleAngleRad = FMath::DegreesToRadians(5.0);

		const FVector PlaneOrigin = InPlane.Origin;
		FVector PlaneNormal;
		FVector PlaneUp;
		FVector PlaneSide;
		UE::InteractiveToolsFramework::BreakPlaneFrame(InPlane, PlaneNormal, PlaneUp, PlaneSide);

		// Scale sample length so it's meaningful at different gizmo scales (should act more like pixels)
		const double SampleLength = FVector::Distance(InView->GetViewLocation(), PlaneOrigin) * 0.1;

		// Project samples onto plane
		const FVector SamplePoint1 = PlaneOrigin + PlaneUp * SampleLength;
		const FQuat Theta = FQuat(PlaneNormal, SampleAngleRad);
		const FVector SamplePoint2 = PlaneOrigin + Theta.RotateVector(PlaneUp) * SampleLength;

		// Project world samples to view
		const FVector2d PlaneOrigin2D = InView->WorldToPixel(PlaneOrigin);
		const FVector2d SamplePoint2D1 = InView->WorldToPixel(SamplePoint1);
		const FVector2d SamplePoint2D2 = InView->WorldToPixel(SamplePoint2);

		const FVector2d OriginToSample1 = SamplePoint2D1 - PlaneOrigin2D;
		const FVector2d OriginToSample2 = SamplePoint2D2 - PlaneOrigin2D;

		const double SignedArea = (OriginToSample1.X * OriginToSample2.Y) - (OriginToSample1.Y * OriginToSample2.X);

		if (OriginToSample1.SquaredLength() < UE_DOUBLE_KINDA_SMALL_NUMBER
			|| OriginToSample2.SquaredLength() < UE_DOUBLE_KINDA_SMALL_NUMBER)
		{
			// Samples are too close to origin, so we cannot determine the sign
			return InDefaultSign;
		}

		// Area is negative if the second point is ccw from the first, so we flip the resulting sign
		return (SignedArea > 0.0) ? +1.0f : -1.0f;
	}

	EAxisList::Type GetAxisListFromBools(const bool bInEnableX, const bool bInEnableY, const bool bInEnableZ)
	{
		EAxisList::Type Result = EAxisList::None;

		if (bInEnableX)
		{
			EnumAddFlags(Result, EAxisList::X);
		}

		if (bInEnableY)
		{
			EnumAddFlags(Result, EAxisList::Y);
		}

		if (bInEnableZ)
		{
			EnumAddFlags(Result, EAxisList::Z);
		}

		return Result;
	}

	bool GetBoolsFromAxisList(const EAxisList::Type InAxisList, bool bOutActiveAxis[3])
	{
		// Initialize all to false
		bOutActiveAxis[0] = bOutActiveAxis[1] = bOutActiveAxis[2] = false;

		const bool bEnableX = static_cast<uint8>(InAxisList) & static_cast<uint8>(EAxisList::X);
		const bool bEnableY = static_cast<uint8>(InAxisList) & static_cast<uint8>(EAxisList::Y);
		const bool bEnableZ = static_cast<uint8>(InAxisList) & static_cast<uint8>(EAxisList::Z);
		const bool bEnableAll = bEnableX && bEnableY && bEnableZ;

		bOutActiveAxis[0] = bEnableX;
		bOutActiveAxis[1] = bEnableY;
		bOutActiveAxis[2] = bEnableZ;

		return bEnableAll;
	}

	FVector GetAxisMultiplier(const EAxisList::Type InAxisList, const FVector& InScreenSpaceMultiplier)
	{
		if (InAxisList == EAxisList::Screen)
		{
			return InScreenSpaceMultiplier;
		}

		return FVector(
			(InAxisList & EAxisList::X) || (InAxisList & EAxisList::Left) ? 1.0f : 0.0f,
			(InAxisList & EAxisList::Y) || (InAxisList & EAxisList::Up) ? 1.0f : 0.0f,
			(InAxisList & EAxisList::Z) || (InAxisList & EAxisList::Forward) ? 1.0f : 0.0f);
	}

	FVector GetAxisCoordinateSystemMultiplier(const EAxisList::Type InAxisCoordinateSystem)
	{
		return (InAxisCoordinateSystem == EAxisList::LeftUpForward
				? FVector(1, -1, 1)
				: FVector::OneVector); 
	}

	FVector GetAxisCoordinateSystemMultiplier()
	{
		return GetAxisCoordinateSystemMultiplier(AxisDisplayInfo::GetAxisDisplayCoordinateSystem());
	}
}
