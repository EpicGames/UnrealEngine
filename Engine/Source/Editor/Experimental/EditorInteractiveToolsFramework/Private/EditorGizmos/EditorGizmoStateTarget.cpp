// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/EditorGizmoStateTarget.h"
#include "EditorGizmos/TransformGizmo.h"
#include "EditorModeManager.h"
#include "GizmoEdModeInterface.h"
#include "SnappingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorGizmoStateTarget)

void UEditorGizmoStateTarget::BeginUpdate()
{
	const TSharedPtr<FEditorModeTools> ModeTools = WeakModeTools.IsValid() ? WeakModeTools.Pin() : nullptr;
	if (ensure(ModeTools.IsValid()))
	{
		if (TransactionManager)
		{
			TransactionManager->BeginUndoTransaction(TransactionDescription);
		}

		FGizmoState State;
		State.TransformMode = GetTransformMode();
		(void)ModeTools->BeginTransform(State);
	}
}

void UEditorGizmoStateTarget::EndUpdate()
{
	// For compatibility with users of FSnappingUtils - previously executed in MouseDeltaTracker for legacy gizmos
	{
		constexpr bool bClearImmediately = true;
		FSnappingUtils::ClearSnappingHelpers(bClearImmediately);	
	}
	
	const TSharedPtr<FEditorModeTools> ModeTools = WeakModeTools.IsValid() ? WeakModeTools.Pin() : nullptr;
	if (ensure(ModeTools))
	{
		FGizmoState State;
		State.TransformMode = GetTransformMode();
		(void)ModeTools->EndTransform(State);

		if (TransactionManager)
		{
			TransactionManager->EndUndoTransaction();
		}
	}
}

void UEditorGizmoStateTarget::CancelUpdate()
{
	// For compatibility with users of FSnappingUtils - previously executed in MouseDeltaTracker for legacy gizmos
	{
		constexpr bool bClearImmediately = true;
		FSnappingUtils::ClearSnappingHelpers(bClearImmediately);	
	}

	const TSharedPtr<FEditorModeTools> ModeTools = WeakModeTools.IsValid() ? WeakModeTools.Pin() : nullptr;
	if (ensure(ModeTools))
	{
		FGizmoState State;
		State.TransformMode = GetTransformMode();
		(void)ModeTools->EndTransform(State);

		if (TransactionManager)
		{
			TransactionManager->CancelUndoTransaction();
		}
		
		(void)ModeTools->OnTransformCanceled(State);
	}
}

UEditorGizmoStateTarget* UEditorGizmoStateTarget::Construct(
	FEditorModeTools* InModeManager,
	const FText& InDescription,
	IToolContextTransactionProvider* TransactionManagerIn,
	UObject* Outer)
{
	UEditorGizmoStateTarget* NewTarget = NewObject<UEditorGizmoStateTarget>(Outer);
	NewTarget->WeakModeTools = InModeManager->AsShared();
	NewTarget->TransactionDescription = InDescription;

	// have to explicitly configure this because we only have IToolContextTransactionProvider pointer
	NewTarget->TransactionManager.SetInterface(TransactionManagerIn);
	NewTarget->TransactionManager.SetObject(CastChecked<UObject>(TransactionManagerIn));
		
	return NewTarget;
}

void UEditorGizmoStateTarget::SetTransformGizmo(UTransformGizmo* InGizmo)
{
	TransformGizmo = InGizmo;
}

EGizmoTransformMode UEditorGizmoStateTarget::GetTransformMode() const
{
	EGizmoTransformMode TransformMode = EGizmoTransformMode::None;
	if (TransformGizmo.IsValid() && TransformGizmo->TransformGizmoSource)
	{
		TransformMode = TransformGizmo->TransformGizmoSource->GetGizmoMode();
	}

	return TransformMode;
}
