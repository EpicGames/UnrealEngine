// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/EditorTRSGizmo.h"

#include "AnimationCoreLibrary.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/AxisSources.h"
#include "BaseGizmos/GizmoElementGroup.h"
#include "BaseGizmos/GizmoElementShapes.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoUtil.h"
#include "BaseGizmos/ViewBasedTransformAdjusters.h"
#include "Behaviors/SingleClickAndDragBehavior.h"
#include "Behaviors/ViewportClickDragBehavior.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "ContextObjectStore.h"
#include "DrawDebugHelpers.h"
#include "Debug/GizmoDebugProvider.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.inl"
#include "EditorGizmos/EditorGizmoMath.h"
#include "EditorGizmos/EditorTransformProxy.h"
#include "EditorGizmos/GizmoElementGimbal.h"
#include "EditorGizmos/GizmoElementRotateAxis.h"
#include "EditorGizmos/GizmoElementTranslate.h"
#include "EditorGizmos/TransformGizmo.h"
#include "EditorInteractiveToolsFrameworkStyle.h"
#include "EditorModeManager.h"
#include "TransformGizmoEditorSettings.h"
#include "EditorViewportClient.h"
#include "GizmoEdModeInterface.h"
#include "Elements/Framework/TypedElementCommonActions.h"
#include "Framework/Application/SlateApplication.h"
#include "GizmoElementRotateAxisSet.h"
#include "GizmoElementScaleAxis.h"
#include "GizmoElementScaleGroup.h"
#include "GizmoElementTranslateGroup.h"
#include "Intersection/IntersectionUtil.h"
#include "Materials/Material.h"
#include "Math/UnitConversion.h"
#include "Misc/AxisDisplayInfo.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "Selection.h"
#include "Widgets/SGizmoCursor.h"
#include "BaseBehaviors/KeyInputBehavior.h"
#include "Tools/AssetEditorContextInterface.h"
#include "UnrealEdGlobals.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Toolkits/IToolkitHost.h"
#include "ViewportInteractions/ViewportInteraction.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

#define LOCTEXT_NAMESPACE "UEditorTRSGizmo"

DEFINE_LOG_CATEGORY_STATIC(LogEditorTRSGizmo, Log, All);

namespace GizmoLocals
{

// NOTE these variables are not intended to remain here indefinitely.
// Their purpose is to experiment the new behavior of rotation gizmos.

static bool DoDebugDraw()
{
	constexpr const TCHAR* CVarName = TEXT("Gizmos.DebugDraw");
	if (IConsoleVariable* DebugDrawCVar = IConsoleManager::Get().FindConsoleVariable(CVarName, false))
	{
		return DebugDrawCVar->GetBool();
	}

	return false;
}

TAutoConsoleVariable<bool> CVarScreenspaceOrtho(
	TEXT("Editor.Gizmo.ScreenspaceOrtho"),
	false,
	TEXT("When enabled, orthographic manipulations are always in screen space")
);

TAutoConsoleVariable<int32> CVarAlternateSnappingElements(
	TEXT("Editor.Gizmo.AlternateSnappingElements"),
	1,
	TEXT("When set, uses alternative gizmo representations when snapping is enabled.\n")
	  TEXT("0 = Disabled\n")
	  TEXT("1 = Enabled\n")
	  TEXT("2 = Enabled, Swapped\n")
);

TAutoConsoleVariable<bool> CVarUseTranslationSnapRecalibration(
	TEXT("Editor.Gizmo.UseTranslationSnapRecalibration"),
	true,
	TEXT("When enabled, adjust tracked mouse position to account for snapping offsets")
);

}

extern ENGINE_API float GAverageFPS;

namespace UE::Editor::InteractiveToolsFramework
{
	namespace Private
	{
		namespace TransformGizmoLocals
		{
			constexpr bool bAllowMultiElementParts = true;
			constexpr bool bUseScreenspaceScaling = true;

			constexpr bool bLogStateChanges = false;
			constexpr bool bLogTranslation = false;
			constexpr bool bLogRotation = false;
			constexpr bool bLogScale = false;

			constexpr bool bEnableProfiling = false;

			constexpr FLazyName RenderName = TEXT("Render");
			constexpr FLazyName DrawHUDName = TEXT("DrawHUD");
			constexpr FLazyName DrawDebugName = TEXT("DrawDebug");
			constexpr FLazyName PendingFunctionExecutionName = TEXT("PendingFunctionExecution");
			constexpr FLazyName GetActiveTransformName = TEXT("GetActiveTransform");
			constexpr FLazyName DragTranslateAxisName = TEXT("DragTranslateAxis");
			constexpr FLazyName DragTranslatePlanarName = TEXT("DragTranslatePlanar");
			constexpr FLazyName DragTranslateUniformName = TEXT("DragScreenSpaceTranslate");
			constexpr FLazyName DragRotateAxisName = TEXT("DragRotateAxis");
			constexpr FLazyName DragGimbalRotateAxisName = TEXT("DragGimbalRotateAxis");
			constexpr FLazyName DragRotateScreenName = TEXT("DragRotateScreen");
			constexpr FLazyName DragRotateArcBallName = TEXT("DragRotateArcBall");
			constexpr FLazyName DragScaleAxisName = TEXT("DragScaleAxis");
			constexpr FLazyName DragScalePlanarName = TEXT("DragScalePlanar");
			constexpr FLazyName DragScaleUniformName = TEXT("DragScaleUniform");

			// Returns true if all axis have a uniform sign, and outputs that common sign to OutUniformSign.
			// @see: FComponentElementLevelEditorViewportInteractionCustomization::ValidateScale
			bool GetEffectiveSignForAxisList(const EAxisList::Type InAxisList, const FVector& InVector, int8& OutUniformSign)
			{
				// For LUF, Y = -1 is positive ("left")
				const FVector SignedVector = GizmoMath::GetAxisCoordinateSystemMultiplier();

				bool bEnabledAxes[3];
				GizmoMath::GetBoolsFromAxisList(InAxisList, bEnabledAxes);

				int32 NumEnabledAxis = 0;
				bool bFirstSignIsPositive = true;
				bool bHasUniformSign = true;

				for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
				{
					if (bEnabledAxes[AxisIndex])
					{
						bool bIsZero = FMath::IsNearlyZero(InVector[AxisIndex]);
						bool bIsPositive = bIsZero ? true : InVector[AxisIndex] * SignedVector[AxisIndex] > 0.0f;
						if (NumEnabledAxis == 0) // Check first
						{
							bFirstSignIsPositive = bIsPositive;
						}
						else
						{
							if (bFirstSignIsPositive != bIsPositive)
							{
								bHasUniformSign = false;
							}
						}

						OutUniformSign = bIsPositive ? 1 : -1;

						NumEnabledAxis++;
					}
				}

				return bHasUniformSign;
			}

			const FString& GetInteractionStateName(const EGizmoElementInteractionState InState)
			{
				static TArray<FString> ValueNames = {
					TEXT("None"),
					TEXT("Hovering"),
					TEXT("Interacting"),
					TEXT("Selected"),
					TEXT("Subdued")
				};

				return ValueNames[static_cast<int32>(InState)];
			}

			FName GetTransformPartName(const ETransformGizmoPartIdentifier InPart)
			{
				static TArray<FName> ValueNames;

				// Init
				if (ValueNames.IsEmpty())
				{
					ValueNames.Reserve(static_cast<int32>(ETransformGizmoPartIdentifier::Max));

					UEnum* Enum = StaticEnum<ETransformGizmoPartIdentifier>();
					if (Enum)
					{
						// First element is "Default", which shouldn't show with any text
						ValueNames.Emplace(NAME_None);
						for (int32 ValueIdx = 1; ValueIdx < static_cast<int32>(ETransformGizmoPartIdentifier::Max); ++ValueIdx)
						{
							ValueNames.Emplace(FName(Enum->GetDisplayNameTextByIndex(ValueIdx).ToString()));
						}
					}
				}

				return ValueNames[static_cast<int32>(InPart)];
			}

			// Wraps AxisDisplayInfo::GetAxisColor to adjust according to Style settings
			FLinearColor GetAxisColor(const EAxisList::Type InAxis, const FTransformGizmoStyle& InStyle)
			{
				EAxisList::Type SingleAxis = InAxis;
				if (Internal::IsAxisPlanar(InAxis))
				{
					SingleAxis = Internal::GetPlaneNormalAxis(InAxis);
				}

				return AxisDisplayInfo::GetAxisColor(SingleAxis);
			}

			// @todo: move to GizmoViewContext
			FRay3d ScreenPointToWorldRay(const FVector2D& InScreenPoint, const UGizmoViewContext* InGizmoViewContext)
			{
				if (!InGizmoViewContext)
				{
					return FRay3d();
				}

				FVector WorldPos;
				FVector WorldDirection;
				FSceneView::DeprojectScreenToWorld(
					InScreenPoint,
					InGizmoViewContext->GetUnscaledViewRect(),
					InGizmoViewContext->ViewMatrices.GetInvViewProjectionMatrix(), WorldPos, WorldDirection);

				return FRay3d(WorldPos, WorldDirection);
			}
			
			FVector ScreenPointToWorldPoint(const FVector2D& InScreenPoint, const UGizmoViewContext* InGizmoViewContext)
			{
				if (!InGizmoViewContext)
				{
					return FVector();
				}

				const FRay WorldRay = ScreenPointToWorldRay(InScreenPoint, InGizmoViewContext);
				return WorldRay.Origin; // It might be better to offset this by the Ray direction to avoid near-clipping, but we just used the origin previously - so stay with that.
			}

			// @todo: move to GizmoMath (as FVector2D overload)
			FVector2D NearestPointOnLine2D(
				const FVector2D& InLineOrigin,
				const FVector2D& InLineDirection,
				const FVector2D& InQueryPoint,
				float& OutLineParameter)
			{
				const double LineParameter = FVector2D::DotProduct((InQueryPoint - InLineOrigin), InLineDirection);
				const FVector2D NearestPoint = InLineOrigin + LineParameter * InLineDirection;
				OutLineParameter = static_cast<float>(LineParameter);
				return NearestPoint;
			}

			/** 
			 * Finds the closest point on a 3D line from a screen-point.
			 * First find the closest point on the line represented in 2D space,
			 * project it into a world ray, and then find the closest point on that ray.
			 */
			void NearestPointOnLineFromScreen(
				const FVector2D& InScreenPoint,
				const UGizmoViewContext* InGizmoViewContext,
				const FVector2D& InLineOrigin2D, const FVector2D& InLineDirection2D,
				const FVector& InLineOrigin3D, const FVector& InLineDirection3D,
				FVector& OutNearestPointOnLine)
			{
				float DistanceToNearestPoint = 0.0f;
				FVector2D NearestPoint2D = NearestPointOnLine2D(InLineOrigin2D, InLineDirection2D, InScreenPoint, DistanceToNearestPoint);

				const FRay3d RayFromScreenPoint = ScreenPointToWorldRay(NearestPoint2D, InGizmoViewContext);
				FVector UnusedNearestPointOnRay = FVector::ZeroVector;
				float UnusedLineParameter, UnusedRayParameter = 0.0f;
				::GizmoMath::NearestPointOnLineToRay(
					InLineOrigin3D, InLineDirection3D,
					RayFromScreenPoint.Origin, RayFromScreenPoint.Direction,
					OutNearestPointOnLine, UnusedLineParameter,
					UnusedNearestPointOnRay, UnusedRayParameter);
			}

			void LogStateChanged(const FName InStateName, const bool bInStateToggle)
			{
				if constexpr (bLogStateChanges)
				{
					UE_LOGF(LogEditorTRSGizmo, Verbose, "[%lli] %ls: %ls",
						GFrameCounter,
						*InStateName.ToString(),
						bInStateToggle ? TEXT("On") : TEXT("Off"));
				}
			}

			/** Applies values to the given Style, if those corresponding values are Defaults. */
			template <typename ToType
				UE_REQUIRES(std::is_base_of_v<FGizmoStyleBase, ToType>)>
			static void ApplyTransformStyleTo(const FTransformGizmoStyle& InFrom, ToType& InTo)
			{
				InTo.MinLineThickness = InTo.MinLineThickness == InTo.DefaultMinLineThickness ? InFrom.MinLineThickness : InTo.MinLineThickness;
				InTo.LineThicknessMultiplier = InTo.LineThicknessMultiplier == InTo.DefaultLineThicknessMultiplier ? InFrom.LineThicknessMultiplier : InTo.LineThicknessMultiplier;
				InTo.HoverLineThicknessMultiplier = InTo.HoverLineThicknessMultiplier == InTo.DefaultHoverLineThicknessMultiplier ? InFrom.HoverLineThicknessMultiplier : InTo.HoverLineThicknessMultiplier;
			}
		}

		struct FTransformGizmoAccessorPrivate
		{
			template <typename FromEnum, typename ToEnum>
			static void SetCorrespondingFlag(const FromEnum InSourceValue, const FromEnum InSourceFlag, ToEnum& OutTargetValue, const ToEnum InTargetFlag)
			{
				if (EnumHasAnyFlags(InSourceValue, InSourceFlag))
				{
					EnumAddFlags(OutTargetValue, InTargetFlag);
				}

				// @note: Target is initialized as None, so we don't need to also remove flags here
			}

#pragma region Translate
			static FGizmoElementTranslateAxisStyleOverride MakeTranslateAxisStyleOverride(UEditorTRSGizmo& InGizmo, const EAxis::Type InAxis)
			{
				FGizmoElementTranslateAxisStyleOverride StyleOverride;
				StyleOverride.Colors = InGizmo.GetAxisColors(InAxis);
				StyleOverride.Materials = InGizmo.GetAxisMaterials(InAxis);
				StyleOverride.DeltaMaterial = InGizmo.GetAxisMaterialTranslucent(InAxis);
				StyleOverride.VertexColorMaterial = InGizmo.TransparentVertexColorMaterial;

				return StyleOverride;
			}

			static UGizmoElementTranslateGroup::FAxisParameters MakeTranslateAxisParameters(UEditorTRSGizmo& InGizmo, const ETransformGizmoPartIdentifier InPartId, const EAxis::Type InAxis)
			{
				return UGizmoElementTranslateGroup::FAxisParameters{
					static_cast<uint32>(InPartId),
					EAxisList::FromAxis(InAxis),
					MakeTranslateAxisStyleOverride(InGizmo, InAxis)
				};
			}

			static FGizmoElementTranslatePlanarStyleOverride MakeTranslatePlanarStyleOverride(UEditorTRSGizmo& InGizmo, const EAxisList::Type InAxis, const EAxis::Type InNormalAxis)
			{
				FGizmoElementTranslatePlanarStyleOverride StyleOverride;
				StyleOverride.Colors = InGizmo.GetAxisColors(InAxis);
				StyleOverride.Materials = InGizmo.GetAxisMaterials(InNormalAxis);
				StyleOverride.VertexColorMaterial = InGizmo.TransparentVertexColorMaterial;

				return StyleOverride;
			}

			static UGizmoElementTranslateGroup::FPlanarParameters MakeTranslatePlanarParameters(UEditorTRSGizmo& InGizmo, const ETransformGizmoPartIdentifier InPartId, const EAxisList::Type InAxis, const EAxis::Type InNormalAxis)
			{
				return UGizmoElementTranslateGroup::FPlanarParameters{
					static_cast<uint32>(InPartId),
					InAxis,
					MakeTranslatePlanarStyleOverride(InGizmo, InAxis, InNormalAxis)
				};
			}
			
			static EGizmoElementTranslateShowFlags GizmoShowFlagsToTranslateShowFlags(const ETransformGizmoShowFlags InShowFlags)
			{
				EGizmoElementTranslateShowFlags TranslateShowFlags = EGizmoElementTranslateShowFlags::None;

				SetCorrespondingFlag<ETransformGizmoShowFlags, EGizmoElementTranslateShowFlags>(
					InShowFlags, 
					ETransformGizmoShowFlags::DeltaLabel, 
					TranslateShowFlags, 
					EGizmoElementTranslateShowFlags::DeltaLabel);
				
				SetCorrespondingFlag<ETransformGizmoShowFlags, EGizmoElementTranslateShowFlags>(
					InShowFlags, 
					ETransformGizmoShowFlags::DeltaLine, 
					TranslateShowFlags, 
					EGizmoElementTranslateShowFlags::DeltaLine);
				
				SetCorrespondingFlag<ETransformGizmoShowFlags, EGizmoElementTranslateShowFlags>(
					InShowFlags, 
					ETransformGizmoShowFlags::DeltaOrigin, 
					TranslateShowFlags, 
					EGizmoElementTranslateShowFlags::DeltaOrigin);

				return TranslateShowFlags;
			}

			static FGizmoElementTranslateAxisStyle MakeTranslateAxisStyle(const FTransformGizmoStyle& InParentStyle, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementTranslateAxisStyle ElementStyle = InParentStyle.TranslateStyle.AxisStyle;
				TransformGizmoLocals::ApplyTransformStyleTo(InParentStyle, ElementStyle);
				ElementStyle.SizeCoefficient = InSizeCoefficient;
				ElementStyle.AxisLengthMultiplier = InParentStyle.GetModifiedAxisSizeMultiplier();
				ElementStyle.HandleSizeMultiplier = InParentStyle.HandleSizeMultiplier;
				ElementStyle.VertexColorMaterial = InVertexColorMaterial;
				ElementStyle.ShowFlags = GizmoShowFlagsToTranslateShowFlags(static_cast<ETransformGizmoShowFlags>(InParentStyle.ShowFlags));
				ElementStyle.DeltaTextBackgroundColor = InParentStyle.DeltaTextBackgroundColor;

				return ElementStyle;
			}

			static FGizmoElementTranslateAxisStyle MakeTranslateAxisStyle(const FTransformGizmoStyle& InParentStyle, const FGizmoPerStateValueMaterialVariant& InMaterials, const FGizmoPerStateValueLinearColor& InColors, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementTranslateAxisStyle ElementStyle = MakeTranslateAxisStyle(InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.Materials = InMaterials;
				ElementStyle.Colors = InColors;

				return ElementStyle;
			}

			static FGizmoElementTranslatePlanarStyle MakeTranslatePlanarStyle(const FTransformGizmoStyle& InParentStyle, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementTranslatePlanarStyle ElementStyle = InParentStyle.TranslateStyle.PlanarStyle;
				TransformGizmoLocals::ApplyTransformStyleTo(InParentStyle, ElementStyle);
				ElementStyle.SizeCoefficient = InSizeCoefficient;
				ElementStyle.VertexColorMaterial = InVertexColorMaterial;
				ElementStyle.OffsetFromOrigin = InParentStyle.PlanarAxisOffsetFromOrigin;
				ElementStyle.SizeMultiplier = InParentStyle.HandleSizeMultiplier;

				return ElementStyle;
			}

			static FGizmoElementTranslatePlanarStyle MakeTranslatePlanarStyle(const FTransformGizmoStyle& InParentStyle, const FGizmoPerStateValueMaterialVariant& InMaterials, const FGizmoPerStateValueLinearColor& InColors, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementTranslatePlanarStyle ElementStyle = MakeTranslatePlanarStyle(InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.Materials = InMaterials;
				ElementStyle.Colors = InColors;

				return ElementStyle;
			}

			static FGizmoElementTranslateStyle MakeTranslateStyle(const FTransformGizmoStyle& InParentStyle, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementTranslateStyle ElementStyle = InParentStyle.TranslateStyle;
				TransformGizmoLocals::ApplyTransformStyleTo(InParentStyle, ElementStyle);
				ElementStyle.SizeCoefficient = InSizeCoefficient;
				ElementStyle.AxisStyle = MakeTranslateAxisStyle(InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.PlanarStyle = MakeTranslatePlanarStyle(InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.UniformStyle.SizeMultiplier = InParentStyle.HandleSizeMultiplier;
				ElementStyle.UniformStyle.LineColors.Default = InParentStyle.GreyColor;
				ElementStyle.UniformStyle.Materials.Default = FGizmoMaterialVariant{ InVertexColorMaterial, { } };
				ElementStyle.UniformStyle.Materials.Subdue = ElementStyle.UniformStyle.Materials.Default;
				ElementStyle.ShowFlags = GizmoShowFlagsToTranslateShowFlags(static_cast<ETransformGizmoShowFlags>(InParentStyle.ShowFlags));

				return ElementStyle;
			}

			static FGizmoElementTranslateStyle MakeTranslateStyle(const FTransformGizmoStyle& InParentStyle, const FGizmoPerStateValueMaterialVariant& InMaterials, const FGizmoPerStateValueLinearColor& InColors, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementTranslateStyle ElementStyle = MakeTranslateStyle(InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.AxisStyle = MakeTranslateAxisStyle(InParentStyle, InMaterials, InColors, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.PlanarStyle = MakeTranslatePlanarStyle(InParentStyle, InMaterials, InColors, InVertexColorMaterial, InSizeCoefficient);

				return ElementStyle;
			}
#pragma endregion Translate

#pragma region Rotate
			static FGizmoElementRotateAxisStyleOverride MakeRotateAxisStyleOverride(UEditorTRSGizmo& InGizmo, const EAxis::Type InAxis)
			{
				FGizmoElementRotateAxisStyleOverride StyleOverride;
				StyleOverride.Colors = InGizmo.GetAxisColors(InAxis);
				StyleOverride.Materials = InGizmo.GetAxisMaterials(InAxis);
				StyleOverride.DeltaMaterial = InGizmo.GetAxisMaterialTranslucent(InAxis);
				StyleOverride.VertexColorMaterial = InGizmo.TransparentVertexColorMaterial;

				return StyleOverride;
			}

			static UGizmoElementRotateAxisSet::FAxisParameters MakeRotateAxisParameters(UEditorTRSGizmo& InGizmo, const ETransformGizmoPartIdentifier InPartId, const EAxis::Type InAxis)
			{
				return UGizmoElementRotateAxisSet::FAxisParameters{
					static_cast<uint32>(InPartId),
					EAxisList::FromAxis(InAxis),
					MakeRotateAxisStyleOverride(InGizmo, InAxis)
				};
			}

			static EGizmoElementRotateShowFlags GizmoShowFlagsToRotateShowFlags(const ETransformGizmoShowFlags InShowFlags)
			{
				EGizmoElementRotateShowFlags RotateShowFlags = EGizmoElementRotateShowFlags::None;

				SetCorrespondingFlag<ETransformGizmoShowFlags, EGizmoElementRotateShowFlags>(
					InShowFlags, 
					ETransformGizmoShowFlags::DeltaLabel, 
					RotateShowFlags, 
					EGizmoElementRotateShowFlags::DeltaLabel);

				SetCorrespondingFlag<ETransformGizmoShowFlags, EGizmoElementRotateShowFlags>(
					InShowFlags, 
					ETransformGizmoShowFlags::DeltaArc, 
					RotateShowFlags, 
					EGizmoElementRotateShowFlags::DeltaArc);

				return RotateShowFlags;
			}

			static FGizmoElementRotateAxisStyle MakeRotateAxisStyle(const FTransformGizmoStyle& InParentStyle, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementRotateAxisStyle ElementStyle = InParentStyle.RotateStyle.AxisStyle;
				TransformGizmoLocals::ApplyTransformStyleTo(InParentStyle, ElementStyle);
				ElementStyle.SizeCoefficient = InSizeCoefficient;
				ElementStyle.RadiusMultiplier = InParentStyle.GetModifiedAxisSizeMultiplier();
				ElementStyle.CursorColor = FLinearColor::Black;
				ElementStyle.VertexColorMaterial = InVertexColorMaterial;
				ElementStyle.ShowFlags = GizmoShowFlagsToRotateShowFlags(static_cast<ETransformGizmoShowFlags>(InParentStyle.ShowFlags));
				ElementStyle.DeltaTextBackgroundColor = InParentStyle.DeltaTextBackgroundColor;

				return ElementStyle;
			}

			static FGizmoElementRotateAxisStyle MakeRotateAxisStyle(const FTransformGizmoStyle& InParentStyle, const FGizmoPerStateValueMaterialVariant& InMaterials, const FGizmoPerStateValueLinearColor& InColors, UMaterialInterface* InDeltaMaterial, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementRotateAxisStyle ElementStyle = MakeRotateAxisStyle(InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.Materials = InMaterials;
				ElementStyle.Colors = InColors;
				ElementStyle.DeltaMaterial = InDeltaMaterial;

				return ElementStyle;
			}
#pragma endregion Rotate

#pragma region Scale
			static FGizmoElementScaleAxisStyleOverride MakeScaleAxisStyleOverride(UEditorTRSGizmo& InGizmo, const EAxis::Type InAxis)
			{
				FGizmoElementScaleAxisStyleOverride StyleOverride;
				StyleOverride.Colors = InGizmo.GetAxisColors(InAxis);
				StyleOverride.Materials = InGizmo.GetAxisMaterials(InAxis);
				StyleOverride.VertexColorMaterial = InGizmo.TransparentVertexColorMaterial;

				return StyleOverride;
			}

			static UGizmoElementScaleGroup::FAxisParameters MakeScaleAxisParameters(UEditorTRSGizmo& InGizmo, const ETransformGizmoPartIdentifier InPartId, const EAxis::Type InAxis)
			{
				return UGizmoElementScaleGroup::FAxisParameters{
					static_cast<uint32>(InPartId),
					EAxisList::FromAxis(InAxis),
					MakeScaleAxisStyleOverride(InGizmo, InAxis)
				};
			}
			
			static EGizmoElementScaleShowFlags GizmoShowFlagsToScaleShowFlags(const ETransformGizmoShowFlags InShowFlags)
			{
				EGizmoElementScaleShowFlags ScaleShowFlags = EGizmoElementScaleShowFlags::None;

				SetCorrespondingFlag<ETransformGizmoShowFlags, EGizmoElementScaleShowFlags>(
					InShowFlags, 
					ETransformGizmoShowFlags::DeltaLabel, 
					ScaleShowFlags, 
					EGizmoElementScaleShowFlags::DeltaLabel);

				SetCorrespondingFlag<ETransformGizmoShowFlags, EGizmoElementScaleShowFlags>(
					InShowFlags, 
					ETransformGizmoShowFlags::DeltaLine, 
					ScaleShowFlags, 
					EGizmoElementScaleShowFlags::DeltaLine);

				return ScaleShowFlags;
			}

			static FGizmoElementScaleAxisStyle MakeScaleAxisStyle(const FTransformGizmoStyle& InParentStyle, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementScaleAxisStyle ElementStyle = InParentStyle.ScaleStyle.AxisStyle;
				TransformGizmoLocals::ApplyTransformStyleTo(InParentStyle, ElementStyle);
				ElementStyle.SizeCoefficient = InSizeCoefficient;
				ElementStyle.AxisLengthMultiplier = InParentStyle.GetModifiedAxisSizeMultiplier();
				ElementStyle.HandleSizeMultiplier  = InParentStyle.HandleSizeMultiplier;
				ElementStyle.VertexColorMaterial = InVertexColorMaterial;
				ElementStyle.ShowFlags = GizmoShowFlagsToScaleShowFlags(static_cast<ETransformGizmoShowFlags>(InParentStyle.ShowFlags));
				ElementStyle.DeltaTextBackgroundColor = InParentStyle.DeltaTextBackgroundColor;

				return ElementStyle;
			}

			static FGizmoElementScaleAxisStyle MakeScaleAxisStyle(const FTransformGizmoStyle& InParentStyle, const FGizmoPerStateValueMaterialVariant& InMaterials, const FGizmoPerStateValueLinearColor& InColors, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementScaleAxisStyle ElementStyle = MakeScaleAxisStyle(InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.Materials = InMaterials;
				ElementStyle.Colors = InColors;

				return ElementStyle;
			}

			static FGizmoElementScalePlanarStyleOverride MakeScalePlanarStyleOverride(UEditorTRSGizmo& InGizmo, const EAxisList::Type InAxis, const EAxis::Type InNormalAxis)
			{
				FGizmoElementScalePlanarStyleOverride StyleOverride;
				StyleOverride.Colors = InGizmo.GetAxisColors(InAxis);
				StyleOverride.Materials = InGizmo.GetAxisMaterials(InNormalAxis);
				StyleOverride.VertexColorMaterial = InGizmo.TransparentVertexColorMaterial;

				return StyleOverride;
			}

			static UGizmoElementScaleGroup::FPlanarParameters MakeScalePlanarParameters(UEditorTRSGizmo& InGizmo, const ETransformGizmoPartIdentifier InPartId, const EAxisList::Type InAxis, const EAxis::Type InNormalAxis)
			{
				return UGizmoElementScaleGroup::FPlanarParameters{
					static_cast<uint32>(InPartId),
					InAxis,
					MakeScalePlanarStyleOverride(InGizmo, InAxis, InNormalAxis)
				};
			}

			static FGizmoElementScalePlanarStyle MakeScalePlanarStyle(const FTransformGizmoStyle& InParentStyle, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementScalePlanarStyle ElementStyle = InParentStyle.ScaleStyle.PlanarStyle;
				TransformGizmoLocals::ApplyTransformStyleTo(InParentStyle, ElementStyle);
				ElementStyle.SizeCoefficient = InSizeCoefficient;
				ElementStyle.VertexColorMaterial = InVertexColorMaterial;
				ElementStyle.OffsetFromOrigin = InParentStyle.PlanarAxisOffsetFromOrigin;
				ElementStyle.SizeMultiplier = InParentStyle.HandleSizeMultiplier;

				return ElementStyle;
			}

			static FGizmoElementScalePlanarStyle MakeScalePlanarStyle(const FTransformGizmoStyle& InParentStyle, const FGizmoPerStateValueMaterialVariant& InMaterials, const FGizmoPerStateValueLinearColor& InColors, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementScalePlanarStyle ElementStyle = MakeScalePlanarStyle(InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.Materials = InMaterials;
				ElementStyle.Colors = InColors;

				return ElementStyle;
			}

			static FGizmoElementScaleStyle MakeScaleStyle(const UEditorTRSGizmo& InGizmo, const FTransformGizmoStyle& InParentStyle, UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementScaleStyle ElementStyle = InParentStyle.ScaleStyle;
				TransformGizmoLocals::ApplyTransformStyleTo(InParentStyle, ElementStyle);
				ElementStyle.SizeCoefficient = InSizeCoefficient;
				ElementStyle.AxisStyle = MakeScaleAxisStyle(InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.PlanarStyle = MakeScalePlanarStyle(InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.UniformStyle.SizeMultiplier = InParentStyle.HandleSizeMultiplier;
				ElementStyle.UniformStyle.Materials.Default = FGizmoMaterialVariant{ InGizmo.UniformScaleMaterial, { } };
				ElementStyle.ShowFlags = GizmoShowFlagsToScaleShowFlags(static_cast<ETransformGizmoShowFlags>(InParentStyle.ShowFlags));

				return ElementStyle;
			}

			static FGizmoElementScaleStyle MakeScaleStyle(const UEditorTRSGizmo& InGizmo, const FTransformGizmoStyle& InParentStyle, const FGizmoPerStateValueMaterialVariant& InMaterials, const FGizmoPerStateValueLinearColor& InColors,  UMaterialInterface* InVertexColorMaterial, const float InSizeCoefficient)
			{
				FGizmoElementScaleStyle ElementStyle = MakeScaleStyle(InGizmo, InParentStyle, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.AxisStyle = MakeScaleAxisStyle(InParentStyle, InMaterials, InColors, InVertexColorMaterial, InSizeCoefficient);
				ElementStyle.PlanarStyle = MakeScalePlanarStyle(InParentStyle, InMaterials, InColors, InVertexColorMaterial, InSizeCoefficient);

				return ElementStyle;
			}
#pragma endregion Scale
		};
	}

	/** Contains one or more named execution timers. */
	class FTransformGizmoProfiler : public TSharedFromThis<FTransformGizmoProfiler>
	{
	private:
		struct FTimer
		{
			explicit FTimer(const FName InName)
				: Name(InName)
			{

			}

			void Begin()
			{
				ExecutionTimeStart = FPlatformTime::Cycles64();

				FrameCountStart = GFrameCounter;
				FrameRateStart = GAverageFPS;

				bIsTiming = true;
			}

			void End()
			{
				const double ExecutionTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - ExecutionTimeStart);

				TotalExecutionTime += ExecutionTime;
				ExecutionCount++;

				AverageExecutionTime = TotalExecutionTime / static_cast<double>(ExecutionCount);

				FrameCountDelta = GFrameCounter - FrameCountStart;
				FrameRateDelta = GAverageFPS - FrameRateStart;

				bIsTiming = false;
			}

			/** Resets the timer, clearing current accumulators etc. */
			void Reset()
			{
				bIsTiming = false;
				TotalExecutionTime = 0;
				ExecutionCount = 0;
				AverageExecutionTime = 0.0;
			}

			FName GetName() const
			{
				return Name;
			}

			FString GetAverageExecutionTimeStr() const
			{
				FNumericUnit<double> DisplayUnit = FUnitConversion::QuantizeUnitsToBestFit<double>(GetAverageExecutionTime(), EUnit::Milliseconds);

				return FString::Format(TEXT("{0}{1}"), FStringFormatOrderedArguments({
					FString::Printf(TEXT("%.2f"), DisplayUnit.Value),
					FUnitConversion::GetUnitDisplayString(DisplayUnit.Units)
					}));
			}

			double GetAverageExecutionTime() const
			{
				return AverageExecutionTime;
			}

			std::atomic_bool bIsTiming = false;

			FName Name;
			float FrameRateStart = 0.0f;
			uint64 FrameCountStart = 0;
			uint64 ExecutionTimeStart = 0;
			double AverageExecutionTime = 0.0;
			double TotalExecutionTime = 0.0;
			uint64 ExecutionCount = 0;
			int64 FrameCountDelta = 0;
			float FrameRateDelta = 0.0f;
		};

	public:
		/** Convenience object to begin profiling on creation, and end on destruction. */
		struct FScopedTimer
		{
			explicit FScopedTimer(const TSharedPtr<FTimer>& InTimer)
				: Timer(InTimer)
			{
				if (Private::TransformGizmoLocals::bEnableProfiling && Timer.IsValid())
				{
					Timer->Begin();
				}
			}

			~FScopedTimer()
			{
				if (Private::TransformGizmoLocals::bEnableProfiling && Timer.IsValid())
				{
					Timer->End();
				}
			}

		private:
			TSharedPtr<FTimer> Timer;
		};

	public:
		FScopedTimer BeginScoped(const FName InTimerName)
		{
			if (!Private::TransformGizmoLocals::bEnableProfiling)
			{
				return FScopedTimer(nullptr);
			}

			return FScopedTimer(FindOrAddTimer(InTimerName));
		}

		void Begin(const FName InTimerName)
		{
			if (!Private::TransformGizmoLocals::bEnableProfiling)
			{
				return;
			}

			const TSharedPtr<FTimer> Timer = FindOrAddTimer(InTimerName);
			Timer->Begin();
		}

		void End(const FName InTimerName)
		{
			if (!Private::TransformGizmoLocals::bEnableProfiling)
			{
				return;
			}

			if (const TSharedPtr<FTimer>* FoundTimer = Timers.Find(InTimerName))
			{
				if ((*FoundTimer)->bIsTiming)
				{
					(*FoundTimer)->End();
				}
				else
				{
					ensureAlwaysMsgf(false, TEXT("FGizmoProfiler::End: Timer with name '%s' was not started. Call Begin before you call End."), *InTimerName.ToString());
				}
			}
			else
			{
				ensureAlwaysMsgf(false, TEXT("FGizmoProfiler::End: No timer found with name '%s'"), *InTimerName.ToString());
			}
		}

		/** Resets and restarts the given timer, clearing current accumulators etc. */
		void Reset(const FName InTimerName)
		{
			if (!Private::TransformGizmoLocals::bEnableProfiling)
			{
				return;
			}

			if (const TSharedPtr<FTimer>* FoundTimer = Timers.Find(InTimerName))
			{
				(*FoundTimer)->Reset();
				(*FoundTimer)->Begin();
			}
		}

		FString GetAverageExecutionTimeStr(const FName InTimerName) const
		{
			if (!Private::TransformGizmoLocals::bEnableProfiling)
			{
				return FString();
			}

			if (const TSharedPtr<FTimer>* FoundTimer = Timers.Find(InTimerName))
			{
				return (*FoundTimer)->GetAverageExecutionTimeStr();
			}

			return FString();
		}

		double GetAverageExecutionTime(const FName InTimerName) const
		{
			if (!Private::TransformGizmoLocals::bEnableProfiling)
			{
				return 0.0;
			}

			if (const TSharedPtr<FTimer>* FoundTimer = Timers.Find(InTimerName))
			{
				return (*FoundTimer)->GetAverageExecutionTime();
			}

			return 0.0;
		}

	private:
		TSharedPtr<FTimer> FindOrAddTimer(const FName InTimerName)
		{
			if (const TSharedPtr<FTimer>* FoundTimer = Timers.Find(InTimerName))
			{
				return *FoundTimer;
			}

			return Timers.Emplace(InTimerName, MakeShared<FTimer>(InTimerName));
		}

	private:
		TMap<FName, TSharedPtr<FTimer>> Timers;
	};


	static constexpr float NudgeInitialRepeatDelay = 0.2f;
	static constexpr float NudgeRepeatInterval = 0.1f;

	// Retrieve world space interaction axis for the specified transform mode and nudge direction
	FVector GetInteractionWorldDirection(const UGizmoViewContext* InGizmoViewContext, const FKey& InDirection, EGizmoTransformMode InTransformMode)
	{
		if (!InGizmoViewContext)
		{
			return FVector::ZeroVector;
		}

		float VerticalComponent = InDirection == EKeys::Up ? -1.0f : InDirection == EKeys::Down ? 1.0f : 0.0f;
		float HorizontalComponent = InDirection == EKeys::Left ? -1.0f : InDirection == EKeys::Right ? 1.0f : 0.0f;

		// Specific case to get proper rotation
		if (InTransformMode == EGizmoTransformMode::Rotate)
		{
			Swap(VerticalComponent, HorizontalComponent);
			HorizontalComponent *= -1.0f;
		}

		// Pixel to Screen
		const FVector4 ScreenTransformedEnd = InGizmoViewContext->PixelToScreen(FVector2D(HorizontalComponent, VerticalComponent), false);
		const FVector4 ScreenTransformedStart = InGizmoViewContext->PixelToScreen(FVector2D::ZeroVector);

		// Screen to World
		const FVector TransformedEnd = InGizmoViewContext->ViewMatrices.GetInvViewProjectionMatrix().TransformFVector4(ScreenTransformedEnd);
		const FVector TransformedStart = InGizmoViewContext->ViewMatrices.GetInvViewProjectionMatrix().TransformFVector4(ScreenTransformedStart);

		return (TransformedEnd - TransformedStart).GetSafeNormal();
	}

	TNudgeDelta<FVector> GetScreenSpaceNudgeTransformAxis(const UGizmoViewContext* InGizmoViewContext, const FTransform& InTargetTransform, UEditorTRSGizmo::ENudgeDirection InNudgeDirection, const FKey& InNudgeKey)
	{
		FVector WorldDirection = FVector::ZeroVector;
		if (InNudgeDirection == UEditorTRSGizmo::ENudgeDirection::Secondary)
		{
			// Secondary interaction uses camera view direction
			WorldDirection = InGizmoViewContext->GetViewDirection();

			if (InNudgeKey == EKeys::Down)
			{
				WorldDirection *= -1.0f;
			}
		}
		else
		{
			WorldDirection = GetInteractionWorldDirection(InGizmoViewContext, InNudgeKey, EGizmoTransformMode::Translate);
		}

		const FVector XAxis = InTargetTransform.GetUnitAxis(EAxis::X);
		const FVector YAxis = InTargetTransform.GetUnitAxis(EAxis::Y);
		const FVector ZAxis = InTargetTransform.GetUnitAxis(EAxis::Z);

		FVector DotProducts;
		DotProducts.X = FVector::DotProduct(XAxis, WorldDirection);
		DotProducts.Y = FVector::DotProduct(YAxis, WorldDirection);
		DotProducts.Z = FVector::DotProduct(ZAxis, WorldDirection);

		// Max3 will tell us which axis better matches the current vertical interaction
		const int32 MaxIndex = FMath::Max3Index(FMath::Abs(DotProducts.X), FMath::Abs(DotProducts.Y), FMath::Abs(DotProducts.Z));

		FVector TransformAxis = FVector::ZeroVector;
		TransformAxis[MaxIndex] = DotProducts[MaxIndex] > 0.0f ? 1.0f : -1.0f;
		
		const EAxisList::Type Axis = [MaxIndex]
		{
			switch (MaxIndex)
			{
			case 0:
				return EAxisList::X;
			case 1:
				return EAxisList::Y;
			case 2:
				return EAxisList::Z;
			default:
				return EAxisList::None;
			}	
		}();
		
		return { TransformAxis, Axis };
	}

	TNudgeDelta<FVector> GetAbsoluteNudgeTransformAxis(UEditorTRSGizmo::ENudgeDirection InNudgeDirection, const FKey& InNudgeKey)
	{
		FVector TransformAxis = FVector::ZeroVector;
		EAxisList::Type Axis = EAxisList::None;
		if (InNudgeDirection == UEditorTRSGizmo::Horizontal)
		{
			TransformAxis.X = InNudgeKey == EKeys::Right ? 1.0f : InNudgeKey == EKeys::Left ? -1.0f : 0.0f;
			Axis = EAxisList::X;
		}
		else
		{
			const float Direction = InNudgeKey == EKeys::Up ? 1.0f : InNudgeKey == EKeys::Down ? -1.0f : 0.0f;
			if (InNudgeDirection == UEditorTRSGizmo::Vertical)
			{
				TransformAxis.Y = Direction;
				Axis = EAxisList::Y;
			}
			else if (InNudgeDirection == UEditorTRSGizmo::Secondary)
			{
				TransformAxis.Z = Direction;
				Axis = EAxisList::Z;
			}
		}
		
		return { TransformAxis, Axis };
	}
}

#pragma region FGizmoElementInteraction
TMap<ETransformGizmoPartIdentifier, UEditorTRSGizmo::FGizmoElementInteraction::FPartSet> UEditorTRSGizmo::FGizmoElementInteraction::HoverParts =
{
	{
		ETransformGizmoPartIdentifier::TranslateXAxis,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll})
		},
	{
		ETransformGizmoPartIdentifier::TranslateYAxis,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll})
	},
	{
		ETransformGizmoPartIdentifier::TranslateZAxis,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll})
		},{
		ETransformGizmoPartIdentifier::TranslateXYPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll})
	},
	{
		ETransformGizmoPartIdentifier::TranslateXYPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll})
	},
	{
		ETransformGizmoPartIdentifier::TranslateYZPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll})
	},
	{
		ETransformGizmoPartIdentifier::TranslateXZPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll})
	}
};

TMap<ETransformGizmoPartIdentifier, UEditorTRSGizmo::FGizmoElementInteraction::FPartSet> UEditorTRSGizmo::FGizmoElementInteraction::ClickDragParts =
{
	{
		ETransformGizmoPartIdentifier::TranslateXAxis,
		FPartSet(
			{
				ETransformGizmoPartIdentifier::TranslateAll
			})
	},
	{
		ETransformGizmoPartIdentifier::TranslateYAxis,
		FPartSet(
			{
				ETransformGizmoPartIdentifier::TranslateAll
			})
	},
	{
		ETransformGizmoPartIdentifier::TranslateZAxis,
		FPartSet(
			{
				ETransformGizmoPartIdentifier::TranslateAll
			})
	},
	{
		ETransformGizmoPartIdentifier::TranslateXYPlanar,
		FPartSet(
			{
				ETransformGizmoPartIdentifier::TranslateAll
			})
	},
	{
		ETransformGizmoPartIdentifier::TranslateXYPlanar,
		FPartSet(
			{
				ETransformGizmoPartIdentifier::TranslateAll
			})
	},
	{
		ETransformGizmoPartIdentifier::TranslateYZPlanar,
		FPartSet(
			{
				ETransformGizmoPartIdentifier::TranslateAll
			})
	},
	{
		ETransformGizmoPartIdentifier::TranslateXZPlanar,
		FPartSet(
			{
				ETransformGizmoPartIdentifier::TranslateAll
			})
	},
	{
		ETransformGizmoPartIdentifier::TranslateScreenSpace,
		FPartSet(
			{
				ETransformGizmoPartIdentifier::TranslateAll
			})
	}
};

TMap<ETransformGizmoPartIdentifier, UEditorTRSGizmo::FGizmoElementInteraction::FPartSet> UEditorTRSGizmo::FGizmoElementInteraction::SubdueOnInteractParts =
{
	{
		ETransformGizmoPartIdentifier::RotateXAxis,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::RotateXAxis,
			ETransformGizmoPartIdentifier::RotateZGimbal,
			ETransformGizmoPartIdentifier::RotateXAxis))
	},
	{
		ETransformGizmoPartIdentifier::RotateYAxis,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::RotateXAxis,
			ETransformGizmoPartIdentifier::RotateZGimbal,
			ETransformGizmoPartIdentifier::RotateYAxis))
	},
	{
		ETransformGizmoPartIdentifier::RotateZAxis,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::RotateXAxis,
			ETransformGizmoPartIdentifier::RotateZGimbal,
			ETransformGizmoPartIdentifier::RotateZAxis))
	},
	{
		ETransformGizmoPartIdentifier::RotateXGimbal,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::RotateXAxis,
			ETransformGizmoPartIdentifier::RotateZGimbal,
			ETransformGizmoPartIdentifier::RotateXGimbal))
	},
	{
		ETransformGizmoPartIdentifier::RotateYGimbal,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::RotateXAxis,
			ETransformGizmoPartIdentifier::RotateZGimbal,
			ETransformGizmoPartIdentifier::RotateYGimbal))
	},
	{
		ETransformGizmoPartIdentifier::RotateZGimbal,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::RotateXAxis,
			ETransformGizmoPartIdentifier::RotateZGimbal,
			ETransformGizmoPartIdentifier::RotateZGimbal))
	},
};

TMap<ETransformGizmoPartIdentifier, UEditorTRSGizmo::FGizmoElementInteraction::FPartSet> UEditorTRSGizmo::FGizmoElementInteraction::HideOnInteractParts =
{
	{
		ETransformGizmoPartIdentifier::TranslateXAxis,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::TranslateXAxis,
			ETransformGizmoPartIdentifier::TranslateScreenSpace,
			ETransformGizmoPartIdentifier::TranslateXAxis))
	},
	{
		ETransformGizmoPartIdentifier::TranslateYAxis,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::TranslateXAxis,
			ETransformGizmoPartIdentifier::TranslateScreenSpace,
			ETransformGizmoPartIdentifier::TranslateYAxis))
	},
	{
		ETransformGizmoPartIdentifier::TranslateZAxis,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::TranslateXAxis,
			ETransformGizmoPartIdentifier::TranslateScreenSpace,
			ETransformGizmoPartIdentifier::TranslateZAxis))
	},
	{
		ETransformGizmoPartIdentifier::TranslateXYPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateZAxis,
			ETransformGizmoPartIdentifier::TranslateScreenSpace,
			ETransformGizmoPartIdentifier::TranslateYZPlanar,
			ETransformGizmoPartIdentifier::TranslateXZPlanar})
	},
	{
		ETransformGizmoPartIdentifier::TranslateYZPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateXAxis,
			ETransformGizmoPartIdentifier::TranslateScreenSpace,
			ETransformGizmoPartIdentifier::TranslateXYPlanar,
			ETransformGizmoPartIdentifier::TranslateXZPlanar})
	},
	{
		ETransformGizmoPartIdentifier::TranslateXZPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateYAxis,
			ETransformGizmoPartIdentifier::TranslateScreenSpace,
			ETransformGizmoPartIdentifier::TranslateXYPlanar,
			ETransformGizmoPartIdentifier::TranslateYZPlanar})
	},
	{
		ETransformGizmoPartIdentifier::TranslateScreenSpace,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::TranslateXAxis,
			ETransformGizmoPartIdentifier::TranslateXZPlanar))
	},
	{
		ETransformGizmoPartIdentifier::RotateXAxis,
		FPartSet({
			ETransformGizmoPartIdentifier::RotateScreenSpace
		})
	},
	{
		ETransformGizmoPartIdentifier::RotateYAxis,
		FPartSet({
			ETransformGizmoPartIdentifier::RotateScreenSpace
		})
	},
	{
		ETransformGizmoPartIdentifier::RotateZAxis,
		FPartSet({
			ETransformGizmoPartIdentifier::RotateScreenSpace
		})
	},
	{
		ETransformGizmoPartIdentifier::RotateXGimbal,
		FPartSet({
			ETransformGizmoPartIdentifier::RotateScreenSpace
		})
	},
	{
		ETransformGizmoPartIdentifier::RotateYGimbal,
		FPartSet({
			ETransformGizmoPartIdentifier::RotateScreenSpace
		})
	},
	{
		ETransformGizmoPartIdentifier::RotateZGimbal,
		FPartSet({
			ETransformGizmoPartIdentifier::RotateScreenSpace
		})
	},
	{
		ETransformGizmoPartIdentifier::RotateArcball,
		FPartSet({
			ETransformGizmoPartIdentifier::RotateScreenSpace
		})
	},
	{
		ETransformGizmoPartIdentifier::RotateScreenSpace,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::RotateXAxis,
			ETransformGizmoPartIdentifier::RotateZGimbal,
			ETransformGizmoPartIdentifier::RotateScreenSpace
		))
	},
	{
		ETransformGizmoPartIdentifier::ScaleXAxis,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::ScaleXAxis,
			ETransformGizmoPartIdentifier::ScaleXZPlanar,
			ETransformGizmoPartIdentifier::ScaleXAxis))
	},
	{
		ETransformGizmoPartIdentifier::ScaleYAxis,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::ScaleXAxis,
			ETransformGizmoPartIdentifier::ScaleXZPlanar,
			ETransformGizmoPartIdentifier::ScaleYAxis))
	},
	{
		ETransformGizmoPartIdentifier::ScaleZAxis,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::ScaleXAxis,
			ETransformGizmoPartIdentifier::ScaleXZPlanar,
			ETransformGizmoPartIdentifier::ScaleZAxis))
	},
	{
		ETransformGizmoPartIdentifier::ScaleXYPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::ScaleZAxis,
			ETransformGizmoPartIdentifier::ScaleYZPlanar,
			ETransformGizmoPartIdentifier::ScaleXZPlanar})
	},
	{
		ETransformGizmoPartIdentifier::ScaleYZPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::ScaleXAxis,
			ETransformGizmoPartIdentifier::ScaleXYPlanar,
			ETransformGizmoPartIdentifier::ScaleXZPlanar})
	},
	{
		ETransformGizmoPartIdentifier::ScaleXZPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::ScaleYAxis,
			ETransformGizmoPartIdentifier::ScaleXYPlanar,
			ETransformGizmoPartIdentifier::ScaleYZPlanar})
	},
	{
		ETransformGizmoPartIdentifier::ScaleUniform,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::ScaleXYPlanar,
			ETransformGizmoPartIdentifier::ScaleXZPlanar))
	},
};

TMap<ETransformGizmoPartIdentifier, UEditorTRSGizmo::FGizmoElementInteraction::FPartSet> UEditorTRSGizmo::FGizmoElementInteraction::InteractGroupParts =
{
	{
		ETransformGizmoPartIdentifier::TranslateXAxis,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll})
		},
	{
		ETransformGizmoPartIdentifier::TranslateYAxis,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll})
	},
	{
		ETransformGizmoPartIdentifier::TranslateZAxis,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll})
		},{
		ETransformGizmoPartIdentifier::TranslateXYPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll,
			ETransformGizmoPartIdentifier::TranslateXAxis,
			ETransformGizmoPartIdentifier::TranslateYAxis})
	},
	{
		ETransformGizmoPartIdentifier::TranslateXYPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll,
			ETransformGizmoPartIdentifier::TranslateXAxis,
			ETransformGizmoPartIdentifier::TranslateYAxis})
	},
	{
		ETransformGizmoPartIdentifier::TranslateYZPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll,
			ETransformGizmoPartIdentifier::TranslateYAxis,
			ETransformGizmoPartIdentifier::TranslateZAxis})
	},
	{
		ETransformGizmoPartIdentifier::TranslateXZPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll,
			ETransformGizmoPartIdentifier::TranslateXAxis,
			ETransformGizmoPartIdentifier::TranslateZAxis})
	},
	{
		ETransformGizmoPartIdentifier::TranslateScreenSpace,
		FPartSet({
			ETransformGizmoPartIdentifier::TranslateAll,
			ETransformGizmoPartIdentifier::TranslateXAxis,
			ETransformGizmoPartIdentifier::TranslateYAxis,
			ETransformGizmoPartIdentifier::TranslateZAxis,
			ETransformGizmoPartIdentifier::TranslateScreenSpace})
	},
	{
		ETransformGizmoPartIdentifier::ScaleXYPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::ScaleXAxis,
			ETransformGizmoPartIdentifier::ScaleYAxis,
			ETransformGizmoPartIdentifier::ScaleUniform})
	},
	{
		ETransformGizmoPartIdentifier::ScaleYZPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::ScaleYAxis,
			ETransformGizmoPartIdentifier::ScaleZAxis,
			ETransformGizmoPartIdentifier::ScaleUniform})
	},
	{
		ETransformGizmoPartIdentifier::ScaleXZPlanar,
		FPartSet({
			ETransformGizmoPartIdentifier::ScaleXAxis,
			ETransformGizmoPartIdentifier::ScaleZAxis,
			ETransformGizmoPartIdentifier::ScaleUniform})
	},
	{
		ETransformGizmoPartIdentifier::ScaleUniform,
		FPartSet(FPartSet::FPartRange(
			ETransformGizmoPartIdentifier::ScaleXAxis,
			ETransformGizmoPartIdentifier::ScaleXZPlanar))
	},
};
#pragma endregion FGizmoElementInteraction

void UEditorTRSGizmo::Setup()
{
	if (IsValid(GizmoElementRoot))
	{
		return;
	}

	CacheSnappingStates();

	UInteractiveGizmo::Setup();

	using namespace UE::Editor::InteractiveToolsFramework;

	Profiler = MakeShared<FTransformGizmoProfiler>();

	SAssignNew(DirectionalCursorWidget, SGizmoCursor)
		.Image(UE::Editor::InteractiveToolsFramework::Private::FEditorInteractiveToolsFrameworkStyle::Get().GetBrush("Cursor.ArrowLeftRight"))
		.ColorAndOpacity(FLinearColor::Black)
		.Rotation_UObject(this, &UEditorTRSGizmo::GetCursorRotation);

	SetupBehaviors();
	SetupIndirectBehaviors();
	SetupMaterials();
	SetupOnHoverFunctions();
	SetupOnClickFunctions();

	DebugProvider = NewObject<UGizmoDebugProvider>();
	DebugProvider->Setup();

	// @todo: Gizmo element construction will be moved to the UEditorTransformGizmoBuilder to decouple
	// the rendered elements from the transform gizmo.
	GizmoElementRoot = NewObject<UGizmoElementGroup>();
	GizmoElementRoot->SetConstantScale(true);
	GizmoElementRoot->SetHoverMaterial(HoverAxisMaterial);
	GizmoElementRoot->SetSelectMaterial(SelectAxisMaterial);
	GizmoElementRoot->SetInteractMaterial(InteractAxisMaterial);
	GizmoElementRoot->SetHoverLineColor(Style.HoverColor);
	GizmoElementRoot->SetSelectVertexColor(Style.SelectColor);
	GizmoElementRoot->SetInteractLineColor(Style.InteractColor);

	// the main gimbal rotation element that manages the three gimbal rotation axis as a group
	RotateGimbalElement = NewObject<UGizmoElementGimbal>();
	RotateGimbalElement->SetHoverMaterial(HoverAxisMaterial);
	RotateGimbalElement->SetSelectMaterial(SelectAxisMaterial);
	RotateGimbalElement->SetInteractMaterial(InteractAxisMaterial);
	RotateGimbalElement->SetHoverLineColor(Style.HoverColor);
	RotateGimbalElement->SetInteractLineColor(Style.InteractColor);
	GizmoElementRoot->Add(RotateGimbalElement);

	UpdateElements();

	bInInteraction = false;

	SetModeLastHitPart(EGizmoTransformMode::None, ETransformGizmoPartIdentifier::Default);
	SetModeLastHitPart(EGizmoTransformMode::Translate, ETransformGizmoPartIdentifier::TranslateScreenSpace);
	SetModeLastHitPart(EGizmoTransformMode::Rotate, ETransformGizmoPartIdentifier::RotateArcball);
	SetModeLastHitPart(EGizmoTransformMode::Scale, ETransformGizmoPartIdentifier::ScaleUniform);
}

void UEditorTRSGizmo::SetupBehaviors()
{
	// Add default mouse hover behavior
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	HoverBehavior->HoverModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState)
	{
		// Filter out Alt unless we need it for duplication
		if (FInputDeviceState::IsAltKeyDown(InputDeviceState))
		{
			return AllowsDragDuplicate(EDragDuplicateContext::Capture);
		}

		return true;
	};
	AddInputBehavior(HoverBehavior);

	// Add default mouse input behavior
	USingleClickAndDragBehavior* MouseBehavior = NewObject<USingleClickAndDragBehavior>();
	MouseBehavior->bBeginDragIfClickTargetNotHit = false;
	MouseBehavior->Initialize(this);
	MouseBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	MouseBehavior->SetUsesUnboundedCursor(
		TAttribute<bool>::CreateLambda(
			[this]()
			{
				return CameraFollowsMovement() || IsRotationPrecisionMode();
			}
		)
	);

	MouseBehavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState)
	{
		// Filter out Alt unless we need it for duplication
		if (FInputDeviceState::IsAltKeyDown(InputDeviceState))
		{
			return AllowsDragDuplicate(EDragDuplicateContext::Capture);
		}

		return true;
	};
	AddInputBehavior(MouseBehavior);
	ClickAndDragBehaviorWeak = MouseBehavior;

	// Listen to selection changes, so we can interrupt interaction if nothing is selected
	if (const UInteractiveGizmoManager* const GizmoManager = GetGizmoManager())
	{
		if (const UContextObjectStore* const ContextObjectStore = GizmoManager->GetContextObjectStore())
		{
			if (IAssetEditorContextInterface* AssetEditorContext = ContextObjectStore->FindContext<IAssetEditorContextInterface>())
			{
				if (UTypedElementSelectionSet* SelectionSet = AssetEditorContext->GetMutableSelectionSet())
				{
					SelectionSet->OnChanged().AddUObject(this, &UEditorTRSGizmo::OnSelectionChanged);
				}
			}
		}
	}
	
	// Add Alt+MMB pivot manipulation behavior
	ULocalSingleClickAndDragBehavior* AltMiddleMouseBehavior = NewObject<ULocalSingleClickAndDragBehavior>();
	AltMiddleMouseBehavior->SetUseMiddleMouseButton();
	AltMiddleMouseBehavior->bBeginDragIfClickTargetNotHit = false;
	AltMiddleMouseBehavior->CanBeginSingleClickAndDragSequenceFunc = [this](const FInputDeviceRay& InDeviceRay)
	{
		// Only the translation controls are supported
		FInputRayHit RayHit;
		switch (GetHitPartInternal(InDeviceRay, RayHit))
		{
		case ETransformGizmoPartIdentifier::TranslateAll:
		case ETransformGizmoPartIdentifier::TranslateXAxis:
		case ETransformGizmoPartIdentifier::TranslateYAxis:
		case ETransformGizmoPartIdentifier::TranslateZAxis:
		case ETransformGizmoPartIdentifier::TranslateXYPlanar:
		case ETransformGizmoPartIdentifier::TranslateXZPlanar:
		case ETransformGizmoPartIdentifier::TranslateYZPlanar:
		case ETransformGizmoPartIdentifier::TranslateScreenSpace:
			return RayHit;
		default:
			return FInputRayHit();
		}
	};
	AltMiddleMouseBehavior->ModifierCheckFunc = [](const FInputDeviceState& InInputDeviceState)
	{
		return InInputDeviceState.bAltKeyDown;
	};
	AltMiddleMouseBehavior->OnClickPressFunc = [this](const FInputDeviceRay& InDragPos)
	{
		bOnlyUpdatePivot = true;
		OnClickPress(InDragPos);
	};
	AltMiddleMouseBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY).MakeHigher());
	AltMiddleMouseBehavior->Initialize(this);
	AddInputBehavior(AltMiddleMouseBehavior);

	// Keys used for nudging. Press Alt + Arrow key to translate/rotate/scale selection
	// Ctrl can be used to switch to the current transform secondary axis
	// Pressing Shift during nudge-translation will have the camera follow the movement
	const TArray<FKey> NudgeKeys = {EKeys::Up, EKeys::Down, EKeys::Right, EKeys::Left};

	TArray<FKey> KeyInputBehaviorKeys;
	KeyInputBehaviorKeys.Append(NudgeKeys);
	KeyInputBehaviorKeys.Add(EKeys::Escape); // Escape key is used to cancel the current update

	UKeyInputBehavior* KeyInputBehavior = NewObject<UKeyInputBehavior>();
	KeyInputBehavior->Initialize(this, KeyInputBehaviorKeys);
	KeyInputBehavior->bRequireAllKeys = false;
	KeyInputBehavior->Modifiers.RegisterModifier(UE::Editor::ViewportInteractions::ShiftKeyMod, FInputDeviceState::IsShiftKeyDown);
	KeyInputBehavior->Modifiers.RegisterModifier(UE::Editor::ViewportInteractions::AltKeyMod, FInputDeviceState::IsAltKeyDown);
	KeyInputBehavior->Modifiers.RegisterModifier(UE::Editor::ViewportInteractions::CtrlKeyMod, FInputDeviceState::IsCtrlKeyDown);
	KeyInputBehavior->bAlwaysUpdateModifiers = true;
	KeyInputBehavior->ModifierCheckFunc = [this, NudgeKeys](const FInputDeviceState& InputDeviceState)
	{
		const FKey& ActiveButton = InputDeviceState.Keyboard.ActiveKey.Button;
		if (bInInteraction)
		{
			return ActiveButton == EKeys::Escape;
		}

		if (FInputDeviceState::IsAltKeyDown(InputDeviceState))
		{
			for (const FKey& Key : NudgeKeys)
			{
				if (Key == ActiveButton)
				{
					return true;
				}
			}
		}

		return false;
	};

	AddInputBehavior(KeyInputBehavior);
}

void UEditorTRSGizmo::SetupIndirectBehaviors()
{
	using namespace UE::Editor::ViewportInteractions;

	// Add middle mouse input behavior for indirect manipulation
	ULocalViewportClickDragBehavior* MiddleClickDragBehavior = NewObject<ULocalViewportClickDragBehavior>();
	MiddleClickDragBehavior->Initialize();
	MiddleClickDragBehavior->SetUsesUnboundedCursor(true);
	MiddleClickDragBehavior->SetBindings({
		FButtonBinding(EKeys::MiddleMouseButton).TriggersStart()
	});

	MiddleClickDragBehavior->CanBeginClickDragFunc = [this](const FInputDeviceRay& PressPos)
	{
		if (OnPreCanInteractDelegate.IsBound())
		{
			FGizmoInteractionDescription Desc({.Ray=PressPos, .bIndirect=true});
			OnPreCanInteractDelegate.Broadcast(Desc);
		}
			
		static const FInputRayHit InvalidRayHit;
		// HACK: This lets this behavior win as a tiebreaker vs viewport interactions with the same binding priorities
		static const FInputRayHit ValidRayHit(UE_LARGE_WORLD_MAX);
		return (CanInteract() && Interaction.bPersistHandleSelection) ? ValidRayHit : InvalidRayHit;
	};
	MiddleClickDragBehavior->OnDragStartFunc = [this](const FInputDeviceRay& InDragStartPos)
	{
		bIndirectManipulation = true;
		// Check if scaling should always be uniform
		if (CurrentMode == EGizmoTransformMode::Scale && bUniformIndirectScale)
		{
			LastHitPart = ETransformGizmoPartIdentifier::ScaleUniform;
		}
		else if (LastHitPart == ETransformGizmoPartIdentifier::Default)
		{
			LastHitPart = GetCurrentModeLastHitPart();
		}
		OnClickPress(InDragStartPos);
		OnDragStart(InDragStartPos);
	};
	MiddleClickDragBehavior->OnDragFunc = [this](const IViewportClickDragBehaviorTarget::FDragArgs& InDrag)
	{
		bIndirectManipulation = true;
		return OnClickDrag(InDrag.Ray);
	};
	MiddleClickDragBehavior->OnDragEndFunc = [this](const FInputDeviceRay& InDragEndPos)
	{
		bIndirectManipulation = false;
		return OnClickRelease(InDragEndPos, true);
	};
	MiddleClickDragBehavior->OnEndCaptureFunc = [this](IViewportClickDragBehaviorTarget::EEndCaptureReason Reason)
	{
		bIndirectManipulation = false;
		if (Reason == IViewportClickDragBehaviorTarget::EEndCaptureReason::Forced)
		{
			OnTerminateSingleClickAndDragSequence();
		}
	};

	AddInputBehavior(MiddleClickDragBehavior);

	// Add left/right mouse input behavior for indirect manipulation
	IndirectClickDragBehavior = NewObject<ULocalViewportClickDragBehavior>();
	IndirectClickDragBehavior->Initialize();
	IndirectClickDragBehavior->SetUsesUnboundedCursor(true);

	IndirectClickDragBehavior->SetBindings({
		FButtonBinding(EKeys::LeftControl),
		FButtonBinding(EKeys::LeftMouseButton).Required(false).TriggersStart(),
		FButtonBinding(EKeys::RightMouseButton).Required(false).TriggersStart(),
		FButtonBinding(EKeys::MiddleMouseButton).Required(false).TriggersStart(),
	});

	IndirectClickDragBehavior->CanBeginClickDragFunc = [this](const FInputDeviceRay& PressPos)
	{
		if (OnPreCanInteractDelegate.IsBound())
		{
			FGizmoInteractionDescription Desc({.Ray=PressPos, .bIndirect=true});
			OnPreCanInteractDelegate.Broadcast(Desc);
		}
		
		static const FInputRayHit InvalidRayHit;
		static const FInputRayHit ValidRayHit(TNumericLimits<double>::Max());

		return CanInteract() ? ValidRayHit : InvalidRayHit;
	};
	IndirectClickDragBehavior->OnDragStartFunc = [this](const FInputDeviceRay& InDragStartPos)
	{
		bIndirectManipulation = true;
		// Check if scaling should always be uniform
		if (CurrentMode == EGizmoTransformMode::Scale && bUniformIndirectScale)
		{
			LastHitPart = ETransformGizmoPartIdentifier::ScaleUniform;
		}
		else if (LastHitPart == ETransformGizmoPartIdentifier::Default)
		{
			LastHitPart = GetCurrentModeLastHitPart();
		}
		OnClickPress(InDragStartPos);
		OnDragStart(InDragStartPos);
	};
	IndirectClickDragBehavior->OnDragFunc = [this](const IViewportClickDragBehaviorTarget::FDragArgs& InDrag)
	{
		bIndirectManipulation = true;
		return OnClickDrag(InDrag.Ray);
	};
	IndirectClickDragBehavior->OnDragEndFunc = [this](const FInputDeviceRay& InDragEndPos)
	{
		bIndirectManipulation = false;
		return OnClickRelease(InDragEndPos, true);
	};
	IndirectClickDragBehavior->OnEndCaptureFunc = [this](IViewportClickDragBehaviorTarget::EEndCaptureReason Reason)
	{
		UE_CLOGF(ModeAxisOverride.IsSet(), LogEditorTRSGizmo, Log, "Dropping Mode axis override");
		ModeAxisOverride.Reset();
	
		bIndirectManipulation = false;
		if (Reason == IViewportClickDragBehaviorTarget::EEndCaptureReason::Forced)
		{
			OnTerminateSingleClickAndDragSequence();
		}
	};

	IndirectClickDragBehavior->OnStateUpdatedFunc = [this](const FInputDeviceState& Input)
	{
		struct FSimpleButtonState
		{
			bool bDown = false;
			bool bPressed = false;
			
			FSimpleButtonState() = default;
			
			FSimpleButtonState(const FDeviceButtonState& InState)
				: bDown(InState.bDown), bPressed(InState.bPressed)
			{
			}
			
			static FSimpleButtonState Or(const FSimpleButtonState& A, const FSimpleButtonState& B)
			{
				FSimpleButtonState Result;
				Result.bDown = A.bDown || B.bDown;
				Result.bPressed	= A.bPressed || B.bPressed;
				return Result;
			}
			
			static FSimpleButtonState And(const FSimpleButtonState& A, const FSimpleButtonState& B)
			{
				FSimpleButtonState Result;
				Result.bDown = A.bDown && B.bDown;
				Result.bPressed = Result.bDown && (A.bPressed || B.bPressed);
				return Result;
			}
		};
		
		struct FSimpleBinding
		{
			FSimpleButtonState Button;
			EAxisList::Type Axis;
		};
		
		const FMouseInputDeviceState& Mouse = Input.Mouse;
		
		// The binding list determines which axis has precedence.
		FSimpleBinding Bindings[3];
		if (bSequentialIndirectAxesButtons)
		{
			Bindings[0] = { Mouse.Left, EAxisList::X };
			Bindings[1] = { Mouse.Middle, EAxisList::Y };
			Bindings[2] = { Mouse.Right, EAxisList::Z };
		}
		else
		{
			// The middle button binding should be first here as it is potentially the LMB+RMB instead.
			// With additive axes off, the combination of LMB & RMB is equivalent to the MMB
			const FSimpleButtonState MiddleButton = bAdditiveIndirectAxes ? Mouse.Middle : FSimpleButtonState::Or(Mouse.Middle, FSimpleButtonState::And(Mouse.Left, Mouse.Right));
			
			Bindings[0] = { MiddleButton, EAxisList::Z };
			Bindings[1] = { Mouse.Left, EAxisList::X };
			Bindings[2] = { Mouse.Right, EAxisList::Y };
		}
		
		EAxisList::Type TargetAxes = EAxisList::None;
		if (bScreenSpaceIndirectOrthographicManipulation && GizmoViewContext && !GizmoViewContext->IsPerspectiveProjection())
		{
			if (Input.Mouse.Right.bDown)
			{
				ModeAxisOverride = FModeAxisOverride {
					.Mode = EGizmoTransformMode::Rotate,
					.AxisToDraw = EAxisList::All,
				};
				
				UpdateMode();
					
				TargetAxes = EAxisList::Screen;
			}
			else if (Input.Mouse.Left.bDown)
			{
				TargetAxes = EAxisList::Screen;
			}
		}
		else if (bAdditiveIndirectAxes)
		{
			for (const FSimpleBinding& Binding : Bindings)
			{
				if (Binding.Button.bDown)
				{
					TargetAxes = static_cast<EAxisList::Type>(TargetAxes | Binding.Axis);
				}
			}
		}
		else
		{
			auto GetAxis = [&Bindings](bool bUsePressed)
			{
				for (const FSimpleBinding& Binding : Bindings)
				{
					if ((bUsePressed && Binding.Button.bPressed) || (!bUsePressed && Binding.Button.bDown))
					{
						return Binding.Axis;
					}
				}
				return EAxisList::None;
			};
			
			// Attempt to pull from bPressed so that the most recent mouse button gets priority
			TargetAxes = GetAxis(true);
			
			if (TargetAxes == EAxisList::None)
			{
				// Fall back to just bDown
				TargetAxes = GetAxis(false);
			}
		}

		// Disable indirect if the current axis is none
		if (TargetAxes == EAxisList::None)
		{
			if (bInInteraction)
			{
				// Otherwise, the EndDrag handler will call the click release
				if (!IndirectClickDragBehavior->IsDragging())
				{
					UE_LOGF(LogEditorTRSGizmo, Verbose, "Calling OnClickRelease() from non-dragging indirect interaction");
					OnClickRelease(Input, true);
				}
			}
			else
			{
				// Reset interaction visuals
				UpdateInteractingState(false, LastHitPart);
			}
			bIndirectManipulation = false;
			return;
		}

		// Switch to uniform scaling based on preferences (bUniformIndirectScale) or LMB + RMB being down
		const bool bUniformScaling = CurrentMode == EGizmoTransformMode::Scale && (bUniformIndirectScale || (!bAdditiveIndirectAxes && Input.Mouse.Left.bDown && Input.Mouse.Right.bDown));

		// If "forcing" uniform scaling, we swap the actual axis with ScaleUniform
		const ETransformGizmoPartIdentifier HitPart = bUniformScaling ? ETransformGizmoPartIdentifier::ScaleUniform : GetPartForAxisList(TargetAxes);

		const ETransformGizmoPartIdentifier ModeLastHitPart = GetCurrentModeLastHitPart();
		if (HitPart != ModeLastHitPart)
		{
			UE_LOGF(LogEditorTRSGizmo, Verbose, "Changing indirect target to %i", EnumToUnderlyingType(HitPart));
			// When changing target hit parts in the middle of an interaction:
			// 1. Stop the current edit action
			// 2. Prepare to start a new edit action
			// Resetting drag accomplishes both: the current edit action is stopped and a new one
			// will be started when a new drag begins.
			if (bInInteraction)
			{
				if (!IndirectClickDragBehavior->IsDragging())
				{
					// If the behavior is in a drag, the ResetDrag and OnDragEnd will handle this
					// Otherwise it would not be called
					OnClickRelease(Input, true);
				}
				
				// Resetting the drag means that the mouse movement threshold is reset
				// This helps prevent small errors when a user release two mouse buttons at once.
				IndirectClickDragBehavior->ResetDrag(Input);
			}
			
			LastHitPart = HitPart;
			SetModeLastHitPart(CurrentMode, HitPart);
			
			// Immediately change selection & interaction visualization
			// This prevents a UI flicker that would be caused while waiting 
			// in the interstitial period between resetting the drag and starting a new drag.  
			UpdateInteractingState(false, ModeLastHitPart); 
			UpdateInteractingState(true, LastHitPart);
		}
		
		bIndirectManipulation = true;
	};
	AddInputBehavior(IndirectClickDragBehavior);
}

void UEditorTRSGizmo::SetupMaterials()
{
	UpdateMaterials();
}

void UEditorTRSGizmo::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (CanRender(RenderAPI))
	{
		UGizmoElementBase::FRenderTraversalState RenderState;
		RenderState.Initialize(UE::GizmoRenderingUtil::FSceneViewWrapper(*RenderAPI->GetSceneView()), GetGizmoTransform(), RenderAPI->GetCameraState().DPIScale);

		bool bRenderGizmos = IsVisible();
		if (GizmoLocals::DoDebugDraw())
		{
			bRenderGizmos &= GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bDrawGizmos;
		}

		if (bRenderGizmos)
		{
			Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::RenderName);
			GizmoElementRoot->Render(RenderAPI, RenderState);
		}

		DrawDebug(nullptr, RenderAPI, RenderState.PixelToWorldScale);
	}
}

void UEditorTRSGizmo::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (CanRender(RenderAPI))
	{
		UGizmoElementBase::FRenderTraversalState RenderState;
		RenderState.Initialize(UE::GizmoRenderingUtil::FSceneViewWrapper(*RenderAPI->GetSceneView()), GetGizmoTransform());

		bool bRenderGizmos = true;
		if (GizmoLocals::DoDebugDraw())
		{
			bRenderGizmos = GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bDrawGizmos;
		}

		if (bRenderGizmos)
		{
			Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DrawHUDName);
			GizmoElementRoot->DrawHUD(Canvas, RenderAPI, RenderState);
		}

		DrawDebug(Canvas, RenderAPI, RenderState.PixelToWorldScale);
	}
}

TSharedPtr<SWidget> UEditorTRSGizmo::GetCursorWidget(const ETransformGizmoPartIdentifier InHitPartId) const
{
	// @note: all custom cursor widgets currently only occur when interacting
	if (!bInInteraction)
	{
		return nullptr;
	}

	switch (CurrentMode)
	{
	case EGizmoTransformMode::Translate:
		break;

	case EGizmoTransformMode::Rotate:
	{
		if (InHitPartId != ETransformGizmoPartIdentifier::RotateArcball
			&& !bIndirectManipulation
			&& (RotateMode == EAxisRotateMode::Arc || RotateMode == EAxisRotateMode::ScreenArc))
		{
			return DirectionalCursorWidget;
		}
	}
	break;

	case EGizmoTransformMode::Scale:
		break;

	case EGizmoTransformMode::None:
	default:
		break;
	}

	return nullptr;
}

float UEditorTRSGizmo::GetCursorRotation() const
{
	return CursorRotation;
}

void UEditorTRSGizmo::UpdateCursorRotation(const FInputDeviceRay& InPressPos)
{
	auto GetCursorRotation3D = [&](const FVector& InPointOnPlane, const double InAngleOffsetRad = 0.0) -> float // ie. add 90deg to the result
	{
		const FVector OriginToCursor = (InPointOnPlane - InteractionPlanarOrigin).GetSafeNormal();
		const FVector CursorDirection = OriginToCursor ^ InteractionPlanarNormal;

		const FVector CursorDirectionScreenSpace = GizmoViewContext->ViewMatrices.GetViewMatrix().TransformVector(CursorDirection);

		const double SignedAngle = FMath::Atan2(-CursorDirectionScreenSpace.Y, CursorDirectionScreenSpace.X) + InAngleOffsetRad;

		// Update Debug
		{
			DebugData.CursorDirectionWS = CursorDirection;
			DebugData.CursorDirectionSS = FVector2D(CursorDirectionScreenSpace.X, -CursorDirectionScreenSpace.Y);
		}

		return static_cast<float>(SignedAngle);
	};

	auto GetCursorRotation2D = [&](const FVector2D& InCursorPosition, const double InAngleOffsetRad = 0.0) -> float // ie. add 90deg to the result
	{
		const FVector2D OriginToCursor = (InCursorPosition - InteractionScreenObjectPos2D).GetSafeNormal();

		const double SignedAngle = FMath::Atan2(OriginToCursor.Y, OriginToCursor.X) + InAngleOffsetRad;

		// Update Debug
		{
			DebugData.CursorDirectionWS = FVector::ZeroVector;
			DebugData.CursorDirectionSS = FVector2D(OriginToCursor.X, -OriginToCursor.Y);
		}

		return static_cast<float>(SignedAngle);
	};

	using namespace UE::Editor::GizmoMath;

	switch (CurrentMode)
	{
	case EGizmoTransformMode::Translate:
		break;

	case EGizmoTransformMode::Rotate:
		{
			FVector HitPoint;
			if (ComputeProjectedPointOnPlaneFromScreen(GizmoViewContext, InPressPos.ScreenPosition, InteractionPlane, HitPoint))
			{
				CursorRotation = GetCursorRotation3D(HitPoint);
			}
			else
			{
				// Fall back to screen-space cursor rotation when the plane intersection fails
				// (e.g. camera nearly edge-on to the rotation axis, or cursor outside viewport bounds)
				CursorRotation = GetCursorRotation2D(InPressPos.ScreenPosition);
			}
		}
		break;

	case EGizmoTransformMode::Scale:
		{
			constexpr double CursorAngleOffset = UE_DOUBLE_HALF_PI; // 90 degrees in radians
			if constexpr (UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::bUseScreenspaceScaling)
			{
				CursorRotation = GetCursorRotation2D(InPressPos.ScreenPosition);
			}
			else
			{
				FVector HitPoint;
				if (!ComputeProjectedPointOnPlaneFromScreen(GizmoViewContext, InPressPos.ScreenPosition, InteractionPlane, HitPoint))
				{
					return;
				}

				CursorRotation = GetCursorRotation3D(HitPoint, CursorAngleOffset);
			}
		}
		break;

	case EGizmoTransformMode::None:
	default:
		break;
	}
}

void UEditorTRSGizmo::UpdateCursor(const EGizmoElementInteractionState InState, const ETransformGizmoPartIdentifier InHitPartId, const bool bIsExitingState)
{
	// Update Cursor
	if (UToolsContextCursorAPI* CursorAPI = GetGizmoManager()->GetContextObjectStore()->FindContext<UToolsContextCursorAPI>())
	{
		if (bIsExitingState
			&& (InState == EGizmoElementInteractionState::Hovering && !bInInteraction))
		{
			// Always clear when exiting hover state (and not interacting)
			CursorAPI->ClearCursorOverride();
		}
		else
		{
			if (CameraFollowsMovement() || IsRotationPrecisionMode())
			{
				CursorAPI->SetCursorOverride(EMouseCursor::None);
			}
			else if (const TSharedPtr<SWidget> CursorWidget = GetCursorWidget(InHitPartId);
				CursorWidget.IsValid())
			{
				if (CurrentMode == EGizmoTransformMode::Rotate)
				{
					CursorAPI->SetUseSoftwareCursor(true);
				}

				CursorAPI->SetCursorOverrideWidget(CursorWidget.ToSharedRef());
			}
			else if (!CanInteractWithPart(InHitPartId))
			{
				// Clear when the part is not interactable, and a custom cursor widget is not explicitly set
				CursorAPI->ClearCursorOverride();
			}
			else
			{
				constexpr EMouseCursor::Type DefaultHoverCursor = EMouseCursor::CardinalCross;
				constexpr EMouseCursor::Type DefaultInteractionCursor = EMouseCursor::GrabHandClosed;

				EMouseCursor::Type CursorType = bInInteraction ? DefaultInteractionCursor : DefaultHoverCursor;

				switch (CurrentMode)
				{
				case EGizmoTransformMode::Translate:
					break;

				case EGizmoTransformMode::Rotate:
					break;

				case EGizmoTransformMode::Scale:
					break;

				case EGizmoTransformMode::None:
				default:
					break;
				}

				CursorAPI->SetCursorOverride(CursorType);
			}
		}
	}
}

void UEditorTRSGizmo::DrawDebug(FCanvas* InCanvas, IToolsContextRenderAPI* RenderAPI, const double InPixelToWorldScale)
{
	if (!GizmoLocals::DoDebugDraw()
		|| !EnumHasAnyFlags(RenderAPI->GetViewInteractionState(), EViewInteractionState::Focused))
	{
		return;
	}

	Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DrawDebugName);

	if (DebugProvider && GizmoElementRoot && RenderAPI->GetSceneView())
	{
		const FGizmoDebugSettings& DebugSettings = GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug;

		UGizmoElementBase::FRenderTraversalState RenderState;
		RenderState.Initialize(UE::GizmoRenderingUtil::FSceneViewWrapper(*RenderAPI->GetSceneView()), GetGizmoTransform());

		if (InCanvas)
		{
			DebugProvider->DrawCanvas(FGizmoDebugObjectVariant(TInPlaceType<const UInteractiveGizmo*>(), this), InCanvas, RenderAPI, RenderState, DebugSettings);

			// Timing
			if (DebugSettings.bDrawTiming)
			{
				using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;

				FIntRect ViewRectI = InCanvas->GetViewRect();
				FBox2D ViewRect = FBox2D(FVector2D(ViewRectI.Min), FVector2D(ViewRectI.Max));
				FVector2D ViewSize = ViewRect.Max - ViewRect.Min;

				constexpr float LabelPadding = 8.0f;
				constexpr float LabelFontHeight = 20.0f;

				auto DrawText = [&ViewSize, InCanvas](const FString& InString, const int32 RowOffset = 0, const FLinearColor& InColor = FLinearColor::Yellow)
				{
					FCanvasTextItem TextItem(
						FVector2D(LabelPadding * 2.0f, ViewSize.Y - ((LabelPadding + LabelFontHeight) * static_cast<float>(RowOffset + 1))),
						FText::FromString(InString),
						GEngine->GetMediumFont(),
						InColor);

					TextItem.EnableShadow(FLinearColor::Black, FVector2D(1.0f, 1.0f));
					TextItem.Draw(InCanvas);
				};

				auto DrawTimer = [Profiler = Profiler, DrawText](const FName InTimerName, const int32 InRowOffset = 0, const FLinearColor& InColor = FLinearColor::Yellow)
				{
					const FString ExecutionTime = Profiler->GetAverageExecutionTimeStr(InTimerName);

					DrawText(FString::Printf(TEXT("%s Avg. Time: %s"),
						*InTimerName.ToString(),
						*ExecutionTime), InRowOffset, InColor);
				};

				constexpr float TextBackgroundBoxWidth = 300.0f;

				auto DrawSeparator = [ViewSize, InCanvas](const int32 InRowOffset = 0)
				{
					FCanvasLineItem LineItem(
						FVector2D(LabelPadding, ViewSize.Y - ((LabelPadding + LabelFontHeight) * static_cast<float>(InRowOffset + 1)) + LabelFontHeight * 0.5f),
						FVector2D(LabelPadding + TextBackgroundBoxWidth, ViewSize.Y - ((LabelPadding + LabelFontHeight) * static_cast<float>(InRowOffset + 1)) + LabelFontHeight * 0.5f));
					LineItem.SetColor(FLinearColor::White.CopyWithNewOpacity(0.5f));
					LineItem.LineThickness = 1.0f;
					LineItem.Draw(InCanvas);
				};

				int32 RowCounter = 1;
				DrawTimer(PendingFunctionExecutionName, RowCounter++);
				DrawTimer(GetActiveTransformName, RowCounter++);
				DrawSeparator(RowCounter++);
				DrawTimer(RenderName, RowCounter++);
				DrawTimer(DrawHUDName, RowCounter++);
				DrawTimer(DrawDebugName, RowCounter++);
				DrawSeparator(RowCounter++);

				const FLinearColor ModeTimerColor = FLinearColor::White;
				const FLinearColor ActiveTimerColor = FLinearColor::Green;

				using FPartSet = FGizmoElementInteraction::FPartSet;
				using FPartRange = FPartSet::FPartRange;
				auto GetModeTimerColor = [this, ActiveTimerColor, ModeTimerColor](FPartSet&& InParts)
				{
					bool bIsAnyPartActive = false;
					InParts.ForEachPart([this, &bIsAnyPartActive](ETransformGizmoPartIdentifier PartId)
					{
						if (PartId == GetCurrentModeLastHitPart())
						{
							bIsAnyPartActive = true;
						}
					});

					return bIsAnyPartActive ? ActiveTimerColor : ModeTimerColor;
				};

				switch (CurrentMode)
				{
				case EGizmoTransformMode::Translate:
					DrawTimer(
						DragTranslateAxisName,
						RowCounter++,
						GetModeTimerColor(FPartSet(FPartRange{
						ETransformGizmoPartIdentifier::TranslateXAxis,
						ETransformGizmoPartIdentifier::TranslateZAxis})));

					DrawTimer(
						DragTranslatePlanarName,
						RowCounter++,
						GetModeTimerColor(FPartSet(FPartRange{
						ETransformGizmoPartIdentifier::TranslateXYPlanar,
						ETransformGizmoPartIdentifier::TranslateXZPlanar})));

					DrawTimer(
						DragTranslateUniformName,
						RowCounter++,
						GetModeTimerColor(FPartSet({ ETransformGizmoPartIdentifier::TranslateScreenSpace })));

					break;

				case EGizmoTransformMode::Rotate:
					DrawTimer(
						DragRotateAxisName,
						RowCounter++,
						GetModeTimerColor(FPartSet(FPartRange{
							ETransformGizmoPartIdentifier::RotateXAxis,
							ETransformGizmoPartIdentifier::RotateZAxis})));

					DrawTimer(
						DragRotateArcBallName,
						RowCounter++,
						GetModeTimerColor(FPartSet({ ETransformGizmoPartIdentifier::RotateArcball })));

					DrawTimer(
						DragRotateScreenName,
						RowCounter++,
						GetModeTimerColor(FPartSet({ ETransformGizmoPartIdentifier::RotateScreenSpace })));

					break;

				case EGizmoTransformMode::Scale:
					DrawTimer(
						DragScaleAxisName,
						RowCounter++,
						GetModeTimerColor(FPartSet(FPartRange{
							ETransformGizmoPartIdentifier::ScaleXAxis,
							ETransformGizmoPartIdentifier::ScaleZAxis})));

					DrawTimer(
						DragScalePlanarName,
						RowCounter++,
						GetModeTimerColor(FPartSet(FPartRange({
							ETransformGizmoPartIdentifier::ScaleXYPlanar,
							ETransformGizmoPartIdentifier::ScaleXZPlanar}))));

					DrawTimer(
						DragScaleUniformName,
						RowCounter++,
						GetModeTimerColor(FPartSet({ ETransformGizmoPartIdentifier::ScaleUniform })));

					break;

				case EGizmoTransformMode::None:
				case EGizmoTransformMode::Max:
				default:
					break;
				}
			}
		}
		else
		{
			DebugProvider->Draw(FGizmoDebugObjectVariant(TInPlaceType<const UInteractiveGizmo*>(), this), RenderAPI, RenderState, DebugSettings);
		}
	}
}

FInputRayHit UEditorTRSGizmo::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	const ETransformGizmoPartIdentifier PreviousHitPart = LastHitPart;
	const FInputRayHit RayHit = UpdateHoveredPart(PressPos);

	if (RayHit.bHit
		&& PreviousHitPart != LastHitPart)
	{
		UpdateCursor(EGizmoElementInteractionState::Hovering, LastHitPart, false);
	}

	return RayHit;
}

void UEditorTRSGizmo::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	const FInputRayHit RayHit = UpdateHoveredPart(DevicePos);

	if (RayHit.bHit)
	{
		UpdateCursor(EGizmoElementInteractionState::Hovering, LastHitPart, false);
	}
}

bool UEditorTRSGizmo::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	const FInputRayHit RayHit = UpdateHoveredPart(DevicePos);

	if (RayHit.bHit)
	{
		UpdateCursorRotation(DevicePos);
	}

	return RayHit.bHit;
}

void UEditorTRSGizmo::OnEndHover()
{
	if (HitTarget)
	{
		if (LastHitPart != ETransformGizmoPartIdentifier::Default)
		{
			UpdateHoverState(false, LastHitPart);
		}

		UpdateCursor(EGizmoElementInteractionState::Hovering, LastHitPart, true);
	}
}

FInputRayHit UEditorTRSGizmo::UpdateHoveredPart(const FInputDeviceRay& PressPos)
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;

	if (!HitTarget)
	{
		return FInputRayHit();
	}

	FInputRayHit RayHit;
	const ETransformGizmoPartIdentifier HitPart = GetHitPartInternal(PressPos, RayHit);

	if (HitPart != LastHitPart)
	{
		if (LastHitPart != ETransformGizmoPartIdentifier::Default)
		{
			UpdateHoverState(false, LastHitPart);
		}

		if (HitPart != ETransformGizmoPartIdentifier::Default)
		{
			UpdateHoverState(true, HitPart);
		}

		LastHitPart = HitPart;
	}

	return RayHit;
}

void UEditorTRSGizmo::UpdateMode()
{
	auto GetTransformMode = [&]()
	{
		if (const FModeAxisOverride* Override = ModeAxisOverride.GetPtrOrNull())
		{
			return Override->Mode;
		}
	
		if (TransformGizmoSource)
		{
			return TransformGizmoSource->GetGizmoMode();
		}

		const EToolContextTransformGizmoMode ActiveGizmoMode = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentTransformGizmoMode();
		switch (ActiveGizmoMode)
		{
			case EToolContextTransformGizmoMode::Translation: return EGizmoTransformMode::Translate;
			case EToolContextTransformGizmoMode::Rotation: return EGizmoTransformMode::Rotate;
			case EToolContextTransformGizmoMode::Scale: return EGizmoTransformMode::Scale;
		}
		return EGizmoTransformMode::None;
	};

	auto GetAxisToDraw = [&]()
	{
		if (const FModeAxisOverride* Override = ModeAxisOverride.GetPtrOrNull())
		{
			return Override->AxisToDraw;
		}
	
		if (TransformGizmoSource)
		{
			return TransformGizmoSource->GetGizmoAxisToDraw(TransformGizmoSource->GetGizmoMode());
		}
		return EAxisList::Type::All;
	};

	const EGizmoTransformMode NewMode = GetTransformMode();
	const EAxisList::Type NewAxisToDraw = GetAxisToDraw();

	if (NewMode != CurrentMode)
	{
		EnableMode(CurrentMode, EAxisList::None);
		EnableMode(NewMode, NewAxisToDraw);

		CurrentMode = NewMode;
		CurrentAxisToDraw = NewAxisToDraw;

		bTRSSnapDirty = true;

		// Reset various profilers so that we can see the cost of the new mode
		Profiler->Reset(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::RenderName);
		Profiler->Reset(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DrawHUDName);
		Profiler->Reset(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DrawDebugName);
	}
	else if (NewAxisToDraw != CurrentAxisToDraw)
	{
		EnableMode(CurrentMode, NewAxisToDraw);
		CurrentAxisToDraw = NewAxisToDraw;
	}
	else if (CurrentMode == EGizmoTransformMode::Rotate && bGimbalRotationMode != GetRotationContext().bUseExplicitRotator)
	{
		UpdateRotationMode();
	}
}

void UEditorTRSGizmo::UpdateRotationMode()
{
	EnableMode(EGizmoTransformMode::Rotate, CurrentAxisToDraw);

	if (GetCurrentModeLastHitPart() != ETransformGizmoPartIdentifier::Default)
	{
		const ETransformGizmoPartIdentifier PreviousHitPart = GetCurrentModeLastHitPart();

		UpdateInteractingState(false, PreviousHitPart, true);

		ETransformGizmoPartIdentifier NewHitPart = ETransformGizmoPartIdentifier::RotateArcball;

		constexpr uint8 RotateId = static_cast<uint8>(EGizmoTransformMode::Rotate);
		if (bGimbalRotationMode)
		{
			switch (LastHitPartPerMode[RotateId])
			{
			case ETransformGizmoPartIdentifier::RotateXAxis:
				NewHitPart = ETransformGizmoPartIdentifier::RotateXGimbal;
				break;
			case ETransformGizmoPartIdentifier::RotateYAxis:
				NewHitPart = ETransformGizmoPartIdentifier::RotateYGimbal;
				break;
			case ETransformGizmoPartIdentifier::RotateZAxis:
				NewHitPart = ETransformGizmoPartIdentifier::RotateZGimbal;
				break;
			case ETransformGizmoPartIdentifier::RotateScreenSpace:
				NewHitPart = ETransformGizmoPartIdentifier::RotateArcball;
				break;
			default:
				break;
			}
		}
		else
		{
			switch (LastHitPartPerMode[RotateId])
			{
			case ETransformGizmoPartIdentifier::RotateXGimbal:
				NewHitPart = ETransformGizmoPartIdentifier::RotateXAxis;
				break;
			case ETransformGizmoPartIdentifier::RotateYGimbal:
				NewHitPart = ETransformGizmoPartIdentifier::RotateYAxis;
				break;
			case ETransformGizmoPartIdentifier::RotateZGimbal:
				NewHitPart = ETransformGizmoPartIdentifier::RotateZAxis;
				break;
			default:
				break;
			}
		}

		SetModeLastHitPart(CurrentMode, NewHitPart);

		UpdateSelectedState(true, NewHitPart);

		constexpr bool bInInteracting = true, bIdOnly = true;
		UpdateInteractingState(bInInteracting, NewHitPart, bIdOnly);
	}
}

void UEditorTRSGizmo::EnableMode(EGizmoTransformMode InMode, EAxisList::Type InAxisListToDraw)
{
	if (InMode == EGizmoTransformMode::Translate)
	{
		EnableTranslate(InAxisListToDraw);
	}
	else if (InMode == EGizmoTransformMode::Rotate)
	{
		EnableRotate(InAxisListToDraw);
	}
	else if (InMode == EGizmoTransformMode::Scale)
	{
		EnableScale(InAxisListToDraw);
	}
}

void UEditorTRSGizmo::EnableTranslate(EAxisList::Type InAxisListToDraw)
{
	check(GizmoElementRoot);

	const bool bEnableX = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::X);
	const bool bEnableY = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Y);
	const bool bEnableZ = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Z);
	const bool bEnableAny = bEnableX || bEnableY || bEnableZ;

	if (!TranslateGroupElement)
	{
		TranslateGroupElement = MakeTranslateGroup(EAxisList::XYZ);
		GizmoElementRoot->Add(TranslateGroupElement);
	}

	if (TranslateGroupElement)
	{
		TranslateGroupElement->SetAxisEnabled(InAxisListToDraw);
		TranslateGroupElement->SetPlanarEnabled(UE::Editor::GizmoMath::GetAxisListFromBools(bEnableX, bEnableY, bEnableZ));
		TranslateGroupElement->SetUniformEnabled(bEnableAny);
	}
}

void UEditorTRSGizmo::EnableRotate(EAxisList::Type InAxisListToDraw)
{
	const bool bEnableX = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::X);
	const bool bEnableY = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Y);
	const bool bEnableZ = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Z);
	const bool bEnableAll = bEnableX && bEnableY && bEnableZ;

	bGimbalRotationMode = (bEnableX || bEnableY || bEnableZ) && GetRotationContext().bUseExplicitRotator;

	// Default rotation handles
	{
		if (!RotateAxisSetElement)
		{
			RotateAxisSetElement = MakeRotateAxisSet(EAxisList::XYZ);
			GizmoElementRoot->Add(RotateAxisSetElement);
		}

		if (RotateAxisSetElement)
		{
			RotateAxisSetElement->SetAxisEnabled(bGimbalRotationMode ? EAxisList::None : InAxisListToDraw);
		}
	}

	// Gimbal rotation handles
	{
		auto EnableRotateElement = [&](TObjectPtr<UGizmoElementRotateAxis>& InOutElement, const ETransformGizmoPartIdentifier InPartId, const EAxis::Type InAxis, const bool bEnableAxis, UGizmoElementGroup* InElementGroup)
		{
			if (bEnableAxis)
			{
				if (!InOutElement)
				{
					InOutElement = MakeRotateAxis(InPartId, InAxis);
					InElementGroup->Add(InOutElement);
				}
			}

			if (InOutElement)
			{
				const bool bEnableElement = bEnableAxis && bGimbalRotationMode;
				InOutElement->SetEnabled(bEnableElement);
			}
		};

		EnableRotateElement(RotateXGimbalElement2, ETransformGizmoPartIdentifier::RotateXGimbal, EAxis::X, bEnableX, RotateGimbalElement);
		EnableRotateElement(RotateYGimbalElement2, ETransformGizmoPartIdentifier::RotateYGimbal, EAxis::Y, bEnableY, RotateGimbalElement);
		EnableRotateElement(RotateZGimbalElement2, ETransformGizmoPartIdentifier::RotateZGimbal, EAxis::Z, bEnableZ, RotateGimbalElement);

		if (RotateGimbalElement && !RotateGimbalElement->GetSubElements().IsEmpty())
		{
			RotateGimbalElement->SetEnabled(bEnableAll && bGimbalRotationMode);
		}
	}

	// screen space & arc ball handles
	if (bEnableAll)
	{
		if (!RotateScreenSpaceElement2)
		{
			RotateScreenSpaceElement2 = MakeRotateScreenSpaceHandle();
			GizmoElementRoot->Add(RotateScreenSpaceElement2);
		}

		if (!RotateArcballElement)
		{
			// @note: radius and color params are unused and overridden by the Style
			RotateArcballElement = MakeArcballCircleHandle(ETransformGizmoPartIdentifier::RotateArcball, 50.0f, FLinearColor::White);
			GizmoElementRoot->Add(RotateArcballElement);
		}
	}

	if (RotateScreenSpaceElement2)
    {
        RotateScreenSpaceElement2->SetEnabled(bEnableAll && !bGimbalRotationMode);
    }

    if (RotateArcballElement)
    {
        RotateArcballElement->SetEnabled(bEnableAll);
    }
}

void UEditorTRSGizmo::EnableScale(EAxisList::Type InAxisListToDraw)
{
	check(GizmoElementRoot);

	using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;

	const bool bEnableX = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::X);
	const bool bEnableY = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Y);
	const bool bEnableZ = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Z);

	if (!ScaleGroupElement)
	{
		ScaleGroupElement = MakeScaleGroup(EAxisList::XYZ);
		GizmoElementRoot->Add(ScaleGroupElement);
	}

	if (ScaleGroupElement)
	{
		ScaleGroupElement->SetAxisEnabled(InAxisListToDraw);
		ScaleGroupElement->SetPlanarEnabled(UE::Editor::GizmoMath::GetAxisListFromBools(bEnableX, bEnableY, bEnableZ));
		ScaleGroupElement->SetUniformEnabled(bEnableX || bEnableY || bEnableZ);
	}
}

EToolContextCoordinateSystem UEditorTRSGizmo::GetCoordinateSystem() const
{
	if (TransformGizmoSource)
	{
		return TransformGizmoSource->GetGizmoCoordSystemSpace();
	}

	return GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem();
}

EGizmoTransformScaleType UEditorTRSGizmo::GetScaleType() const
{
	if (TransformGizmoSource)
	{
		return TransformGizmoSource->GetScaleType();
	}
	return FGizmoElementScaleInteraction::GetDefaultScaleType();
}

FGizmoPerStateValueMaterialVariant UEditorTRSGizmo::GetAxisMaterials(const EAxis::Type InAxis)
{
	FGizmoPerStateValueMaterialVariant GizmoMaterials;
	GizmoMaterials.Default = FGizmoMaterialVariant{ GetAxisMaterialSolid(InAxis), { } };
	GizmoMaterials.Hover = FGizmoMaterialVariant{ HoverAxisMaterial, { } };
	GizmoMaterials.Select = FGizmoMaterialVariant{ SelectAxisMaterial, { } };
	GizmoMaterials.Interact = FGizmoMaterialVariant{ InteractAxisMaterial, { } };
	GizmoMaterials.Subdue = FGizmoMaterialVariant{ { }, GetAxisMaterialTranslucent(InAxis, true) };

	return GizmoMaterials;
}

FGizmoPerStateValueLinearColor UEditorTRSGizmo::GetAxisColors(const EAxis::Type InAxis) const
{
	return GetAxisColors(EAxisList::FromAxis(InAxis, AxisDisplayInfo::GetAxisDisplayCoordinateSystem()));
}

FGizmoPerStateValueLinearColor UEditorTRSGizmo::GetAxisColors(const EAxisList::Type InAxis) const
{
	using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;

	FGizmoPerStateValueLinearColor GizmoColors;
	GizmoColors.Default = GetAxisColor(InAxis, Style);
	GizmoColors.Hover = Style.HoverColor;
	GizmoColors.Select = Style.SelectColor;
	GizmoColors.Interact = Style.InteractColor;
	GizmoColors.Subdue = Style.SubdueColor;

	return GizmoColors;
}

UMaterialInterface* UEditorTRSGizmo::GetAxisMaterialSolid(const EAxis::Type InAxis)
{
	switch (InAxis)
	{
	case EAxis::X:
		return AxisMaterialX;

	case EAxis::Y:
		return AxisMaterialY;

	case EAxis::Z:
		return AxisMaterialZ;

	case EAxis::None:
	default:
		return nullptr;
	}
}

UMaterialInterface* UEditorTRSGizmo::GetAxisMaterialTranslucent(const EAxis::Type InAxis, const bool bInSubdued)
{
	switch (InAxis)
	{
	case EAxis::X:
		return bInSubdued ? AxisMaterialXSubdued : AxisMaterialXTranslucent;

	case EAxis::Y:
		return bInSubdued ? AxisMaterialYSubdued : AxisMaterialYTranslucent;

	case EAxis::Z:
		return bInSubdued ? AxisMaterialZSubdued : AxisMaterialZTranslucent;

	case EAxis::None:
	default:
		return nullptr;
	}
}

void UEditorTRSGizmo::Tick(float DeltaTime)
{
	if (bInInteraction && !bIndirectManipulation && !CameraFollowsMovement())
	{
		const FVector NewViewLocation = GizmoViewContext->GetViewLocation();
		if (NewViewLocation != CachedViewLocation)
		{
			const FVector RawDelta = NewViewLocation - CachedViewLocation;
			
			switch (LastHitPart)
			{
			case ETransformGizmoPartIdentifier::TranslateScreenSpace:
			case ETransformGizmoPartIdentifier::TranslateXYPlanar:
			case ETransformGizmoPartIdentifier::TranslateXZPlanar:
			case ETransformGizmoPartIdentifier::TranslateYZPlanar:
				{
					const FVector ProjectedDelta = FVector::VectorPlaneProject(RawDelta, InteractionPlanarNormal);
					InteractionPlanarCurrPoint += ProjectedDelta;
				}
				break;
			case ETransformGizmoPartIdentifier::TranslateXAxis:
			case ETransformGizmoPartIdentifier::TranslateYAxis:
			case ETransformGizmoPartIdentifier::TranslateZAxis:
				{
					const FVector LocalAxisDirection = UE::Editor::InteractiveToolsFramework::Internal::GetAxisVector(EAxis::FromAxisList(GetInteractionAxisList()));
					InteractionAxisCurrParam += RawDelta.Dot(LocalAxisDirection);
				}
				break;
			default:
				break;
			}
			
			CachedViewLocation = NewViewLocation;
		}
	}

	if (PendingDragFunction)
	{
		Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::PendingFunctionExecutionName);

		PendingDragFunction();
		PendingDragFunction.Reset();
	}

	UpdateMode();

	UpdateCameraAxisSource();

	// update gimbal handle's rotation context
	if (RotateGimbalElement && !RotateGimbalElement->GetSubElements().IsEmpty())
	{
		RotateGimbalElement->RotationContext = GetRotationContext();
	}

	if (ActiveTarget)
	{
		CurrentTransform = GetActiveTransform();
	}

	if (!bInInteraction && NudgeData.GetCurrentKey() != EKeys::AnyKey)
	{
		using namespace UE::Editor::InteractiveToolsFramework;

		NudgeData.TimeSinceLastNudge += DeltaTime;
		if (NudgeData.TimeSinceLastNudge > NudgeInitialRepeatDelay)
		{
			if (FMath::Fmod(NudgeData.TimeSinceLastNudge - NudgeInitialRepeatDelay, NudgeRepeatInterval) < DeltaTime)
			{
				NudgeSelection(NudgeData.GetCurrentKey());
			}
		}
	}
}

void UEditorTRSGizmo::HandleWidgetModeChanged(UE::Widget::EWidgetMode InWidgetMode)
{
	auto GetTransformMode = [InWidgetMode]()
	{
		switch (InWidgetMode)
		{
		case UE::Widget::EWidgetMode::WM_Translate: return EGizmoTransformMode::Translate;
		case UE::Widget::EWidgetMode::WM_Rotate: return EGizmoTransformMode::Rotate;
		case UE::Widget::EWidgetMode::WM_Scale: return EGizmoTransformMode::Scale;
		default: return EGizmoTransformMode::None;
		}
	};
	const EGizmoTransformMode NewMode = GetTransformMode();

	if (CurrentMode != EGizmoTransformMode::None && NewMode == CurrentMode)
	{
		const ETransformGizmoPartIdentifier CurrentModeLastHitPart = GetCurrentModeLastHitPart();
		auto GetModeDefaultHitPart = [NewMode, CurrentModeLastHitPart, bGimbal = bGimbalRotationMode]()
		{
			const bool bIsRotateArcBall = (CurrentModeLastHitPart == ETransformGizmoPartIdentifier::RotateArcball);
			switch (NewMode)
			{
			case EGizmoTransformMode::Translate:
				return ETransformGizmoPartIdentifier::TranslateScreenSpace;
			case EGizmoTransformMode::Rotate:
				return bGimbal ? ETransformGizmoPartIdentifier::RotateArcball : bIsRotateArcBall ? ETransformGizmoPartIdentifier::RotateScreenSpace : ETransformGizmoPartIdentifier::RotateArcball;
			case EGizmoTransformMode::Scale:
				return ETransformGizmoPartIdentifier::ScaleUniform;
			default:
				return ETransformGizmoPartIdentifier::Default;
			}
		};

		const ETransformGizmoPartIdentifier DefaultHitPart = GetModeDefaultHitPart();
		if (DefaultHitPart != CurrentModeLastHitPart)
		{
			using namespace UE::Editor::InteractiveToolsFramework::Internal;

			// reset indirect manipulation to default
			ResetSelectedStates(CurrentMode);
			ResetInteractingStates(CurrentMode);
			ResetHoverStates(CurrentMode);

			UpdateSelectedState(true, DefaultHitPart, true);
			UpdateInteractingState(true, DefaultHitPart, true);
			SetModeLastHitPart(CurrentMode, DefaultHitPart);
		}
	}

	LastHitPart = ETransformGizmoPartIdentifier::Default;
}

void UEditorTRSGizmo::OnParametersChanged(const FGizmosParameters& InParameters)
{
	if (InParameters.bSequentialIndirectAxesButtons != bSequentialIndirectAxesButtons)
	{
		bSequentialIndirectAxesButtons = InParameters.bSequentialIndirectAxesButtons;
	}

	if (InParameters.bAdditiveIndirectAxes != bAdditiveIndirectAxes)
	{
		bAdditiveIndirectAxes = InParameters.bAdditiveIndirectAxes;
	}
	
	if (InParameters.bScreenSpaceIndirectOrthographicManipulation != bScreenSpaceIndirectOrthographicManipulation)
	{
		bScreenSpaceIndirectOrthographicManipulation = InParameters.bScreenSpaceIndirectOrthographicManipulation;
	}

	if (InParameters.bUniformIndirectScale != bUniformIndirectScale)
	{
		bUniformIndirectScale = InParameters.bUniformIndirectScale;
	}

	DefaultRotateMode = InParameters.RotateMode;

	if (const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>())
	{
		SetStyle(Settings->GizmosParameters.Style);

		Interaction = Settings->GizmosParameters.Interaction;

		if (GizmoLocals::DoDebugDraw()
			&& !Settings->GizmosParameters.Debug.bFreezeDelta)
		{
			// Reset previously frozen deltas
			EndDeltas();

			ResetInteractingStates(CurrentMode);
		}
	}

	Interaction.bPersistHandleSelection = InParameters.bPersistHandleSelection;

	// If disabled, clear current selection
	if (!Interaction.bPersistHandleSelection)
	{
		// Temporarily enable selection persistence to allow ResetSelectedStates to work correctly
		Interaction.bPersistHandleSelection = true;

		ResetSelectedStates(EGizmoTransformMode::None); // "None" means all modes

		// Restore to actual setting
		Interaction.bPersistHandleSelection = false;
	}

	UpdateMaterials();
	UpdateElements();
}

UGizmoElementTranslateGroup* UEditorTRSGizmo::MakeTranslateGroup(const EAxisList::Type InAxisList)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private;

	UGizmoElementTranslateGroup* Element = NewObject<UGizmoElementTranslateGroup>();

	Element->Setup(
		static_cast<uint32>(ETransformGizmoPartIdentifier::TranslateAll),
		InAxisList,
		FTransformGizmoAccessorPrivate::MakeTranslateStyle(Style, TransparentVertexColorMaterial, GetSizeCoefficient()),
		FTransformGizmoAccessorPrivate::MakeTranslateAxisParameters(*this, ETransformGizmoPartIdentifier::TranslateXAxis, EAxis::X),
		FTransformGizmoAccessorPrivate::MakeTranslateAxisParameters(*this, ETransformGizmoPartIdentifier::TranslateYAxis, EAxis::Y),
		FTransformGizmoAccessorPrivate::MakeTranslateAxisParameters(*this, ETransformGizmoPartIdentifier::TranslateZAxis, EAxis::Z),
		FTransformGizmoAccessorPrivate::MakeTranslatePlanarParameters(*this, ETransformGizmoPartIdentifier::TranslateXYPlanar, EAxisList::XY, EAxis::Z),
		FTransformGizmoAccessorPrivate::MakeTranslatePlanarParameters(*this, ETransformGizmoPartIdentifier::TranslateYZPlanar, EAxisList::YZ, EAxis::X),
		FTransformGizmoAccessorPrivate::MakeTranslatePlanarParameters(*this, ETransformGizmoPartIdentifier::TranslateXZPlanar, EAxisList::XZ, EAxis::Y),
		static_cast<uint32>(ETransformGizmoPartIdentifier::TranslateScreenSpace));

	Element->SetGetCurrentSnappingSettingsFunction([QueriesAPI = GetGizmoManager()->GetContextQueriesAPI()]() -> FToolContextSnappingConfiguration
	{
		if (QueriesAPI)
		{
			return QueriesAPI->GetCurrentSnappingSettings();
		}

		return { };
	});

	if (IAssetEditorContextInterface* AssetEditorContext = GetGizmoManager()->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>())
	{
		if (IToolkitHost* ToolkitHost = AssetEditorContext->GetMutableToolkitHost())
		{
			Element->SetWidgetHost(ToolkitHost);
		}
	}

	UpdateTranslateGroup(Element);

	return Element;
}

void UEditorTRSGizmo::UpdateTranslateGroup(UGizmoElementTranslateGroup* InElement)
{
	if (!InElement)
	{
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Private;

	InElement->SetStyle(
		FTransformGizmoAccessorPrivate::MakeTranslateStyle(Style, TransparentVertexColorMaterial, GetSizeCoefficient()),
		FTransformGizmoAccessorPrivate::MakeTranslateAxisStyleOverride(*this, EAxis::X),
		FTransformGizmoAccessorPrivate::MakeTranslateAxisStyleOverride(*this, EAxis::Y),
		FTransformGizmoAccessorPrivate::MakeTranslateAxisStyleOverride(*this, EAxis::Z),
		FTransformGizmoAccessorPrivate::MakeTranslatePlanarStyleOverride(*this, EAxisList::XY, EAxis::Z),
		FTransformGizmoAccessorPrivate::MakeTranslatePlanarStyleOverride(*this, EAxisList::YZ, EAxis::X),
		FTransformGizmoAccessorPrivate::MakeTranslatePlanarStyleOverride(*this, EAxisList::XZ, EAxis::Y));

	InElement->UpdateElements();
}

UGizmoElementScaleGroup* UEditorTRSGizmo::MakeScaleGroup(const EAxisList::Type InAxisList)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private;

	UGizmoElementScaleGroup* Element = NewObject<UGizmoElementScaleGroup>();

	Element->Setup(
		static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleAll),
		FTransformGizmoAccessorPrivate::MakeScaleStyle(*this, Style, TransparentVertexColorMaterial, GetSizeCoefficient()),
		FTransformGizmoAccessorPrivate::MakeScaleAxisParameters(*this, ETransformGizmoPartIdentifier::ScaleXAxis, EAxis::X),
		FTransformGizmoAccessorPrivate::MakeScaleAxisParameters(*this, ETransformGizmoPartIdentifier::ScaleYAxis, EAxis::Y),
		FTransformGizmoAccessorPrivate::MakeScaleAxisParameters(*this, ETransformGizmoPartIdentifier::ScaleZAxis, EAxis::Z),
		FTransformGizmoAccessorPrivate::MakeScalePlanarParameters(*this, ETransformGizmoPartIdentifier::ScaleXYPlanar, EAxisList::XY, EAxis::Z),
		FTransformGizmoAccessorPrivate::MakeScalePlanarParameters(*this, ETransformGizmoPartIdentifier::ScaleYZPlanar, EAxisList::YZ, EAxis::X),
		FTransformGizmoAccessorPrivate::MakeScalePlanarParameters(*this, ETransformGizmoPartIdentifier::ScaleXZPlanar, EAxisList::XZ, EAxis::Y),
		static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleUniform));

	Element->SetGetCurrentSnappingSettingsFunction([QueriesAPI = GetGizmoManager()->GetContextQueriesAPI()]() -> FToolContextSnappingConfiguration
	{
		if (QueriesAPI)
		{
			return QueriesAPI->GetCurrentSnappingSettings();
		}

		return { };
	});

	if (IAssetEditorContextInterface* AssetEditorContext = GetGizmoManager()->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>())
	{
		if (IToolkitHost* ToolkitHost = AssetEditorContext->GetMutableToolkitHost())
		{
			Element->SetWidgetHost(ToolkitHost);
		}
	}

	UpdateScaleGroup(Element);

	return Element;
}

void UEditorTRSGizmo::UpdateScaleGroup(UGizmoElementScaleGroup* InElement)
{
	if (!InElement)
	{
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Private;

	InElement->SetStyle(
		FTransformGizmoAccessorPrivate::MakeScaleStyle(*this, Style, TransparentVertexColorMaterial, GetSizeCoefficient()),
		FTransformGizmoAccessorPrivate::MakeScaleAxisStyleOverride(*this, EAxis::X),
		FTransformGizmoAccessorPrivate::MakeScaleAxisStyleOverride(*this, EAxis::Y),
		FTransformGizmoAccessorPrivate::MakeScaleAxisStyleOverride(*this, EAxis::Z),
		FTransformGizmoAccessorPrivate::MakeScalePlanarStyleOverride(*this, EAxisList::XY, EAxis::Z),
		FTransformGizmoAccessorPrivate::MakeScalePlanarStyleOverride(*this, EAxisList::YZ, EAxis::X),
		FTransformGizmoAccessorPrivate::MakeScalePlanarStyleOverride(*this, EAxisList::XZ, EAxis::Y));

	InElement->SetInteraction(Interaction.ScaleInteraction);

	InElement->UpdateElements();
}

UGizmoElementTorus* UEditorTRSGizmo::MakeRotateAxis(ETransformGizmoPartIdentifier InPartId, const FVector& TorusAxis0, const FVector& TorusAxis1, UMaterialInterface* InMaterial)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;

	UGizmoElementTorus* RotateAxisElement = NewObject<UGizmoElementTorus>();
	RotateAxisElement->SetPartIdentifier(static_cast<uint32>(InPartId), true);
	RotateAxisElement->SetCenter(FVector::ZeroVector);
	RotateAxisElement->SetNumSegments(Style.RotateStyle.AxisStyle.NumSegments);
	RotateAxisElement->SetNumInnerSlices(Style.RotateStyle.AxisStyle.NumInnerSlices);
	RotateAxisElement->SetAxisBitangent(TorusAxis0);
	RotateAxisElement->SetAxisTangent(TorusAxis1);
	const FVector TorusNormal = RotateAxisElement->GetAxisTangent() ^ RotateAxisElement->GetAxisBitangent();
	RotateAxisElement->SetPartialType(EGizmoElementPartialType::PartialViewDependent);
	RotateAxisElement->SetPartialStartAngle(0.0f);
	RotateAxisElement->SetPartialEndAngle(UE_PI);
	RotateAxisElement->SetViewDependentAxis(TorusNormal);
	RotateAxisElement->SetViewAlignType(EGizmoElementViewAlignType::Axial);
	RotateAxisElement->SetViewAlignAxialAngleTol(static_cast<float>(UE_DOUBLE_SMALL_NUMBER));
	RotateAxisElement->SetViewAlignAxis(TorusNormal);
	RotateAxisElement->SetViewAlignNormal(TorusAxis1);
	RotateAxisElement->SetMaterial(InMaterial);
	RotateAxisElement->SetSubdueMaterial(InMaterial);
	UpdateRotateAxis(RotateAxisElement);

	return RotateAxisElement;
}

void UEditorTRSGizmo::UpdateRotateAxis(UGizmoElementTorus* InElement)
{
	if (!InElement)
	{
		return;
	}

	const float SizeCoeff = GetSizeCoefficient();

	InElement->SetRadius(((Style.RotateStyle.AxisStyle.Radius * Style.GetModifiedAxisSizeMultiplier())) * SizeCoeff);
	InElement->SetInnerRadius(FMath::Max(Style.MinLineThickness * 0.5f, Style.RotateStyle.AxisStyle.LineThickness * Style.LineThicknessMultiplier) * SizeCoeff);

	InElement->SetHoverLineColor(Style.HoverColor);
	InElement->SetHoverVertexColor(Style.HoverColor);
	InElement->SetInteractLineColor(Style.InteractColor);
	InElement->SetInteractVertexColor(Style.InteractColor);

	UpdateElement(InElement, Style.RotateStyle.AxisStyle);
}

UGizmoElementRotateAxisSet* UEditorTRSGizmo::MakeRotateAxisSet(const EAxisList::Type InAxisList)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private;

	UGizmoElementRotateAxisSet* Element = NewObject<UGizmoElementRotateAxisSet>();

	Element->Setup(
		InAxisList,
		FTransformGizmoAccessorPrivate::MakeRotateAxisStyle(Style, TransparentVertexColorMaterial, GetSizeCoefficient()),
		FTransformGizmoAccessorPrivate::MakeRotateAxisParameters(*this, ETransformGizmoPartIdentifier::RotateXAxis, EAxis::X),
		FTransformGizmoAccessorPrivate::MakeRotateAxisParameters(*this, ETransformGizmoPartIdentifier::RotateYAxis, EAxis::Y),
		FTransformGizmoAccessorPrivate::MakeRotateAxisParameters(*this, ETransformGizmoPartIdentifier::RotateZAxis, EAxis::Z));

	if (IAssetEditorContextInterface* AssetEditorContext = GetGizmoManager()->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>())
	{
		if (IToolkitHost* ToolkitHost = AssetEditorContext->GetMutableToolkitHost())
		{
			Element->SetWidgetHost(ToolkitHost);
		}
	}

	UpdateRotateAxisSet(Element);

	return Element;
}

void UEditorTRSGizmo::UpdateRotateAxisSet(UGizmoElementRotateAxisSet* InElement)
{
	if (!InElement)
	{
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Private;

	InElement->SetStyle(
	FTransformGizmoAccessorPrivate::MakeRotateAxisStyle(Style, TransparentVertexColorMaterial, GetSizeCoefficient()),
		FTransformGizmoAccessorPrivate::MakeRotateAxisStyleOverride(*this, EAxis::X),
		FTransformGizmoAccessorPrivate::MakeRotateAxisStyleOverride(*this, EAxis::Y),
		FTransformGizmoAccessorPrivate::MakeRotateAxisStyleOverride(*this, EAxis::Z));

	InElement->UpdateElements();
}

UGizmoElementRotateAxis* UEditorTRSGizmo::MakeRotateAxis(const ETransformGizmoPartIdentifier InPartId, const EAxis::Type InAxis)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private;

	UGizmoElementRotateAxis* Element = NewObject<UGizmoElementRotateAxis>();

	Element->SetGizmoViewContext(GizmoViewContext);

	const FGizmoPerStateValueMaterialVariant GizmoMaterials = GetAxisMaterials(InAxis);
	const FGizmoPerStateValueLinearColor GizmoColors = GetAxisColors(InAxis);

	Element->Setup(
		static_cast<uint32>(InPartId),
		EAxisList::FromAxis(InAxis),
		FTransformGizmoAccessorPrivate::MakeRotateAxisStyle(
			Style,
			GizmoMaterials,
			GizmoColors,
			GetAxisMaterialTranslucent(InAxis),
			TransparentVertexColorMaterial,
			GetSizeCoefficient()));

	if (IAssetEditorContextInterface* AssetEditorContext = GetGizmoManager()->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>())
	{
		if (IToolkitHost* ToolkitHost = AssetEditorContext->GetMutableToolkitHost())
		{
			Element->SetWidgetHost(ToolkitHost);
		}
	}

	UpdateRotateAxis(Element, InAxis);

	return Element;
}

UGizmoElementTorus* UEditorTRSGizmo::MakeRotateAxis(ETransformGizmoPartIdentifier InPartId, const FVector& TorusAxis0, const FVector& TorusAxis1, UMaterialInterface* InMaterial, UMaterialInterface* InCurrentMaterial)
{
	return UTransformGizmo::MakeRotateAxis(InPartId, TorusAxis0, TorusAxis1, InMaterial, InCurrentMaterial);
}

void UEditorTRSGizmo::UpdateRotateAxis(UGizmoElementRotateAxis* InElement, const EAxis::Type InAxis)
{
	if (!InElement)
	{
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Private;

	const FGizmoPerStateValueMaterialVariant GizmoMaterials = GetAxisMaterials(InAxis);
	const FGizmoPerStateValueLinearColor GizmoColors = GetAxisColors(InAxis);

	InElement->SetStyle(
		FTransformGizmoAccessorPrivate::MakeRotateAxisStyle(Style, GizmoMaterials, GizmoColors, GetAxisMaterialTranslucent(InAxis), TransparentVertexColorMaterial, GetSizeCoefficient()));

	InElement->UpdateElements();
}

UGizmoElementArc* UEditorTRSGizmo::MakeRotateAxisArc(ETransformGizmoPartIdentifier InPartId, const FVector& InAxis0, const FVector& InAxis1, UMaterialInterface* InMaterial)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;

	UGizmoElementArc* RotateArcElement = NewObject<UGizmoElementArc>();
	RotateArcElement->SetPartIdentifier(static_cast<uint32>(InPartId), true);
	RotateArcElement->SetCenter(FVector::ZeroVector);
	RotateArcElement->SetNumSegments(Style.RotateStyle.AxisStyle.NumSegments);
	RotateArcElement->SetAxisBitangent(InAxis0);
	RotateArcElement->SetAxisTangent(InAxis1);
	RotateArcElement->SetPartialType(EGizmoElementPartialType::Partial);
	RotateArcElement->SetViewAlignType(EGizmoElementViewAlignType::None);

	RotateArcElement->SetHittableState(false);
	RotateArcElement->SetEnabledForDefaultState(true);
	RotateArcElement->SetEnabledForHoveringState(true);
	RotateArcElement->SetEnabledForInteractingState(true);

	UpdateRotateAxisArc(RotateArcElement, InMaterial);

	return RotateArcElement;
}

void UEditorTRSGizmo::UpdateRotateAxisArc(UGizmoElementArc* InElement, UMaterialInterface* InMaterial)
{
	if (!InElement)
	{
		return;
	}

	const float SizeCoeff = GetSizeCoefficient();

	InElement->SetRadius(((Style.RotateStyle.AxisStyle.Radius * Style.GetModifiedAxisSizeMultiplier())) * SizeCoeff);
	InElement->SetInnerRadius(0.0f); // We want a pizza slice here, so the inner radius is 0

	InElement->SetPartialStartAngle(0.0);
	InElement->SetPartialEndAngle(UE_DOUBLE_TWO_PI);

	InElement->SetMaterial(InMaterial);
	InElement->SetSubdueMaterial(InMaterial);

	InElement->SetHoverLineColor(Style.HoverColor);
	InElement->SetHoverVertexColor(Style.HoverColor);
	InElement->SetInteractLineColor(Style.InteractColor);
	InElement->SetInteractVertexColor(Style.InteractColor);

	UpdateElement(InElement, Style.RotateStyle.AxisStyle);
}

UGizmoElementCircle* UEditorTRSGizmo::MakeArcballCircleHandle(ETransformGizmoPartIdentifier InPartId, float InRadius, const FLinearColor& InColor)
{
	UGizmoElementCircle* Element = NewObject<UGizmoElementCircle>();
	Element->SetPartIdentifier(static_cast<uint32>(InPartId), true);
	Element->SetHitPriority(-10); // Ensure this element is hit after all other candidates are considered
	Element->SetCenter(FVector::ZeroVector);
	Element->SetAxisBitangent(FVector::UpVector);
	Element->SetAxisTangent(-FVector::RightVector);
	Element->SetViewAlignType(EGizmoElementViewAlignType::PointOnly);
	Element->SetViewAlignNormal(-FVector::ForwardVector);

	// Disable Draw/Hit by default, and toggle below based on the provided flags
	Element->SetDrawMesh(false);
	Element->SetHitMesh(false);
	Element->SetDrawLine(false);
	Element->SetHitLine(false);

	Element->SetEnabledForDefaultState(true);
	Element->SetEnabledForHoveringState(true);
	Element->SetEnabledForInteractingState(true);

	UpdateArcballCircleHandle(Element);

	return Element;
}

void UEditorTRSGizmo::UpdateArcballCircleHandle(UGizmoElementCircle* InElement)
{
	if (!InElement)
	{
		return;
	}

	const float SizeCoeff = GetSizeCoefficient();

	const FLinearColor FillColor = Style.RotateStyle.ArcballStyle.Colors.Default.Get(FLinearColor::White);
	const FLinearColor LineColor = Style.RotateStyle.ArcballStyle.LineColors.Default.Get(FLinearColor::White);

	InElement->SetViewDepthOffset(100.0f); // Push behind all other elements

	// Fill
	{
		InElement->SetDrawMesh(true);
		InElement->SetHitMesh(true);

		InElement->SetVertexColor(FLinearColor::Transparent);
		InElement->SetMaterial(TransparentVertexColorMaterial);

		InElement->SetHoverVertexColor(FillColor);
		InElement->SetHoverMaterial(TransparentVertexColorMaterial);

		InElement->SetInteractVertexColor(FillColor);
		InElement->SetInteractMaterial(TransparentVertexColorMaterial);

		InElement->SetSubdueVertexColor(FLinearColor::Transparent);
		InElement->SetSubdueMaterial(TransparentVertexColorMaterial);

		InElement->SetSelectVertexColor(FillColor);
		InElement->SetSelectMaterial(TransparentVertexColorMaterial);
	}

	// Line
	{
		InElement->SetDrawLine(true);
		InElement->SetHitLine(true);

		InElement->SetLineColor(LineColor);
		InElement->SetHoverLineColor(LineColor);
		InElement->SetInteractLineColor(LineColor);
		InElement->SetSubdueLineColor(LineColor);
		InElement->SetSelectLineColor(LineColor);
	}

	InElement->SetRadius((Style.RotateStyle.ArcballStyle.Radius * Style.GetModifiedAxisSizeMultiplier()) * SizeCoeff);
	InElement->SetLineThickness(FMath::Max(Style.MinLineThickness, Style.RotateStyle.AxisStyle.LineThickness * Style.LineThicknessMultiplier));
	InElement->SetHoverLineThicknessMultiplier(Style.HoverLineThicknessMultiplier);
	InElement->SetInteractLineThicknessMultiplier(1.0f);
	InElement->SetSelectLineThicknessMultiplier(1.0f);

	UpdateElement(InElement, Style.RotateStyle.ArcballStyle);
}

UGizmoElementRotateAxis* UEditorTRSGizmo::MakeRotateScreenSpaceHandle()
{
	using namespace UE::Editor::InteractiveToolsFramework::Private;

	UGizmoElementRotateAxis* Element = NewObject<UGizmoElementRotateAxis>();

	FGizmoPerStateValueMaterialVariant GizmoMaterials;
	GizmoMaterials.Default.Emplace(FGizmoMaterialVariant({ }, TransparentVertexColorMaterial));

	FGizmoPerStateValueLinearColor GizmoColors;
	GizmoColors.Default = Style.RotateStyle.ScreenSpaceCircleColor;

	FGizmoElementRotateAxisStyle ElementStyle =
		FTransformGizmoAccessorPrivate::MakeRotateAxisStyle(
			Style,
			GizmoMaterials,
			GizmoColors,
			AxisMaterialScreenSpaceTranslucent,
			TransparentVertexColorMaterial,
			GetSizeCoefficient());

	ElementStyle.Radius = Style.RotateStyle.AxisStyle.Radius + Style.RotateStyle.ScreenSpaceRadiusOffset;
	ElementStyle.LineThickness = Style.RotateStyle.ScreenSpaceLineThickness;
	ElementStyle.LineThicknessMultiplier = Style.LineThicknessMultiplier;
	ElementStyle.DeltaFillHSVModifier = FLinearColor(0.0f, 1.0f, 1.0f);

	Element->Setup(
		static_cast<uint32>(ETransformGizmoPartIdentifier::RotateScreenSpace),
		EAxisList::Screen,
		ElementStyle);

	UpdateRotateScreenSpaceHandle(Element);

	Element->SetGizmoViewContext(GizmoViewContext);
	Element->SetViewAlignType(EGizmoElementViewAlignType::PointEye);
	Element->SetViewAlignNormal(-FVector::ForwardVector);
	Element->SetViewAlignAxis(FVector::UpVector);

	if (IAssetEditorContextInterface* AssetEditorContext = GetGizmoManager()->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>())
	{
		if (IToolkitHost* ToolkitHost = AssetEditorContext->GetMutableToolkitHost())
		{
			Element->SetWidgetHost(ToolkitHost);
		}
	}

	return Element;
}

void UEditorTRSGizmo::UpdateRotateScreenSpaceHandle(UGizmoElementRotateAxis* InElement)
{
	using namespace UE::Editor::InteractiveToolsFramework;

	if (!InElement)
	{
		return;
	}

	FGizmoPerStateValueMaterialVariant GizmoMaterials;
	GizmoMaterials.Default.Emplace(FGizmoMaterialVariant({ }, TransparentVertexColorMaterial));

	FGizmoPerStateValueLinearColor GizmoColors;
	GizmoColors.Default = Style.RotateStyle.ScreenSpaceCircleColor;

	FGizmoElementRotateAxisStyle ElementStyle = Private::FTransformGizmoAccessorPrivate::MakeRotateAxisStyle(
		Style,
		GizmoMaterials,
		GizmoColors,
		AxisMaterialScreenSpaceTranslucent,
		TransparentVertexColorMaterial,
		GetSizeCoefficient());

	ElementStyle.Radius = Style.RotateStyle.AxisStyle.Radius + Style.RotateStyle.ScreenSpaceRadiusOffset;
	ElementStyle.LineThickness = Style.RotateStyle.ScreenSpaceLineThickness;
	ElementStyle.LineThicknessMultiplier = Style.LineThicknessMultiplier;
	ElementStyle.DeltaFillHSVModifier = FLinearColor(0.0f, 1.0f, 1.0f);

	InElement->SetStyle(ElementStyle);
	InElement->UpdateElements();

	InElement->SetViewAlignType(EGizmoElementViewAlignType::PointEye);
	InElement->SetViewAlignNormal(-FVector::ForwardVector);
	InElement->SetViewAlignAxis(FVector::UpVector);

	UpdateElement(InElement, Style.RotateStyle.AxisStyle);
}

void UEditorTRSGizmo::UpdateElement(UGizmoElementBase* InElement, const TOptional<FGizmoStyleBase>& InStyle)
{
	if (!ensure(InElement))
	{
		return;
	}

	InElement->SetPixelHitDistanceThreshold(InStyle.IsSet() ? InStyle->PixelHitDistanceThreshold : Style.PixelHitDistanceThreshold);
	InElement->SetMinimumPixelHitDistanceThreshold(InStyle.IsSet() ? InStyle->MinimumPixelHitDistanceThreshold : Style.MinimumPixelHitDistanceThreshold);
}

void UEditorTRSGizmo::UpdateElements()
{
	const FVector XAxis = FVector::XAxisVector;
	const FVector YAxis = FVector::YAxisVector;
	const FVector ZAxis = FVector::ZAxisVector;

	if (GizmoElementRoot)
	{
		GizmoElementRoot->SetHoverMaterial(HoverAxisMaterial);
		GizmoElementRoot->SetInteractMaterial(InteractAxisMaterial);
		GizmoElementRoot->SetSelectMaterial(SelectAxisMaterial);
		GizmoElementRoot->SetSubdueMaterial(SubdueAxisMaterial);

		GizmoElementRoot->SetHoverLineColor(Style.HoverColor);
		GizmoElementRoot->SetHoverVertexColor(Style.HoverColor);
		GizmoElementRoot->SetInteractLineColor(Style.InteractColor);
		GizmoElementRoot->SetInteractVertexColor(Style.InteractColor);
		GizmoElementRoot->SetSelectLineColor(Style.SelectColor);
		GizmoElementRoot->SetSelectVertexColor(Style.SelectColor);
	}

	if (RotateGimbalElement && !RotateGimbalElement->GetSubElements().IsEmpty())
	{
		RotateGimbalElement->SetHoverMaterial(HoverAxisMaterial);
		RotateGimbalElement->SetInteractMaterial(InteractAxisMaterial);
		RotateGimbalElement->SetSelectMaterial(SelectAxisMaterial);
		RotateGimbalElement->SetSubdueMaterial(SubdueAxisMaterial);

		RotateGimbalElement->SetHoverLineColor(Style.HoverColor);
		RotateGimbalElement->SetHoverVertexColor(Style.HoverColor);
		RotateGimbalElement->SetInteractLineColor(Style.InteractColor);
		RotateGimbalElement->SetInteractVertexColor(Style.InteractColor);
		RotateGimbalElement->SetSelectLineColor(Style.SelectColor);
		RotateGimbalElement->SetSelectVertexColor(Style.SelectColor);

		UpdateRotateAxis(RotateGimbalElement->GetAxisElement<UGizmoElementRotateAxis>(EAxis::X), EAxis::X);
		UpdateRotateAxis(RotateGimbalElement->GetAxisElement<UGizmoElementRotateAxis>(EAxis::Y), EAxis::Y);
		UpdateRotateAxis(RotateGimbalElement->GetAxisElement<UGizmoElementRotateAxis>(EAxis::Z), EAxis::Z);
	}

	UpdateTranslateGroup(TranslateGroupElement);
	UpdateRotateAxisSet(RotateAxisSetElement);
	UpdateScaleGroup(ScaleGroupElement);

	UpdateArcballCircleHandle(RotateArcballElement);
	UpdateRotateScreenSpaceHandle(RotateScreenSpaceElement2);
}

FTransform UEditorTRSGizmo::GetGizmoTransform() const
{
	if (TransformAdjuster.IsValid() && GizmoViewContext)
	{
		return TransformAdjuster->GetAdjustedComponentToWorld(*GizmoViewContext, ActiveTarget ? GetActiveTransform() : CurrentTransform);
	}

	return CurrentTransform;
}

void UEditorTRSGizmo::UpdateMaterials()
{
	const FName GizmoColorParameterName = "GizmoColor";

	auto SetCommonParametersWithColor = [&, GizmoColorParameterName](UMaterialInstanceDynamic* InMaterialInstance, const FLinearColor& InColor)
	{
		using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;
		InMaterialInstance->SetVectorParameterValue(GizmoColorParameterName, InColor);
	};

	auto SetCommonParameters = [&](UMaterialInstanceDynamic* InMaterialInstance, const EAxisList::Type InAxis, const float InOpacity = 1.0f)
	{
		using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;
		SetCommonParametersWithColor(InMaterialInstance, GetAxisColor(InAxis, Style).CopyWithNewOpacity(InOpacity));
	};

	auto SetShadingParameters = [&](UMaterialInstanceDynamic* InMaterialInstance)
	{
		if (Style.bUseShading)
		{
			InMaterialInstance->SetScalarParameterValue("Ambient", Style.ShadingAmbient);
			InMaterialInstance->SetScalarParameterValue("SpecularGlossiness", Style.ShadingSpecularGlossiness);
		}
	};

	UMaterialInterface* AxisMaterialSolid = GetMaterialPermutation(Style.bUseShading, false, false);
	UMaterialInterface* AxisMaterialTranslucent = GetMaterialPermutation(Style.bUseShading, true, false);

	// We make this assumption based on a single material, where a series of them are based on the same one (that may have changed)
	bool bInstantiateMaterials = AxisMaterialX == nullptr || (AxisMaterialX && AxisMaterialX->GetBaseMaterial() != AxisMaterialSolid);

	// UseShading can be toggled, so recreate the materials if needed
	if (bInstantiateMaterials)
	{
		AxisMaterialX = UMaterialInstanceDynamic::Create(AxisMaterialSolid, nullptr);
		AxisMaterialY = UMaterialInstanceDynamic::Create(AxisMaterialSolid, nullptr);
		AxisMaterialZ = UMaterialInstanceDynamic::Create(AxisMaterialSolid, nullptr);
	}

	SetCommonParameters(AxisMaterialX, EAxisList::X);
	SetCommonParameters(AxisMaterialY, EAxisList::Y);
	SetCommonParameters(AxisMaterialZ, EAxisList::Z);

	SetShadingParameters(AxisMaterialX);
	SetShadingParameters(AxisMaterialY);
	SetShadingParameters(AxisMaterialZ);

	const float AxisOpacity = Style.RotateStyle.AxisStyle.DeltaFillOpacity;
	const float SubduedOpacity = Style.SubdueColor.A;

	if (bInstantiateMaterials)
	{
		AxisMaterialScreenSpaceTranslucent = UMaterialInstanceDynamic::Create(AxisMaterialTranslucent, nullptr);

		AxisMaterialXTranslucent = UMaterialInstanceDynamic::Create(AxisMaterialTranslucent, nullptr);
		AxisMaterialYTranslucent = UMaterialInstanceDynamic::Create(AxisMaterialTranslucent, nullptr);
		AxisMaterialZTranslucent = UMaterialInstanceDynamic::Create(AxisMaterialTranslucent, nullptr);

		AxisMaterialXSubdued = UMaterialInstanceDynamic::Create(AxisMaterialTranslucent, nullptr);
		AxisMaterialYSubdued = UMaterialInstanceDynamic::Create(AxisMaterialTranslucent, nullptr);
		AxisMaterialZSubdued = UMaterialInstanceDynamic::Create(AxisMaterialTranslucent, nullptr);
	}

	AxisMaterialScreenSpaceTranslucent->SetVectorParameterValue(GizmoColorParameterName, Style.RotateStyle.ScreenSpaceCircleColor.CopyWithNewOpacity(AxisOpacity));

	SetCommonParameters(AxisMaterialXTranslucent, EAxisList::X, AxisOpacity);
	SetCommonParameters(AxisMaterialYTranslucent, EAxisList::Y, AxisOpacity);
	SetCommonParameters(AxisMaterialZTranslucent, EAxisList::Z, AxisOpacity);

	SetCommonParameters(AxisMaterialXSubdued, EAxisList::X, SubduedOpacity);
	SetCommonParameters(AxisMaterialYSubdued, EAxisList::Y, SubduedOpacity);
	SetCommonParameters(AxisMaterialZSubdued, EAxisList::Z, SubduedOpacity);

	auto CreateMIDIfNeeded = [bInstantiateMaterials](TObjectPtr<UMaterialInstanceDynamic>& InOutMaterialInstance, UMaterialInterface* InBaseMaterial)
	{
		if (!InOutMaterialInstance || bInstantiateMaterials)
		{
			InOutMaterialInstance = UMaterialInstanceDynamic::Create(InBaseMaterial, nullptr);
		}
	};

	CreateMIDIfNeeded(GreyMaterial, AxisMaterialSolid);
	CreateMIDIfNeeded(WhiteMaterial, AxisMaterialSolid);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CreateMIDIfNeeded(CurrentAxisMaterial, AxisMaterialSolid);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CreateMIDIfNeeded(HoverAxisMaterial, AxisMaterialSolid);
	CreateMIDIfNeeded(InteractAxisMaterial, AxisMaterialSolid);
	CreateMIDIfNeeded(SelectAxisMaterial, AxisMaterialSolid);
	CreateMIDIfNeeded(SubdueAxisMaterial, AxisMaterialTranslucent);

	CreateMIDIfNeeded(OpaquePlaneMaterialXY, AxisMaterialSolid);
	CreateMIDIfNeeded(UniformScaleMaterial, AxisMaterialSolid);

	SetShadingParameters(GreyMaterial);
	SetShadingParameters(WhiteMaterial);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetShadingParameters(CurrentAxisMaterial);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	SetShadingParameters(OpaquePlaneMaterialXY);
	SetShadingParameters(UniformScaleMaterial);

	GreyMaterial->SetVectorParameterValue(GizmoColorParameterName, Style.GreyColor);
	WhiteMaterial->SetVectorParameterValue(GizmoColorParameterName, Style.WhiteColor);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CurrentAxisMaterial->SetVectorParameterValue(GizmoColorParameterName, Style.CurrentColor);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	UniformScaleMaterial->SetVectorParameterValue(GizmoColorParameterName, Style.ScaleStyle.UniformStyle.Colors.GetDefaultValue());

	SetCommonParametersWithColor(HoverAxisMaterial, Style.HoverColor);
	SetCommonParametersWithColor(InteractAxisMaterial, Style.InteractColor);
	SetCommonParametersWithColor(SelectAxisMaterial, Style.SelectColor);
	SetCommonParametersWithColor(SubdueAxisMaterial, Style.SubdueColor);

	SetShadingParameters(HoverAxisMaterial);
	SetShadingParameters(InteractAxisMaterial);
	SetShadingParameters(SelectAxisMaterial);
	SetShadingParameters(SubdueAxisMaterial);
	SetShadingParameters(UniformScaleMaterial);

	OpaquePlaneMaterialXY->SetVectorParameterValue(GizmoColorParameterName, FLinearColor::White);

	if (!TransparentVertexColorMaterial)
	{
		TransparentVertexColorMaterial = GetMaterialPermutation(false, false, true);
	}

	if (!GridMaterial)
	{
		GridMaterial = (UMaterial*)StaticLoadObject(
			UMaterial::StaticClass(), nullptr,
			TEXT("/Engine/EditorMaterials/WidgetGridVertexColorMaterial_Ma.WidgetGridVertexColorMaterial_Ma"), nullptr,
			LOAD_None, nullptr);

		if (!GridMaterial)
		{
			GridMaterial = TransparentVertexColorMaterial;
		}
	}
}

void UEditorTRSGizmo::SetGizmoViewContext(UGizmoViewContext* InGizmoViewContext)
{
	GizmoViewContext = InGizmoViewContext;
}

void UEditorTRSGizmo::SetupOnHoverFunctions()
{
	constexpr int32 NumParts = static_cast<int>(ETransformGizmoPartIdentifier::Max);
	HoverElements.SetNum(NumParts);
}

UMaterialInterface* UEditorTRSGizmo::GetMaterialPermutation(const bool bInUseShading, const bool bInTranslucent, const bool bInVertexColored) const
{
	constexpr const TCHAR* BaseMaterialPath_SolidShaded = TEXT("/Engine/EditorMaterials/TransformGizmoMaterial_PseudoLitMasked.TransformGizmoMaterial_PseudoLitMasked");
	constexpr const TCHAR* BaseMaterialPath_SolidUnshaded = TEXT("/Engine/EditorMaterials/TransformGizmoMaterial_UnlitMasked.TransformGizmoMaterial_UnlitMasked");
	constexpr const TCHAR* BaseMaterialPath_UnlitMasked = TEXT("/Engine/EditorMaterials/TransformGizmoMaterial_UnlitMasked.TransformGizmoMaterial_UnlitMasked");
	constexpr const TCHAR* BaseMaterialPath_VertexColor_SolidShaded = TEXT("/Engine/EditorMaterials/TransformGizmoMaterial_VertexColor_PseudoLitMasked.TransformGizmoMaterial_VertexColor_PseudoLitMasked");
	constexpr const TCHAR* BaseMaterialPath_VertexColor_SolidUnshaded = TEXT("/Engine/EditorMaterials/TransformGizmoMaterial_VertexColor_UnlitMasked.TransformGizmoMaterial_VertexColor_UnlitMasked");

	auto GetCustomOrBaseMaterial = [this](const FString& InMaterialName) -> UMaterialInterface*
	{
		if (CustomizationFunction)
		{
			const FGizmoCustomization& GizmoCustomization = CustomizationFunction();
			if (IsValid(GizmoCustomization.Material))
			{
				return GizmoCustomization.Material.Get();
			}
		}

		UMaterialInterface* Material = FindObject<UMaterialInterface>(nullptr, *InMaterialName);
		if (!Material)
		{
			Material = LoadObject<UMaterialInterface>(nullptr, *InMaterialName, nullptr, LOAD_None, nullptr);
		}

		return Material ? Material : GEngine->ArrowMaterial.Get();
	};

	if (bInTranslucent)
	{
		return GetCustomOrBaseMaterial(BaseMaterialPath_UnlitMasked);
	}

	if (bInVertexColored)
	{
		return GetCustomOrBaseMaterial(
			bInUseShading
			? BaseMaterialPath_VertexColor_SolidShaded
			: BaseMaterialPath_VertexColor_SolidUnshaded);
	}

	return GetCustomOrBaseMaterial(
			bInUseShading
			? BaseMaterialPath_SolidShaded
			: BaseMaterialPath_SolidUnshaded);
}

bool UEditorTRSGizmo::CanInteractWithPart(const ETransformGizmoPartIdentifier InPartId) const
{
	bool bCanInteractWithPart = true;

	if (bGimbalRotationMode
		&& InPartId == ETransformGizmoPartIdentifier::RotateArcball)
	{
		// In gimbal mode, we don't allow interaction with the arc ball rotation
		bCanInteractWithPart = false;
	}

	return bCanInteractWithPart;
}

ETransformGizmoPartIdentifier UEditorTRSGizmo::GetHitPartInternal(const FInputDeviceRay& InDeviceRay, FInputRayHit& OutRayHit) const
{
	ETransformGizmoPartIdentifier HitPart = ETransformGizmoPartIdentifier::Default;
	if (IsVisible(EViewportContext::Hovered) && CanInteract() && HitTarget)
	{
		OutRayHit = HitTarget->IsHit(InDeviceRay);
		if (OutRayHit.bHit && VerifyPartIdentifier(OutRayHit.HitIdentifier))
		{
			// Update Debug
			if (GizmoLocals::DoDebugDraw())
			{
				DebugData.LastDeviceRay = InDeviceRay;
				DebugData.LastRayHit = OutRayHit;
			}

			HitPart = static_cast<ETransformGizmoPartIdentifier>(OutRayHit.HitIdentifier);
		}
		else
		{
			HitPart = ETransformGizmoPartIdentifier::Default;
		}
	}

	return HitPart;
}

void UEditorTRSGizmo::InitializeInteractionPlane(const EAxisList::Type InAxisList)
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;

	// Interaction Plane
	const FVector PlaneOrigin = CurrentTransform.GetLocation();

	FVector PlaneNormal;
	FVector PlaneAxisX; // Up
	FVector PlaneAxisY; // Side

	if (InAxisList == EAxisList::Screen)
	{
		InteractionPlane = MakeScreenAlignedPlane(PlaneOrigin, GizmoViewContext);
	}
	else
	{
		const EAxisList::Type AxisListForAxisBasis =
			IsAxisPlanar(InAxisList)
			? GetPlaneNormalAxis(InAxisList)
			: InAxisList;

		GetAxisBasis(AxisListForAxisBasis, PlaneNormal, PlaneAxisY, PlaneAxisX);

		PlaneNormal = GetWorldAxis(PlaneNormal);
		PlaneAxisX = GetWorldAxis(PlaneAxisX);
		PlaneAxisY = GetWorldAxis(PlaneAxisY);

		InteractionPlane = UE::InteractiveToolsFramework::MakePlaneFrame(PlaneOrigin, PlaneNormal, PlaneAxisX, PlaneAxisY);
	}

	InitializeInteractionPlane(InteractionPlane);
}

void UEditorTRSGizmo::InitializeInteractionPlane(const UE::Geometry::FFrame3d& InFrame)
{
	InteractionPlane = InFrame;
	UE::InteractiveToolsFramework::BreakPlaneFrame(
		InFrame,
		InteractionPlanarOrigin,
		InteractionPlanarNormal,
		InteractionPlanarAxisX,
		InteractionPlanarAxisY);
}

void UEditorTRSGizmo::InitializeInteractionPlanes(const UE::Geometry::FFrame3d& InFrame)
{
	InitializeInteractionPlane(InFrame);

	AlignedInteractionPlane = InteractionPlane;
	AlignedInteractionPlanarNormal = InteractionPlanarNormal;
}

void UEditorTRSGizmo::InitializeScreenInteraction(const FInputDeviceRay& InDeviceRay)
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;

	// Store view plane rotation
	ScreenPlane = MakeScreenAlignedPlane(CurrentTransform.GetLocation(), GizmoViewContext);
}

void UEditorTRSGizmo::InitializeInteractionSign()
{
	// Only calculate if the direction-to-camera and aligned plane normal are approximately inverse
	if (FVector::DotProduct(-GizmoViewContext->GetViewDirection(), AlignedInteractionPlanarNormal) < 0.0)
	{
		ViewToAlignedSign.X = static_cast<int8>( UE::Editor::GizmoMath::ComputePositivePlaneSignForView(
			GizmoViewContext,
			AlignedInteractionPlane,
			static_cast<float>(ViewToAlignedSign.X)));
	}
	else // Otherwise reset
	{
		ViewToAlignedSign = UE::Math::TIntVector3<int8>(1);
	}

	UE_LOGF(LogEditorTRSGizmo, Verbose,
		"InteractionToAlignedSign: %ls",
		*ViewToAlignedSign.ToString());
}

void UEditorTRSGizmo::ApplyStyle()
{
	if (DirectionalCursorWidget.IsValid())
	{
		float CursorSize = Style.CursorSize;
		if (Style.bUsePlatformCursorSize)
		{
			CursorSize = FSlateApplication::Get().GetCursorSize().GetMax();
		}

		DirectionalCursorWidget->SetSize(FVector2f(CursorSize, CursorSize));
	}

	UpdateElements();
}

EAxis::Type UEditorTRSGizmo::GetAxisForIndex(const int32 InIndex) const
{
	if (!ensure(InIndex != INDEX_NONE && InIndex < 3))
	{
		return EAxis::None;
	}

	static TArray<EAxis::Type> Axis = { EAxis::X, EAxis::Y, EAxis::Z };
	return Axis[InIndex];
}

EAxis::Type UEditorTRSGizmo::GetAxisForPart(const ETransformGizmoPartIdentifier InPartId) const
{
	int32 AxisIndex = INDEX_NONE;

	if (InPartId >= ETransformGizmoPartIdentifier::TranslateAll && InPartId <= ETransformGizmoPartIdentifier::TranslateScreenSpace)
	{
		AxisIndex = GetTranslateAxisIndexForPart(InPartId);
	}
	else if (InPartId >= ETransformGizmoPartIdentifier::RotateAll && InPartId <= ETransformGizmoPartIdentifier::RotateZGimbal)
	{
		AxisIndex = GetRotateAxisIndexForPart(InPartId);
	}
	else if (InPartId >= ETransformGizmoPartIdentifier::ScaleAll && InPartId <= ETransformGizmoPartIdentifier::ScaleUniform)
	{
		AxisIndex = GetScaleAxisIndexForPart(InPartId);
	}

	if (!ensure(AxisIndex != INDEX_NONE))
	{
		return EAxis::None;
	}

	return GetAxisForIndex(AxisIndex);
}

EAxisList::Type UEditorTRSGizmo::GetAxisListForPart(const ETransformGizmoPartIdentifier InPartId) const
{
	const EAxis::Type Axis = GetAxisForPart(InPartId);
	if (!ensure(Axis != EAxis::None))
	{
		return EAxisList::None;
	}

	return EAxisList::FromAxis(Axis, AxisDisplayInfo::GetAxisDisplayCoordinateSystem());
}

ETransformGizmoPartIdentifier UEditorTRSGizmo::GetPartForAxisList(const EAxisList::Type InAxisList) const
{
	switch (CurrentMode)
	{
	case EGizmoTransformMode::Translate:
		switch (InAxisList)
		{
		case EAxisList::X:
			return ETransformGizmoPartIdentifier::TranslateXAxis;
		case EAxisList::Y:
			return ETransformGizmoPartIdentifier::TranslateYAxis;
		case EAxisList::Z:
			return ETransformGizmoPartIdentifier::TranslateZAxis;
			
		case EAxisList::XY:
			return ETransformGizmoPartIdentifier::TranslateXYPlanar;
		case EAxisList::XZ:
			return ETransformGizmoPartIdentifier::TranslateXZPlanar;
		case EAxisList::YZ:
			return ETransformGizmoPartIdentifier::TranslateYZPlanar;
			
		case EAxisList::XYZ:
		case EAxisList::Screen:
			return ETransformGizmoPartIdentifier::TranslateScreenSpace;
			
		default: break;
		}
		break;
	case EGizmoTransformMode::Rotate:
		switch (InAxisList)
		{
		case EAxisList::X:
			return bGimbalRotationMode ? ETransformGizmoPartIdentifier::RotateXGimbal : ETransformGizmoPartIdentifier::RotateXAxis;
		case EAxisList::Y:
			return bGimbalRotationMode ? ETransformGizmoPartIdentifier::RotateYGimbal : ETransformGizmoPartIdentifier::RotateYAxis;
		case EAxisList::Z:
			return bGimbalRotationMode ? ETransformGizmoPartIdentifier::RotateZGimbal : ETransformGizmoPartIdentifier::RotateZAxis;
		
		// It does not make much sense (yet) to do two axis rotation. Fall back to everything
		case EAxisList::XY:
		case EAxisList::XZ:
		case EAxisList::YZ:
		case EAxisList::XYZ:
			return ETransformGizmoPartIdentifier::RotateArcball;
		
		case EAxisList::Screen:
			return ETransformGizmoPartIdentifier::RotateScreenSpace;
			
		default: break;
		}
		break;
	case EGizmoTransformMode::Scale:
		switch (InAxisList)
		{
		case EAxisList::X:
			return ETransformGizmoPartIdentifier::ScaleXAxis;
		case EAxisList::Y:
			return ETransformGizmoPartIdentifier::ScaleYAxis;
		case EAxisList::Z:
			return ETransformGizmoPartIdentifier::ScaleZAxis;
			
		case EAxisList::XY:
			return ETransformGizmoPartIdentifier::ScaleXYPlanar;
		case EAxisList::XZ:
			return ETransformGizmoPartIdentifier::ScaleXZPlanar;
		case EAxisList::YZ:
			return ETransformGizmoPartIdentifier::ScaleYZPlanar;
			
		case EAxisList::XYZ:
		case EAxisList::Screen:
			return ETransformGizmoPartIdentifier::ScaleUniform;
			
		default: break;
		}
		break;
		
	default: break;
	}
	
	return ETransformGizmoPartIdentifier::Default;
}

void UEditorTRSGizmo::InitializeScreenSpaceTranslate(const FInputDeviceRay& InPressPos)
{
	InteractionScreenStartPos = InteractionScreenCurrPos = InPressPos.ScreenPosition;
	StartRotation = CurrentRotation = GetActiveTransform().GetRotation();
	bInInteraction = true;
	SetModeLastHitPart(EGizmoTransformMode::Translate, LastHitPart);
}

void UEditorTRSGizmo::ResetDragTranslate(const FInputDeviceRay& DragPos)
{
	if (CameraFollowsMovement())
	{
		// In case camera should follow movement, re-init might be required
		InitializeScreenSpaceTranslate(DragPos);
		bIndirectManipulation = true;
	}
	else
	{
		using namespace UE::Editor::InteractiveToolsFramework::Internal;

		if (IsAxisPlanar(GetInteractionAxisList()) || LastHitPart == ETransformGizmoPartIdentifier::TranslateScreenSpace)
		{
			// Planar Translate logic (used by ScreenSpace translate as well)
			OnClickReleaseTranslatePlanar(DragPos);
			OnClickPressTranslatePlanar(DragPos);
		}
		else if (IsAxisSingular(GetInteractionAxisList()))
		{
			// Axis Translate logic
			OnClickReleaseTranslateAxis(DragPos);
			OnClickPressAxis(DragPos);
			OnDragStartTranslateAxis(DragPos);
		}
	}
}

bool UEditorTRSGizmo::CameraFollowsMovement() const
{
	return bCameraFollowsMovement && bInInteraction && CurrentMode == EGizmoTransformMode::Translate;
}

void UEditorTRSGizmo::DuplicateSelection(bool bDroppingDuplicate) const
{
	IAssetEditorContextInterface* AssetEditorContext = nullptr;
	UWorld* World = nullptr;

	if (const UInteractiveGizmoManager* const GizmoManager = GetGizmoManager())
	{
		if (const UContextObjectStore* const ContextObjectStore = GizmoManager->GetContextObjectStore())
		{
			AssetEditorContext = ContextObjectStore->FindContext<IAssetEditorContextInterface>();
			World = ContextObjectStore->GetWorld();
		}
	}

	if (!AssetEditorContext || !World)
	{
		return;
	}

	FEditorModeTools* ModeTools = nullptr;
	if (const IToolkitHost* const ToolkitHost = AssetEditorContext->GetToolkitHost())
	{
		ModeTools = &ToolkitHost->GetEditorModeManager();
	}

	if (!ModeTools)
	{
		return;
	}

	UTypedElementCommonActions* CommonActions = AssetEditorContext->GetCommonActions();
	UTypedElementSelectionSet* ModeToolsSelectionSet = AssetEditorContext->GetMutableSelectionSet();

	if (!CommonActions || !CommonActions->CanDuplicateSelectedElements(ModeToolsSelectionSet))
	{
		return;
	}

	const bool bIsGlobalModeTools = &GLevelEditorModeTools() == ModeTools;
	// "Active" Component visualizers process inputs, so they might want to handle Alt + Drag differently (e.g. spline points editing)
	if ((bIsGlobalModeTools && GUnrealEd->ComponentVisManager.IsActive()) || ModeTools->GetActionDragDuplicate() != EEditAction::Skip)
	{
		return;
	}

	StateTarget->EndUpdate();

	// Setting up a dedicated transaction - see UEditorGizmoStateTarget::BeginUpdate()
	if (bDroppingDuplicate)
	{
		GetGizmoManager()->BeginUndoTransaction(LOCTEXT("DropDuplicateTransactionName", "Drop Duplicate"));
	}
	else
	{
		GetGizmoManager()->BeginUndoTransaction(LOCTEXT("DuplicateAndTransformTransactionName", "Duplicate and Transform"));
	}

	(void)ModeTools->BeginTransform(FGizmoState());

	// Do not use the cached manipulation list here, as it will have removed attachments, and we do want to duplicate those
	const TArray<FTypedElementHandle> DuplicatedElements = CommonActions->DuplicateSelectedElements(
		ModeToolsSelectionSet, World, FVector::ZeroVector);

	if (!DuplicatedElements.IsEmpty())
	{
		// (assuming chosen modifier key is Alt)
		// Alt Duplicate has 2 modes:
		// 1 - with Alt pressed, dragging will create and select a copy, then drag it
		// 2 - while dragging, hitting Alt will create a copy, NOT selecting it

		// Select newly created elements: Alt + Drag only
		if (!bDroppingDuplicate)
		{
			// Select the newly created elements
			const FTypedElementSelectionOptions SelectionOptions =
				FTypedElementSelectionOptions()
					.SetAllowLegacyNotifications(false) // Old drag duplicate code didn't use to notify about this selection change
					.SetNameForTEDSIntegration(ModeToolsSelectionSet->GetNameForTedsIntegration());

			ModeToolsSelectionSet->SetSelection(DuplicatedElements, SelectionOptions);
			ModeToolsSelectionSet->NotifyPendingChanges();
		}

		if (bIsGlobalModeTools)
		{
			// Notify the global mode tools, the selection set should be identical to the new actors at this point
			TArray<AActor*> SelectedActors = ModeToolsSelectionSet->GetSelectedObjects<AActor>();
			constexpr bool bDidOffsetDuplicate = false;
			ModeTools->ActorsDuplicatedNotify(SelectedActors, SelectedActors, bDidOffsetDuplicate);
		}
	}
}

bool UEditorTRSGizmo::AllowsDragDuplicate(EDragDuplicateContext Context) const
{
	const bool bDuplicateInCurrentMode = CurrentMode == EGizmoTransformMode::Translate || (CurrentMode == EGizmoTransformMode::Rotate && GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.bDragDuplicateOnRotation);
	if (!bDuplicateInCurrentMode)
	{
		return false;
	}
	
	if (CurrentMode == EGizmoTransformMode::Scale)
	{
		return false;
	}
	
	if (CurrentMode == EGizmoTransformMode::Rotate && !GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.bDragDuplicateOnRotation)
	{
		return false;
	}

	if (bOnlyUpdatePivot)
	{
		return false;
	}
	
	if (ActiveTarget && ActiveTarget->bSetPivotMode)
	{
		return false;
	}

	// Mirror the checks in `DuplicateSelection`. If the duplication would be skipped, don't allow it.
	IAssetEditorContextInterface* AssetEditorContext = nullptr;

	if (const UInteractiveGizmoManager* const GizmoManager = GetGizmoManager())
	{
		if (const UContextObjectStore* const ContextObjectStore = GizmoManager->GetContextObjectStore())
		{
			AssetEditorContext = ContextObjectStore->FindContext<IAssetEditorContextInterface>();
		}
	}

	if (!AssetEditorContext)
	{
		return false;
	}
	
	UTypedElementCommonActions* CommonActions = AssetEditorContext->GetCommonActions();
	UTypedElementSelectionSet* ModeToolsSelectionSet = AssetEditorContext->GetMutableSelectionSet();

	if (!CommonActions || !CommonActions->CanDuplicateSelectedElements(ModeToolsSelectionSet))
	{
		return false;
	}

	FEditorModeTools* ModeTools = nullptr;
	if (const IToolkitHost* const ToolkitHost = AssetEditorContext->GetToolkitHost())
	{
		ModeTools = &ToolkitHost->GetEditorModeManager();
	}

	if (!ModeTools)
	{
		return false;
	}
	
	// "Active" Component visualizers process inputs, so they might want to handle Alt + Drag differently (e.g. spline points editing)
	const bool bIsGlobalModeTools = &GLevelEditorModeTools() == ModeTools;
	if (bIsGlobalModeTools && GUnrealEd->ComponentVisManager.IsActive())
	{
		// The gizmo can capture this action but cannot execute it
		return Context == EDragDuplicateContext::Capture;
	}
	
	switch (ModeTools->GetActionDragDuplicate())
	{
	case EEditAction::Skip:
		return true;
	case EEditAction::Process:
		// Capture the action, but do not execute it
		return Context == EDragDuplicateContext::Capture;
	case EEditAction::Halt:
		return false;
	}
	
	return true;
}

bool UEditorTRSGizmo::IsRotationPrecisionMode() const
{
	return bCtrlKeyDown && bInInteraction && !bIndirectManipulation && CurrentMode == EGizmoTransformMode::Rotate;
}

double UEditorTRSGizmo::GetRotationPrecisionModeMultiplier() const
{
	double Multiplier = 1.0;

	if (IsRotationPrecisionMode())
	{
		if (LastHitPart == ETransformGizmoPartIdentifier::RotateScreenSpace)
		{
			Multiplier = bShiftKeyDown ? Interaction.RotateInteraction.ScreenSpaceRotationPrecisionBoost : Interaction.RotateInteraction.ScreenSpaceRotationPrecisionDamping;
		}
		else if (LastHitPart == ETransformGizmoPartIdentifier::RotateArcball)
		{
			Multiplier = bShiftKeyDown ? Interaction.RotateInteraction.ArcballRotationPrecisionBoost : Interaction.RotateInteraction.ArcballRotationPrecisionDamping;

			if (FMath::IsNearlyZero(Multiplier))
			{
				Multiplier = 1.0;
			}
			else
			{
				Multiplier = 1.0/Multiplier;
			}
		}
		else if (UE::Editor::InteractiveToolsFramework::Internal::IsAxisSingular(GetInteractionAxisList()))
		{
			Multiplier = bShiftKeyDown ? Interaction.RotateInteraction.AxisRotationPrecisionBoost : Interaction.RotateInteraction.AxisRotationPrecisionDamping;
		}
	}

	return Multiplier;
}

FEditorViewportClient* UEditorTRSGizmo::GetEditorViewportClient() const
{
	if (const UInteractiveGizmoManager* const GizmoManager = GetGizmoManager())
	{
		if (const UContextObjectStore* const ContextObjectStore = GizmoManager->GetContextObjectStore())
		{
			if (const UEditorInteractiveToolsContext* const InteractiveToolsContext = ContextObjectStore->GetTypedOuter<UEditorInteractiveToolsContext>())
			{
				if (const FEditorModeTools* const ModeManager = InteractiveToolsContext->GetParentEditorModeManager())
				{
					return ModeManager->GetFocusedViewportClient();
				}
			}
		}
	}
	return nullptr;
}

int32 UEditorTRSGizmo::GetTranslateAxisIndexForPart(const ETransformGizmoPartIdentifier InPartId)
{
	static const TArray TranslateIDs({
		ETransformGizmoPartIdentifier::TranslateXAxis,
		ETransformGizmoPartIdentifier::TranslateYAxis,
		ETransformGizmoPartIdentifier::TranslateZAxis
	});

	return TranslateIDs.IndexOfByKey(InPartId);
}

void UEditorTRSGizmo::BeginTranslateDelta() const
{
	if (!TranslateGroupElement)
	{
		return;
	}

	UGizmoElementTranslateGroup::FDeltaParameters DeltaParameters;
	DeltaParameters.Transform = CurrentTransform;
	DeltaParameters.TransformLocation2D = InteractionScreenObjectPos2D;
	DeltaParameters.AxisList = GetInteractionAxisList() == EAxisList::Screen ? EAxisList::XYZ : GetInteractionAxisList();
	DeltaParameters.bIsIndirectInteraction = bIndirectManipulation;
	DeltaParameters.PlaneNormal = InteractionPlanarNormal;
	DeltaParameters.CoordinateSystem = GetCoordinateSystem();

	TranslateGroupElement->BeginDelta(DeltaParameters);
}

void UEditorTRSGizmo::UpdateTranslateDelta() const
{
	if (!TranslateGroupElement)
	{
		return;
	}

	UGizmoElementTranslateGroup::FDeltaParameters DeltaParameters;
	DeltaParameters.Transform = GetActiveTransform();
	DeltaParameters.TransformLocation2D = InteractionScreenCurrPos;
	DeltaParameters.AxisList = GetInteractionAxisList() == EAxisList::Screen ? EAxisList::XYZ : GetInteractionAxisList();
	DeltaParameters.bIsIndirectInteraction = bIndirectManipulation;
	DeltaParameters.PlaneNormal = InteractionPlanarNormal;
	DeltaParameters.CoordinateSystem = GetCoordinateSystem();

	TranslateGroupElement->UpdateDelta(DeltaParameters);
}

void UEditorTRSGizmo::SetRotateMode(const TEnumAsByte<EAxisRotateMode::Type> InRotateMode)
{
	RotateMode = InRotateMode;

	// Update Debug
	if (GizmoLocals::DoDebugDraw())
	{
		FString RotateModeString;

		switch (RotateMode)
		{
		case EAxisRotateMode::Pull:
			RotateModeString = TEXT("Pull");
			break;

		case EAxisRotateMode::Arc:
			RotateModeString = TEXT("Arc");
			break;

		case EAxisRotateMode::ScreenArc:
			RotateModeString = TEXT("ScreenArc");
			break;

		default:
			RotateModeString = TEXT("(Invalid)");
			break;
		}

		DebugData.DebugString = TEXT("Rotate Mode: ") + RotateModeString;
	}
}

template <typename MakePullQuatFunc, typename MakeArcQuatFunc>
void UEditorTRSGizmo::OnClickDragRotateAxisInternal(
	const FInputDeviceRay& InDragPos,
	UGizmoElementRotateAxis* InElement,
	MakePullQuatFunc&& InMakePullQuat,
	MakeArcQuatFunc&& InMakeArcQuat)
{
	using namespace UE::Geometry;
	using namespace UE::Editor::GizmoMath;

	// Precision mode switches to pull
	if (IsRotationPrecisionMode() && RotateMode != EAxisRotateMode::Pull)
	{
		SetRotateMode(EAxisRotateMode::Pull);
	}

	if (bRotationPrecisionModeDirty)
	{
		if (!IsRotationPrecisionMode() && !FMath::IsNearlyZero(InteractionCurrAngle))
		{
			InteractionScreenCurrPos = InDragPos.ScreenPosition;
		}

		bRotationPrecisionModeDirty = false;
	}

	// Always perform ray/plane intersection
	bool bHitPlane = false;
	FVector HitPoint = FVector::ZeroVector;
	if (ComputePointOnPlaneFromScreen(GizmoViewContext, InDragPos.WorldRay, InDragPos.ScreenPosition, InteractionPlane, HitPoint))
	{
		bHitPlane = true;
	}

	const double PreviousAngle = InteractionCurrAngle;
	const FVector2D PreviousScreenPos = InteractionScreenCurrPos;

	// For single axis, we just store the sign in X - no need to get the corresponding element. Ignore sign for Arc mode, where no screen->world conversion is needed.
	double AngleSign = RotateMode == EAxisRotateMode::Arc
		? 1.0
		: ViewToAlignedSign.X;

	switch (RotateMode)
	{
	case EAxisRotateMode::Pull:
	{
		double Angle = PreviousAngle;

		double DeltaAngle = FMath::DegreesToRadians(ComputeAxisRotateDeltaAngle(PreviousScreenPos, InDragPos));

		// Subtract to counter Cartesian -> UE
		Angle -= DeltaAngle;

		// Correct to avoid wrapping etc.
		Angle = PreviousAngle + FMath::FindDeltaAngleRadians(FMath::UnwindRadians(PreviousAngle), Angle);

		DeltaAngle = FMath::FindDeltaAngleRadians(PreviousAngle, Angle);

		if (IsRotationPrecisionMode())
		{
			DeltaAngle *= GetRotationPrecisionModeMultiplier();
		}

		SnapRotateAngleDelta(DeltaAngle, GetInteractionAxisList());

		// The delta as it applies, corrected for the XY vs. Z discrepancy
		const double AppliedDeltaAngle = DeltaAngle * ClockwiseCorrectionSignForAxis;

		// Reapply the (potentially) snapped delta angle
		Angle = PreviousAngle + DeltaAngle;

		const FQuat Delta = std::forward<MakePullQuatFunc>(InMakePullQuat)(AppliedDeltaAngle, AngleSign, PreviousScreenPos, InDragPos.ScreenPosition);
		ApplyRotateDelta(Delta);

		CumulativeRotationDelta = Delta * CumulativeRotationDelta;

		InteractionDelta.X += (DeltaAngle * AngleSign);
		InteractionCurrAngle = Angle;
		InteractionScreenCurrPos = FMath::IsNearlyZero(DeltaAngle) ? PreviousScreenPos : InDragPos.ScreenPosition;

		// Update Debug
		if (GizmoLocals::DoDebugDraw())
		{
			DebugData.InteractionCurrent.SetLocation(HitPoint);

			const double HitAngle = InteractionCurrAngle + DeltaAngle;
			DebugData.InteractionAngleCurrent += FMath::FindDeltaAngleRadians(DebugData.InteractionAngleCurrent, HitAngle);
			DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
			DebugData.InteractionPlaneNormal = Delta.GetRotationAxis();
		}

		break;
	}
	case EAxisRotateMode::Arc:
	case EAxisRotateMode::ScreenArc:
	{
		if (bHitPlane)
		{
			double HitAngle = 0.0;
			bHitPlane = UE::Editor::GizmoMath::ComputeAngleInPlaneFromScreen(GizmoViewContext, InDragPos.ScreenPosition, InteractionPlane, HitPoint, HitAngle);

			InteractionScreenCurrPos = InDragPos.ScreenPosition;

			const FVector2D HitCoord = GizmoMath::ComputeCoordinatesInPlane(
				HitPoint,
				InteractionPlanarOrigin,
				InteractionPlanarNormal,
		InteractionPlanarAxisX,
		InteractionPlanarAxisY);

			InteractionPlanarCurrPoint2D = HitCoord;

			// Correct to avoid wrapping etc.
			HitAngle = PreviousAngle + FMath::FindDeltaAngleRadians(FMath::UnwindRadians(PreviousAngle), HitAngle);

			double HitAngleDelta = FMath::FindDeltaAngleRadians(PreviousAngle, HitAngle);
			SnapRotateAngleDelta(HitAngleDelta, GetInteractionAxisList());

			// Reapply the (potentially) snapped delta angle
			HitAngle = PreviousAngle + HitAngleDelta;

			const FQuat Delta = std::forward<MakeArcQuatFunc>(InMakeArcQuat)(HitAngleDelta, AngleSign);
			ApplyRotateDelta(Delta);

			CumulativeRotationDelta = CumulativeRotationDelta * Delta;

			InteractionDelta.X += (HitAngleDelta * AngleSign);
			InteractionCurrAngle = HitAngle;

			// Update Debug
			if (GizmoLocals::DoDebugDraw())
			{
				DebugData.InteractionCurrent.SetLocation(HitPoint);
				DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
				DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
			}
		}
		break;
	}
	default:
		ensure(false);
		break;
	}

	if (GizmoLocals::DoDebugDraw())
	{
		UE::Editor::GizmoMath::ComputeAngleInPlaneFromScreen(
			GizmoViewContext,
			InteractionScreenCurrPos,
			AlignedInteractionPlane,
			DebugData.Test,
			DebugData.InteractionAngleCurrent);

		UE::Editor::GizmoMath::ComputePointOnCircleFromScreen(
			GizmoViewContext,
			InteractionScreenCurrPos,
			1.0f,
			AlignedInteractionPlane,
			DebugData.PointOnCircleDirection);

		DebugData.PointOnCircleDirection -= AlignedInteractionPlane.Origin;
		DebugData.PointOnCircleDirection.Normalize();
	}

	// Update element deltas
	if (ensure(InElement))
	{
		// Re-calculate based on screen plane to avoid plane intersection failures
		if (ComputePointOnPlaneFromScreen(GizmoViewContext, InDragPos.WorldRay, InDragPos.ScreenPosition, ScreenPlane, HitPoint))
		{
			InteractionPlanarCurrPoint = HitPoint;
		}

		UpdateRotateAxisDelta(InElement);
	}
}

bool UEditorTRSGizmo::GetRotateAxisVectorForPart(const ETransformGizmoPartIdentifier InPartId, FVector& OutAxisVector) const
{
	const int32 RotateID = GetRotateAxisIndexForPart(InPartId);
	if (RotateID == INDEX_NONE)
	{
		OutAxisVector = GizmoViewContext->GetViewDirection();
		return false;
	}
	
	const EAxis::Type PartAxis = GetAxisForPart(InPartId);
	OutAxisVector = UE::Editor::InteractiveToolsFramework::Internal::GetAxisVector(PartAxis);
	OutAxisVector = GetWorldAxis(OutAxisVector);

	return PartAxis != EAxis::None;
}

int32 UEditorTRSGizmo::GetRotateAxisIndexForPart(const ETransformGizmoPartIdentifier InPartId)
{
	// Accounts for both regular and gimbal
	static const TArray RotateIDs({
		ETransformGizmoPartIdentifier::RotateXAxis,
		ETransformGizmoPartIdentifier::RotateYAxis,
		ETransformGizmoPartIdentifier::RotateZAxis,

		ETransformGizmoPartIdentifier::RotateXGimbal,
		ETransformGizmoPartIdentifier::RotateYGimbal,
		ETransformGizmoPartIdentifier::RotateZGimbal
	});

	return RotateIDs.IndexOfByKey(InPartId) % 3;
}

void UEditorTRSGizmo::BeginRotateAxisDelta(UGizmoElementRotateAxis* InElement) const
{
	if (!InElement)
	{
		return;
	}

	UGizmoElementRotateAxis::FDeltaParameters DeltaParameters;
	DeltaParameters.bIsIndirectManipulation = bIndirectManipulation;
	DeltaParameters.Transform = CurrentTransform;
	DeltaParameters.RotationContext = GetRotationContext();
	DeltaParameters.Angle = InteractionStartAngle - AlignedInteractionAngleOffset;
	DeltaParameters.DisplaySign = ClockwiseCorrectionSignForAxis;
	DeltaParameters.PlaneNormal = InteractionPlanarNormal;
	DeltaParameters.CoordinateSystem = GetCoordinateSystem();
	DeltaParameters.CursorLocation = InteractionPlanarStartPoint;
	DeltaParameters.CursorLocation2D = InteractionPlanarStartPoint2D;
	DeltaParameters.RotateMode = RotateMode;

	InElement->BeginDelta(DeltaParameters);
}

void UEditorTRSGizmo::UpdateRotateAxisDelta(UGizmoElementRotateAxis* InElement) const
{
	if (!InElement)
	{
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;

	UGizmoElementRotateAxis::FDeltaParameters DeltaParameters;
	DeltaParameters.bIsIndirectManipulation = bIndirectManipulation;
	DeltaParameters.Transform = GetActiveTransform();
	DeltaParameters.Angle = (InteractionStartAngle + InteractionDelta.X) - AlignedInteractionAngleOffset;
	DeltaParameters.CursorLocation = InteractionPlanarCurrPoint;
	DeltaParameters.CursorLocation2D = InteractionPlanarCurrPoint2D;
	DeltaParameters.RotateMode = RotateMode;

	UE_CLOGF(bLogRotation, LogEditorTRSGizmo, Verbose, "InteractionDelta: %.1f", FMath::RadiansToDegrees(InteractionDelta.X));

	InElement->UpdateDelta(DeltaParameters);
}

int32 UEditorTRSGizmo::GetScaleAxisIndexForPart(const ETransformGizmoPartIdentifier InPartId)
{
	static const TArray ScaleIDs({
		ETransformGizmoPartIdentifier::ScaleXAxis,
		ETransformGizmoPartIdentifier::ScaleYAxis,
		ETransformGizmoPartIdentifier::ScaleZAxis
	});

	return ScaleIDs.IndexOfByKey(InPartId);
}

double UEditorTRSGizmo::GetScreenSign(const FVector2D& InScreenPosition)
{
	return GetScreenSign(InScreenPosition, InteractionScreenObjectPos2D);
}

double UEditorTRSGizmo::GetScreenSign(const FVector2D& InScreenPosition, const FVector2D& InSignAxisOrigin)
{
	return FMath::Sign(-FVector2D::DotProduct((InSignAxisOrigin - InScreenPosition).GetSafeNormal(), InteractionScreenAxisDirection));
}

void UEditorTRSGizmo::BeginScaleDelta() const
{
	if (!ScaleGroupElement)
	{
		return;
	}

	UGizmoElementScaleGroup::FDeltaParameters DeltaParameters;
	DeltaParameters.Transform = CurrentTransform;
	DeltaParameters.TransformLocation2D = InteractionScreenObjectPos2D;
	DeltaParameters.DeltaScale = InteractionDelta;
	DeltaParameters.AxisList = GetInteractionAxisList() == EAxisList::Screen ? EAxisList::XYZ : GetInteractionAxisList();
	DeltaParameters.bIsIndirectInteraction = bIndirectManipulation;
	DeltaParameters.PlaneNormal = InteractionPlanarNormal;
	DeltaParameters.CoordinateSystem = GetCoordinateSystem();
	DeltaParameters.ScaleType = GetScaleType();
	DeltaParameters.PlaneIntersectionPoint = InteractionPlanarCurrPoint;

	ScaleGroupElement->BeginDelta(DeltaParameters);
}

void UEditorTRSGizmo::UpdateScaleDelta() const
{
	UpdateScaleDelta(InteractionDelta * InteractionDeltaDivisor);
}

void UEditorTRSGizmo::UpdateScaleDelta(const FVector& InDeltaScale) const
{
	if (!ScaleGroupElement)
	{
		return;
	}

	UGizmoElementScaleGroup::FDeltaParameters DeltaParameters;
	DeltaParameters.Transform = GetActiveTransform();
	DeltaParameters.TransformLocation2D = InteractionScreenObjectPos2D;
	DeltaParameters.DeltaScale = InDeltaScale;
	DeltaParameters.AxisList = GetInteractionAxisList() == EAxisList::Screen ? EAxisList::XYZ : GetInteractionAxisList();
	DeltaParameters.bIsIndirectInteraction = bIndirectManipulation;
	DeltaParameters.PlaneNormal = InteractionPlanarNormal;
	DeltaParameters.CoordinateSystem = GetCoordinateSystem();
	DeltaParameters.ScaleType = GetScaleType();
	DeltaParameters.PlaneIntersectionPoint = InteractionPlanarCurrPoint;
	DeltaParameters.bIsTrustworthy = !bDetectedNonStandardScaling;

	ScaleGroupElement->UpdateDelta(DeltaParameters);
}

void UEditorTRSGizmo::SetupOnClickFunctions()
{
	constexpr int32 NumParts = static_cast<int>(ETransformGizmoPartIdentifier::Max);
	ClickDragElements.Init(nullptr, NumParts);
	OnClickPressFunctions.Init(nullptr, NumParts);
	OnDragStartFunctions.Init(nullptr, NumParts);
	OnClickDragFunctions.Init(nullptr, NumParts);
	OnClickReleaseFunctions.Init(nullptr, NumParts);

	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXAxis)] = { &UEditorTRSGizmo::OnClickPressTranslateXAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYAxis)] = { &UEditorTRSGizmo::OnClickPressTranslateYAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateZAxis)] = { &UEditorTRSGizmo::OnClickPressTranslateZAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXYPlanar)] = { &UEditorTRSGizmo::OnClickPressTranslateXYPlanar };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYZPlanar)] = { &UEditorTRSGizmo::OnClickPressTranslateYZPlanar };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXZPlanar)] = { &UEditorTRSGizmo::OnClickPressTranslateXZPlanar };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateScreenSpace)] = { &UEditorTRSGizmo::OnClickPressScreenSpaceTranslate };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXAxis)] = { &UEditorTRSGizmo::OnClickPressScaleXAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYAxis)] = { &UEditorTRSGizmo::OnClickPressScaleYAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleZAxis)] = { &UEditorTRSGizmo::OnClickPressScaleZAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXYPlanar)] = { &UEditorTRSGizmo::OnClickPressScaleXYPlanar };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYZPlanar)] = { &UEditorTRSGizmo::OnClickPressScaleYZPlanar };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXZPlanar)] = { &UEditorTRSGizmo::OnClickPressScaleXZPlanar };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleUniform)] = { &UEditorTRSGizmo::OnClickPressScaleXYZ };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXAxis)] = { &UEditorTRSGizmo::OnClickPressRotateXAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYAxis)] = { &UEditorTRSGizmo::OnClickPressRotateYAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZAxis)] = { &UEditorTRSGizmo::OnClickPressRotateZAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateScreenSpace)] = { &UEditorTRSGizmo::OnClickPressScreenSpaceRotate };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateArcball)] = { &UEditorTRSGizmo::OnClickPressArcBallRotate };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXGimbal)] = { &UEditorTRSGizmo::OnClickPressGimbalRotateAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYGimbal)] = { &UEditorTRSGizmo::OnClickPressGimbalRotateAxis };
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZGimbal)] = { &UEditorTRSGizmo::OnClickPressGimbalRotateAxis };

	OnDragStartFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXAxis)] = { &UEditorTRSGizmo::OnDragStartTranslateAxis };
	OnDragStartFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYAxis)] = { &UEditorTRSGizmo::OnDragStartTranslateAxis };
	OnDragStartFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateZAxis)] = { &UEditorTRSGizmo::OnDragStartTranslateAxis };

	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXAxis)] = { &UEditorTRSGizmo::OnClickDragTranslateAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYAxis)] = { &UEditorTRSGizmo::OnClickDragTranslateAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateZAxis)] = { &UEditorTRSGizmo::OnClickDragTranslateAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXYPlanar)] = { &UEditorTRSGizmo::OnClickDragTranslatePlanar };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYZPlanar)] = { &UEditorTRSGizmo::OnClickDragTranslatePlanar };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXZPlanar)] = { &UEditorTRSGizmo::OnClickDragTranslatePlanar };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateScreenSpace)] = { &UEditorTRSGizmo::OnClickDragScreenSpaceTranslate };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXAxis)] = { &UEditorTRSGizmo::OnClickDragScaleAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYAxis)] = { &UEditorTRSGizmo::OnClickDragScaleAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleZAxis)] = { &UEditorTRSGizmo::OnClickDragScaleAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXYPlanar)] = { &UEditorTRSGizmo::OnClickDragScalePlanar };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYZPlanar)] = { &UEditorTRSGizmo::OnClickDragScalePlanar };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXZPlanar)] = { &UEditorTRSGizmo::OnClickDragScalePlanar };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleUniform)] = { &UEditorTRSGizmo::OnClickDragScaleXYZ };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXAxis)] = { &UEditorTRSGizmo::OnClickDragRotateAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYAxis)] = { &UEditorTRSGizmo::OnClickDragRotateAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZAxis)] = { &UEditorTRSGizmo::OnClickDragRotateAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateScreenSpace)] = { &UEditorTRSGizmo::OnClickDragScreenSpaceRotate };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateArcball)] = { &UEditorTRSGizmo::OnClickDragArcBallRotate };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXGimbal)] = { &UEditorTRSGizmo::OnClickDragGimbalRotateAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYGimbal)] = { &UEditorTRSGizmo::OnClickDragGimbalRotateAxis };
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZGimbal)] = { &UEditorTRSGizmo::OnClickDragGimbalRotateAxis };

	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXAxis)] = { &UEditorTRSGizmo::OnClickReleaseTranslateAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYAxis)] = { &UEditorTRSGizmo::OnClickReleaseTranslateAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateZAxis)] = { &UEditorTRSGizmo::OnClickReleaseTranslateAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXYPlanar)] = { &UEditorTRSGizmo::OnClickReleaseTranslatePlanar };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYZPlanar)] = { &UEditorTRSGizmo::OnClickReleaseTranslatePlanar };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXZPlanar)] = { &UEditorTRSGizmo::OnClickReleaseTranslatePlanar };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateScreenSpace)] = { &UEditorTRSGizmo::OnClickReleaseScreenSpaceTranslate };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXAxis)] = { &UEditorTRSGizmo::OnClickReleaseScaleAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYAxis)] = { &UEditorTRSGizmo::OnClickReleaseScaleAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleZAxis)] = { &UEditorTRSGizmo::OnClickReleaseScaleAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXYPlanar)] = { &UEditorTRSGizmo::OnClickReleaseScalePlanar };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYZPlanar)] = { &UEditorTRSGizmo::OnClickReleaseScalePlanar };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXZPlanar)] = { &UEditorTRSGizmo::OnClickReleaseScalePlanar };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleUniform)] = { &UEditorTRSGizmo::OnClickReleaseScaleXYZ };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXAxis)] = { &UEditorTRSGizmo::OnClickReleaseRotateAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYAxis)] = { &UEditorTRSGizmo::OnClickReleaseRotateAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZAxis)] = { &UEditorTRSGizmo::OnClickReleaseRotateAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateScreenSpace)] = { &UEditorTRSGizmo::OnClickReleaseScreenSpaceRotate };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateArcball)] = { &UEditorTRSGizmo::OnClickReleaseArcBallRotate };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXGimbal)] = { &UEditorTRSGizmo::OnClickReleaseRotateAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYGimbal)] = { &UEditorTRSGizmo::OnClickReleaseRotateAxis };
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZGimbal)] = { &UEditorTRSGizmo::OnClickReleaseRotateAxis };
}

bool UEditorTRSGizmo::GetRayParamIntersectionWithInteractionPlane(const FInputDeviceRay& InRay, FVector::FReal& OutHitParam)
{
	return GetRayParamIntersectionWithPlane(InRay.WorldRay, InteractionPlanarOrigin, InteractionPlanarNormal, OutHitParam);
}

bool UEditorTRSGizmo::GetRayParamIntersectionWithPlane(const FRay& InRay, const FVector& InPlaneOrigin, const FVector& InPlaneNormal, FVector::FReal& OutHitParam)
{
	// if ray is parallel to plane, nothing has been hit
	if (FMath::IsNearlyZero(FVector::DotProduct(InPlaneNormal, InRay.Direction)))
	{
		return false;
	}

	FPlane Plane(InPlaneOrigin, InPlaneNormal);
	OutHitParam = FMath::RayPlaneIntersectionParam(InRay.Origin, InRay.Direction, Plane);
	if (OutHitParam < 0)
	{
		return false;
	}

	return true;
}

void UEditorTRSGizmo::UpdateHoverState(const bool bInHover, const ETransformGizmoPartIdentifier InHitPartId)
{
	if (GizmoLocals::DoDebugDraw()
		&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
	{
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;
	LogStateChanged("UpdateHoverState", bInHover);

	HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(InHitPartId));
}

void UEditorTRSGizmo::UpdateInteractingState(const bool bInInteracting, const ETransformGizmoPartIdentifier InHitPartId, const bool bIdOnly)
{
	if (!bInInteracting
		&& GizmoLocals::DoDebugDraw()
		&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
	{
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;
	LogStateChanged("UpdateInteractingState", bInInteracting);

	HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(InHitPartId));

	if (!bIdOnly)
	{
		// Hide
		if (FGizmoElementInteraction::FPartSet* PartsToHide = GizmoElementInteraction.HideOnInteractParts.Find(InHitPartId))
		{
			PartsToHide->ForEachPart([&](const ETransformGizmoPartIdentifier& PartId)
			{
				GizmoElementRoot->UpdatePartVisibleState(!bInInteracting, static_cast<uint32>(PartId));
			});
		}

		// Interact
		if (FGizmoElementInteraction::FPartSet* OtherPartsToInteract = GizmoElementInteraction.InteractGroupParts.Find(InHitPartId))
		{
			OtherPartsToInteract->ForEachPart([&](const ETransformGizmoPartIdentifier& PartId)
			{
				UpdateInteractingState(bInInteracting, PartId, true);
			});
		}

		// Subdue
		if (FGizmoElementInteraction::FPartSet* PartsToSubdue = GizmoElementInteraction.SubdueOnInteractParts.Find(InHitPartId))
        {
			PartsToSubdue->ForEachPart([&](const ETransformGizmoPartIdentifier& PartId)
			{
				UpdateSubdueState(bInInteracting, PartId, true);
			});
        }
	}
}

void UEditorTRSGizmo::ResetInteractingStates(const EGizmoTransformMode InMode)
{
	if constexpr (UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::bLogStateChanges)
	{
		UE_LOGF(LogEditorTRSGizmo, Verbose, "ResetInteractingStates");
	}

	ETransformGizmoPartIdentifier IdBegin = ETransformGizmoPartIdentifier::Default;
	ETransformGizmoPartIdentifier IdEnd = ETransformGizmoPartIdentifier::Max;
	bool bIdOnly = true;

	switch (InMode)
	{
	case EGizmoTransformMode::Translate:
		IdBegin = ETransformGizmoPartIdentifier::TranslateAll;
		IdEnd = ETransformGizmoPartIdentifier::RotateAll;
		break;
	case EGizmoTransformMode::Rotate:
		IdBegin = ETransformGizmoPartIdentifier::RotateAll;
		IdEnd = ETransformGizmoPartIdentifier::ScaleAll;
		break;
	case EGizmoTransformMode::Scale:
		IdBegin = ETransformGizmoPartIdentifier::ScaleAll;
		IdEnd = ETransformGizmoPartIdentifier::Max;
		bIdOnly = false;
		break;
	default:
		break;
	}

	static constexpr bool bInInteracting = false;
	for (uint32 Id = static_cast<uint32>(IdBegin); Id < static_cast<uint32>(IdEnd); ++Id)
	{
		UpdateInteractingState(bInInteracting, static_cast<ETransformGizmoPartIdentifier>(Id), bIdOnly);

		// Also, reverse any visibility toggles
		if (FGizmoElementInteraction::FPartSet* PartsToHide = GizmoElementInteraction.HideOnInteractParts.Find(static_cast<ETransformGizmoPartIdentifier>(Id)))
		{
			PartsToHide->ForEachPart([&](const ETransformGizmoPartIdentifier& PartId)
			{
				GizmoElementRoot->UpdatePartVisibleState(!bInInteracting, static_cast<uint32>(PartId));
			});
		}
	}
}

void UEditorTRSGizmo::UpdateSelectedState(const bool bInSelected, const ETransformGizmoPartIdentifier InPartId, const bool bIdOnly)
{
	if (!Interaction.bPersistHandleSelection)
	{
		// Bypass if handle selection persistence is disabled
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;
	LogStateChanged("UpdateSelectedState", bInSelected);

	HitTarget->UpdateSelectedState(bInSelected, static_cast<uint32>(InPartId));
}

void UEditorTRSGizmo::ResetSelectedStates(const EGizmoTransformMode InMode)
{
	if constexpr (UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::bLogStateChanges)
	{
		UE_LOGF(LogEditorTRSGizmo, Verbose, "ResetSelectedStates");
	}

	if (GizmoLocals::DoDebugDraw()
		&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
	{
		return;
	}

	ETransformGizmoPartIdentifier IdBegin = ETransformGizmoPartIdentifier::Default;
	ETransformGizmoPartIdentifier IdEnd = ETransformGizmoPartIdentifier::Max;
	bool bIdOnly = true;

	switch (InMode)
	{
	case EGizmoTransformMode::Translate:
		IdBegin = ETransformGizmoPartIdentifier::TranslateAll;
		IdEnd = ETransformGizmoPartIdentifier::RotateAll;
		break;
	case EGizmoTransformMode::Rotate:
		IdBegin = ETransformGizmoPartIdentifier::RotateAll;
		IdEnd = ETransformGizmoPartIdentifier::ScaleAll;
		break;
	case EGizmoTransformMode::Scale:
		IdBegin = ETransformGizmoPartIdentifier::ScaleAll;
		IdEnd = ETransformGizmoPartIdentifier::Max;
		bIdOnly = false;
		break;
	default:
		break;
	}

	constexpr bool bSelected = false;
	for (uint32 Id = static_cast<uint32>(IdBegin); Id < static_cast<uint32>(IdEnd); ++Id)
	{
		UpdateSelectedState(bSelected, static_cast<ETransformGizmoPartIdentifier>(Id), bIdOnly);
	}
}

void UEditorTRSGizmo::UpdateSubdueState(const bool bInSubdued, const ETransformGizmoPartIdentifier InPartId, const bool bIdOnly)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;
	LogStateChanged("UpdateSubdueState", bInSubdued);

	HitTarget->UpdateSubdueState(bInSubdued, static_cast<uint32>(InPartId));
}

void UEditorTRSGizmo::ResetSubdueStates(const EGizmoTransformMode InMode)
{
	if constexpr (UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::bLogStateChanges)
	{
		UE_LOGF(LogEditorTRSGizmo, Verbose, "ResetSubdueStates");
	}

	ETransformGizmoPartIdentifier IdBegin = ETransformGizmoPartIdentifier::Default;
	ETransformGizmoPartIdentifier IdEnd = ETransformGizmoPartIdentifier::Max;
	bool bIdOnly = true;

	switch (InMode)
	{
	case EGizmoTransformMode::Translate:
		IdBegin = ETransformGizmoPartIdentifier::TranslateAll;
		IdEnd = ETransformGizmoPartIdentifier::RotateAll;
		break;
	case EGizmoTransformMode::Rotate:
		IdBegin = ETransformGizmoPartIdentifier::RotateAll;
		IdEnd = ETransformGizmoPartIdentifier::ScaleAll;
		break;
	case EGizmoTransformMode::Scale:
		IdBegin = ETransformGizmoPartIdentifier::ScaleAll;
		IdEnd = ETransformGizmoPartIdentifier::Max;
		bIdOnly = false;
		break;
	default:
		break;
	}

	constexpr bool bSubdued = false;
	for (uint32 Id = static_cast<uint32>(IdBegin); Id < static_cast<uint32>(IdEnd); ++Id)
	{
		UpdateSubdueState(bSubdued, static_cast<ETransformGizmoPartIdentifier>(Id), bIdOnly);
	}
}

void UEditorTRSGizmo::EndDeltas()
{
	if (!GizmoLocals::DoDebugDraw()
		|| !GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
	{
		if (TranslateGroupElement)
		{
			TranslateGroupElement->EndDelta();
		}

		if (RotateAxisSetElement)
		{
			RotateAxisSetElement->GetAxisElement(EAxis::X)->EndDelta();
			RotateAxisSetElement->GetAxisElement(EAxis::Y)->EndDelta();
			RotateAxisSetElement->GetAxisElement(EAxis::Z)->EndDelta();
		}

		if (RotateGimbalElement)
		{
			for (const TObjectPtr<UGizmoElementBase>& Element : RotateGimbalElement->GetSubElements())
			{
				if (UGizmoElementRotateAxis* Axis = Cast<UGizmoElementRotateAxis>(Element))
				{
					Axis->EndDelta();
				}
			}
		}

		if (ScaleGroupElement)
		{
			ScaleGroupElement->EndDelta();
		}
	}
}

void UEditorTRSGizmo::OnClickPress(const FInputDeviceRay& PressPos)
{
	UE_LOGF(LogEditorTRSGizmo, Verbose, "[%lli] OnClickPress", GFrameCounter);

	PendingDragFunction.Reset();

	check(OnClickPressFunctions.Num() == static_cast<int>(ETransformGizmoPartIdentifier::Max));

	const ETransformGizmoPartIdentifier ModeLastHitPart = GetCurrentModeLastHitPart();
	
	if (GizmoLocals::CVarScreenspaceOrtho.GetValueOnAnyThread() && !GizmoViewContext->IsPerspectiveProjection())
	{
		// Force indirect (i.e. screen space) manipulations
		bIndirectManipulation = true;
	}
	
	if (ActiveTarget)
	{
		CurrentTransform = GetActiveTransform();
	}

	if (OnClickPressFunctions[static_cast<int>(LastHitPart)])
	{
		Invoke(OnClickPressFunctions[static_cast<int>(LastHitPart)], this, PressPos);
	}
	
	bDetectedNonStandardScaling = false;
	
	CachedViewLocation = GizmoViewContext->GetViewLocation();

	if (bInInteraction)
	{
		if (HitTarget && LastHitPart != ETransformGizmoPartIdentifier::Default)
		{
			using namespace UE::Editor::InteractiveToolsFramework::Internal;

			ResetSelectedStates(CurrentMode);

			if (LastHitPart != ModeLastHitPart)
			{
				// Clear previously interacting part
				UpdateInteractingState(false, ModeLastHitPart, true);
			}

			UpdateInteractingState(true, LastHitPart);
		}

		BeginTransformEditSequence();

		UpdateCursor(EGizmoElementInteractionState::Interacting, LastHitPart, false);

		// Ensure any new cursor has the correct rotation before displaying
		UpdateCursorRotation(PressPos);
	}

	if (bCtrlKeyDown)
	{
		bRotationPrecisionModeDirty = true;
	}

	CacheSnappingStates();

	CumulativeRotationDelta = FQuat::Identity;
}

void UEditorTRSGizmo::OnClickDrag(const FInputDeviceRay& DragPos)
{
	// Early-out if there's no interaction - ie. arcball in gimbal rotation mode
	if (!bInInteraction)
	{
		return;
	}

	HandleSnappingChanges(DragPos);

	// Update Debug
	{
		DebugData.InteractionRay = DragPos.WorldRay;
		DebugData.InteractionScreenPos = DragPos.ScreenPosition;
	}

	UpdateCursorRotation(DragPos);

	int HitPartIndex = static_cast<int>(LastHitPart);
	check(HitPartIndex < OnClickDragFunctions.Num());

	if (OnClickDragFunctions[HitPartIndex])
	{
		if (bDeferDrag)
		{
			// defer drag function on next tick
			PendingDragFunction = [this, DragPos, HitPartIndex]()
			{
				Invoke(OnClickDragFunctions[HitPartIndex], this, DragPos);
			};
		}
		else
		{
			Invoke(OnClickDragFunctions[HitPartIndex], this, DragPos);
		}
	}
}

FInputRayHit UEditorTRSGizmo::CanBeginSingleClickAndDragSequence(const FInputDeviceRay& InPressPos)
{
	FInputRayHit RayHit;
	ETransformGizmoPartIdentifier HitPart = GetHitPartInternal(InPressPos, RayHit);
	
	if (USingleClickAndDragBehavior* ClickAndDragBehavior = ClickAndDragBehaviorWeak.Get())
	{
		if (HitPart == ETransformGizmoPartIdentifier::RotateArcball)
		{
			using namespace UE::Editor::ViewportInteractions;
			ClickAndDragBehavior->SetPendingDragPriority(FInputCapturePriority(DRAG_PRIORITY - FInputCapturePriority::BINDING_COMPLEXITY_PRIORITY_STEP));
		}
		else
		{
			ClickAndDragBehavior->SetPendingDragPriority(TOptional<FInputCapturePriority>());
		}
	}
	
	return RayHit;
}

void UEditorTRSGizmo::OnDragStart(const FInputDeviceRay& InDragPos)
{
	UE_LOGF(LogEditorTRSGizmo, Verbose, "[%lli] OnDragStart", GFrameCounter);

	UpdateCursorRotation(InDragPos);

	const int32 HitPartIndex = static_cast<int32>(LastHitPart);
	check(HitPartIndex < OnClickDragFunctions.Num());

	if (OnDragStartFunctions[HitPartIndex])
	{
		Invoke(OnDragStartFunctions[HitPartIndex], this, InDragPos);
	}
}

void UEditorTRSGizmo::OnClickRelease(const FInputDeviceRay& InReleasePos, bool bInIsDragOperation)
{
	// If leaving a drag in which camera used to follow movement, reset mouse cursor position
	if (CameraFollowsMovement() || IsRotationPrecisionMode())
	{
		RestoreCursorPosition();
	}

	const int HitPartIndex = static_cast<int>(LastHitPart);
	check(HitPartIndex < OnClickReleaseFunctions.Num());

	if (OnClickReleaseFunctions[HitPartIndex])
	{
		Invoke(OnClickReleaseFunctions[HitPartIndex], this, InReleasePos);
	}

	EndTransformEditSequence();

	bIndirectManipulation = false;
	bInInteraction = false;
	bOnlyUpdatePivot = false;

	UpdateCursor(EGizmoElementInteractionState::Interacting, LastHitPart, true);

	if (HitTarget && LastHitPart != ETransformGizmoPartIdentifier::Default)
	{
		ResetSelectedStates(CurrentMode);

		UpdateSelectedState(true, LastHitPart);
		UpdateInteractingState(false, LastHitPart);

		UpdateSelectedState(true, GetCurrentModeLastHitPart(), true);
	}

	PendingDragFunction.Reset();
	EndDeltas();
	
	if (OnPostInteractionDelegate.IsBound())
	{
		FGizmoInteractionDescription Desc({.Ray=InReleasePos, .bIndirect=bIndirectManipulation});
		OnPostInteractionDelegate.Broadcast(Desc);
	}
}

void UEditorTRSGizmo::OnTerminateSingleClickAndDragSequence()
{
	UE_CLOGF(ModeAxisOverride.IsSet(), LogEditorTRSGizmo, Log, "Dropping Mode axis override");
	ModeAxisOverride.Reset();

	if (!bInInteraction)
	{
		return;
	}

	EndTransformEditSequence();

	bInInteraction = false;
	bOnlyUpdatePivot = false;

	UpdateCursor(EGizmoElementInteractionState::Interacting, LastHitPart, true);
		
	if (HitTarget && LastHitPart != ETransformGizmoPartIdentifier::Default)
	{
		ResetSelectedStates(CurrentMode);

		UpdateSelectedState(true, LastHitPart);
		UpdateInteractingState(false, LastHitPart);

		UpdateSelectedState(true, GetCurrentModeLastHitPart(), true);
	}
	
	PendingDragFunction.Reset();
	EndDeltas();
}

void UEditorTRSGizmo::OnKeyPressed(const FKey& KeyID)
{
	if (KeyID == EKeys::Escape)
	{
		if (ensure(StateTarget))
		{
			StateTarget->CancelUpdate();
			OnTerminateSingleClickAndDragSequence();
		}
	}
	else if (!bInInteraction && ( KeyID == EKeys::Up || KeyID == EKeys::Down || KeyID == EKeys::Right || KeyID == EKeys::Left))
	{
		// Setup transform edit sequence
		if (NudgeData.PressedKeys.IsEmpty())
		{
			BeginTransformEditSequence();
		}

		NudgeData.PressedKeys.AddUnique(KeyID);
		NudgeData.TimeSinceLastNudge = 0.0f;

		NudgeSelection(NudgeData.GetCurrentKey());
	}
}

void UEditorTRSGizmo::OnKeyReleased(const FKey& KeyID)
{
	if (KeyID == EKeys::Up || KeyID == EKeys::Down || KeyID == EKeys::Right || KeyID == EKeys::Left)
	{
		NudgeData.PressedKeys.Remove(KeyID);

		if (NudgeData.PressedKeys.IsEmpty())
		{
			const ETransformGizmoPartIdentifier ModeLastHitPart = GetCurrentModeLastHitPart();
			UpdateInteractingState(false, ModeLastHitPart);
		
			EndTransformEditSequence();
			NudgeData.TimeSinceLastNudge = 0.0f;
		}
	}
}

void UEditorTRSGizmo::OnForceEndCapture()
{
	OnTerminateSingleClickAndDragSequence();

	bCameraFollowsMovement = false;
	bResetDragTranslate = false;
	bRotationPrecisionModeDirty = false;
	bAltKeyDown = false;
	bCtrlKeyDown = false;
	bShiftKeyDown = false;
	NudgeData.Reset();
}

void UEditorTRSGizmo::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == UE::Editor::ViewportInteractions::ShiftKeyMod)
	{
		if (bShiftKeyDown != bIsOn)
		{
			bShiftKeyDown = bIsOn;

			// In precision mode, shift is used for faster movements
			if (IsRotationPrecisionMode())
			{
				PendingDragFunction.Reset();
				bRotationPrecisionModeDirty = true;
			}
		}

		if (bCameraFollowsMovement != bIsOn)
		{
			bResetDragTranslate = true;

			// Execution order might be messed up if we have pending drag functions
			PendingDragFunction.Reset();

			bCameraFollowsMovement = bIsOn;

			if (bInInteraction)
			{
				// Camera follows starting: cache cursor position
				if (bCameraFollowsMovement)
				{
					CacheCursorPosition();
				}
				// Camera follows ends: restore cursor position
				if (!bCameraFollowsMovement && CurrentMode == EGizmoTransformMode::Translate)
				{
					RestoreCursorPosition();
				}
			}

			UpdateCursor(EGizmoElementInteractionState::Interacting, LastHitPart);
		}
	}
	else if (ModifierID == UE::Editor::ViewportInteractions::AltKeyMod)
	{
		if (bAltKeyDown != bIsOn)
		{
			bAltKeyDown = bIsOn;

			if (bAltKeyDown && bInInteraction && AllowsDragDuplicate(EDragDuplicateContext::Action))
			{
				// If already dragging, let's drop a duplicate
				constexpr bool bDroppingDuplicate = true;
				DuplicateSelection(bDroppingDuplicate);
			}
		}
	}
	else if (ModifierID == UE::Editor::ViewportInteractions::CtrlKeyMod)
	{
		if (bCtrlKeyDown != bIsOn)
		{
			bCtrlKeyDown = bIsOn;

			if (CurrentMode == EGizmoTransformMode::Rotate)
			{
				PendingDragFunction.Reset();
				bRotationPrecisionModeDirty = true;

				if (bInInteraction && !bIndirectManipulation)
				{
					if (IsRotationPrecisionMode())
					{
						// Entering precision mode: cache cursor position
						CacheCursorPosition();
					}
					else
					{
						// Precision mode ends: restore cursor position
						RestoreCursorPosition();
					}
				}

				UpdateCursor(EGizmoElementInteractionState::Interacting, LastHitPart);
			}
		}
	}
}

void UEditorTRSGizmo::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	ensureMsgf(false, TEXT("OnClickRelease without a boolean argument is a member of IClickDragBehaviorTarget. It should not be used as this class uses ISingleClickAndDragBehaviorTarget instead."));
	OnClickRelease(ReleasePos, true);
}

void UEditorTRSGizmo::OnTerminateDragSequence()
{
	ensureMsgf(false, TEXT("OnTerminateDragSequence is a member of IClickDragBehaviorTarget. It should not be used as this class uses ISingleClickAndDragBehaviorTarget instead."));
	OnTerminateSingleClickAndDragSequence();
}

void UEditorTRSGizmo::BeginTransformEditSequence()
{
	UE_LOGF(LogEditorTRSGizmo, Verbose, "[%lli] BeginTransformEditSequence", GFrameCounter);

	// Update Debug
	{
		DebugData.bIsEditing = true;
		DebugData.TransformStart = CurrentTransform;
	}

	if (ensure(ActiveTarget))
	{
		if (ActiveTarget->bSetPivotMode || bOnlyUpdatePivot)
		{
			ActiveTarget->BeginPivotEditSequence();
		}
		else
		{
			ActiveTarget->BeginTransformEditSequence();
		}
	}

	if (ensure(StateTarget))
	{
		StateTarget->BeginUpdate();
	}

	// Capture starting state (after systems above have cached stuff)
	StartTransform = GetActiveTransform();
	CurrentTransform = StartTransform;

	UE_LOGF(LogEditorTRSGizmo, Verbose, "StartTransform.Scale: %.1f", StartTransform.GetScale3D().X);

	// Camera follows starting: cache cursor position
	if (bCameraFollowsMovement || IsRotationPrecisionMode())
	{
		CacheCursorPosition();
	}

	// Handle Drag Duplicate interaction
	if (bAltKeyDown && bInInteraction && AllowsDragDuplicate(EDragDuplicateContext::Action))
	{
		DuplicateSelection();
	}
}

void UEditorTRSGizmo::EndTransformEditSequence()
{
	UE_LOGF(LogEditorTRSGizmo, Verbose, "[%lli] EndTransformEditSequence", GFrameCounter);

	// Update Debug
	{
		DebugData.bIsEditing = false;
	}

	if (ensure(StateTarget))
	{
		StateTarget->EndUpdate();
	}

	if (ensure(ActiveTarget))
	{
		if (ActiveTarget->bSetPivotMode || bOnlyUpdatePivot)
		{
			ActiveTarget->EndPivotEditSequence();
		}
		else
		{
			ActiveTarget->EndTransformEditSequence();
		}

		CurrentTransform = GetActiveTransform();
	}
}

const FTransformGizmoStyle& UEditorTRSGizmo::GetStyle() const
{
	return Style;
}

void UEditorTRSGizmo::SetStyle(const FTransformGizmoStyle& InStyle)
{
	Style = InStyle;
	Style.MakeValid(); // clamps some values, etc.

	ApplyStyle();
}

void UEditorTRSGizmo::OnClickPressAxis(const FInputDeviceRay& PressPos)
{
	if (!ensure(TranslateGroupElement))
	{
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Internal;

	// Interaction Plane
	{
		UE::Geometry::FFrame3d Plane = TranslateGroupElement->MakePlane(CurrentTransform, GizmoViewContext, GetCoordinateSystem(), GetInteractionAxisList());
		if (IsAxisSingular(GetInteractionAxisList()))
		{
			Plane = ModifyPlaneForView(Plane, GizmoViewContext, NormalToRemove);
		}

		InitializeInteractionPlanes(Plane);

		InitializeScreenInteraction(PressPos);
	}

	// Reset
	InteractionAxisStartParam = InteractionAxisCurrParam = 0.0f;

	// Interaction
	InteractionScreenObjectPos2D = GizmoViewContext->WorldToPixel(InteractionPlanarOrigin);

	const FVector LocalAxisDirection = GetAxisVector(EAxis::FromAxisList(GetInteractionAxisList()));
	InteractionScreenAxisDirection = GetScreenProjectedAxis(
		GizmoViewContext,
		LocalAxisDirection,
		CurrentTransform)
	.GetSafeNormal();
	
	InteractionScreenStartPos = InteractionScreenCurrPos = PressPos.ScreenPosition;

	// indirect manipulation uses a 2D approach instead as there's no guarantee to intersect a plane
	// When using shift + drag we also use indirect manipulation
	if (bIndirectManipulation || CameraFollowsMovement())
	{
		InitializeScreenSpaceTranslate(PressPos);
	}
	else
	{
		FVector::FReal HitDepth;
		if (GetRayParamIntersectionWithInteractionPlane(PressPos, HitDepth))
		{
			InteractionPlanarStartPoint = PressPos.WorldRay.Origin + PressPos.WorldRay.Direction * HitDepth;
			InteractionPlanarCurrPoint = InteractionPlanarStartPoint;

			// Update Debug
			if (GizmoLocals::DoDebugDraw())
			{
				DebugData.TransformStart.SetLocation(InteractionPlanarOrigin);
				DebugData.InteractionStart.SetLocation(InteractionPlanarStartPoint);
				DebugData.InteractionCurrent.SetLocation(InteractionPlanarStartPoint);
				DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
			}
		}
	}

	if (TranslateGroupElement)
	{
		BeginTranslateDelta();
	}

	bInInteraction = true;
	SetModeLastHitPart(EGizmoTransformMode::Translate, LastHitPart);

	CumulativeDragDelta = FVector::ZeroVector;
}

void UEditorTRSGizmo::OnDragStartTranslateAxis(const FInputDeviceRay& PressPos)
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;

	const FVector LocalAxisDirection = GetAxisVector(EAxis::FromAxisList(GetInteractionAxisList()));

	const FVector2D Bidirection2D = FVector2D(-InteractionScreenAxisDirection.Y, InteractionScreenAxisDirection.X).GetSafeNormal();
	if (!Bidirection2D.IsNearlyZero())
	{
		using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;

		float DistanceToNearestPoint = 0.0f;
		FVector2D NearestPoint2D = NearestPointOnLine2D(InteractionScreenStartPos, InteractionScreenAxisDirection, PressPos.ScreenPosition, DistanceToNearestPoint);

		const FVector AxisOrigin = StartTransform.GetLocation();
		const FVector AxisDirection = GetWorldAxis(LocalAxisDirection);

		const FRay3d RayFromScreenPoint = ScreenPointToWorldRay(NearestPoint2D, GizmoViewContext);
		FVector NearestPointOnAxis = FVector::ZeroVector;
		FVector UnusedNearestPointOnRay = FVector::ZeroVector;
		float DistanceAlongAxis = 0.0f;
		float DistanceAlongRay = 0.0f;
		GizmoMath::NearestPointOnLineToRay(
			AxisOrigin, AxisDirection,
			RayFromScreenPoint.Origin, RayFromScreenPoint.Direction,
			NearestPointOnAxis, DistanceAlongAxis,
			UnusedNearestPointOnRay, DistanceAlongRay);

		// The cursor is way off the axis, do nothing - in most cases this means the object has been pushed far into the horizon, and we don't want to infinitely push it into space
		if (FMath::IsNearlyZero(DistanceAlongRay))
		{
			return;
		}
		
		InteractionAxisStartParam = DistanceAlongAxis;
		InteractionAxisCurrParam = InteractionAxisStartParam;
	}
}

void UEditorTRSGizmo::SnapTranslate(FVector& InOutWorldDelta, const EAxisList::Type InAxisList, const FRay& InRay) const
{
	UInteractiveGizmoManager* GizmoManager = GetGizmoManager();
	if (!GizmoManager)
	{
		return;
	}

	if (const USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GizmoManager))
	{
		const FTransform& ActiveTransform = GetActiveTransform();

		FSceneSnapQueryRequest SnapRequest;
		SnapRequest.Transform = ActiveTransform;
		SnapRequest.RequestCoordinateSpace = GetCurrentPartCoordinateSystem();
		SnapRequest.RequestType = ESceneSnapQueryType::Position;
		SnapRequest.AxisList = InAxisList;
		SnapRequest.WorldRay = InRay;

		// Convert delta from World Space to the desired request space
		FVector PositionToSnap = InOutWorldDelta;
		if (SnapRequest.RequestCoordinateSpace == EToolContextCoordinateSystem::Local)
		{
			// Convert the delta to local space
			const FQuat Rotation = ActiveTransform.GetRotation();
			PositionToSnap = Rotation.Inverse() * InOutWorldDelta;

			SnapRequest.CoordinateTransform = ActiveTransform;
		}
		else if (SnapRequest.RequestCoordinateSpace == EToolContextCoordinateSystem::Screen)
		{
			const FQuat Rotation = FQuat(ScreenPlane.Rotation);

			// Set the coordinate transform to the screen plane rotation
			SnapRequest.CoordinateTransform.SetIdentity();
			SnapRequest.CoordinateTransform.SetRotation(Rotation);

			PositionToSnap = Rotation.Inverse() * InOutWorldDelta;
		}

		SnapRequest.Position = PositionToSnap;

		TArray<FSceneSnapQueryResult> SnapResults;
		if (SnapManager->ExecuteSceneSnapQuery(SnapRequest, SnapResults))
		{
			check(!SnapResults.IsEmpty());
			const FSceneSnapQueryResult& SnapResult = SnapResults.Last();

			// Convert the delta back to world space
			if (SnapResult.GetCoordinateSpace(SnapRequest) == EToolContextCoordinateSystem::Local)
			{
				const FQuat Rotation = ActiveTransform.GetRotation();
				InOutWorldDelta = Rotation * SnapResult.Position;
			}
			else if (SnapResult.GetCoordinateSpace(SnapRequest) == EToolContextCoordinateSystem::Screen)
			{
				const FQuat Rotation = FQuat(ScreenPlane.Rotation);
				InOutWorldDelta = Rotation * SnapResult.Position;
			}
			else
			{
				InOutWorldDelta = SnapResult.Position;
			}
		}
	}
}

void UEditorTRSGizmo::SnapRotateDelta(FQuat& InOutWorldDelta, const EAxisList::Type InAxisList) const
{
	UInteractiveGizmoManager* GizmoManager = GetGizmoManager();
	if (!GizmoManager)
	{
		return;
	}

	if (const USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GizmoManager))
	{
		FSceneSnapQueryRequest SnapRequest;
		SnapRequest.RequestCoordinateSpace = GetCurrentPartCoordinateSystem();
		SnapRequest.RequestType = ESceneSnapQueryType::Rotation;
		SnapRequest.AxisList = InAxisList;
		SnapRequest.DeltaRotation = InOutWorldDelta;

		TArray<FSceneSnapQueryResult> SnapResults;
		if (SnapManager->ExecuteSceneSnapQuery(SnapRequest, SnapResults))
		{
			check(!SnapResults.IsEmpty());

			InOutWorldDelta = SnapResults.Last().DeltaRotation;
		}
	}
}

void UEditorTRSGizmo::SnapRotateAngleDelta(double& InOutAngleDelta, const EAxisList::Type InAxisList) const
{
	UInteractiveGizmoManager* GizmoManager = GetGizmoManager();
	if (!GizmoManager)
	{
		return;
	}

	if (const USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GizmoManager))
	{
		FSceneSnapQueryRequest SnapRequest;
		SnapRequest.RequestCoordinateSpace = GetCurrentPartCoordinateSystem();
		SnapRequest.RequestType = ESceneSnapQueryType::RotationAngle;
		SnapRequest.AxisList = InAxisList;
		SnapRequest.RotationAngle = InOutAngleDelta;

		TArray<FSceneSnapQueryResult> SnapResults;
		if (SnapManager->ExecuteSceneSnapQuery(SnapRequest, SnapResults))
		{
			check(!SnapResults.IsEmpty());

			InOutAngleDelta = SnapResults.Last().RotationAngle;
		}
	}
}

void UEditorTRSGizmo::SnapScaleDelta(FVector& InOutLocalScaleDelta, const EAxisList::Type InAxisList, const FRay& InRay) const
{
	UInteractiveGizmoManager* GizmoManager = GetGizmoManager();
	if (!GizmoManager)
	{
		return;
	}

	IToolsContextQueriesAPI* QueriesAPI = GizmoManager->GetContextQueriesAPI();
	if (!QueriesAPI)
	{
		return;
	}

	if (const USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GizmoManager))
	{
		FSceneSnapQueryRequest SnapRequest;

		// Scale is always in local space. If skew is possible, this might need to be revisited.
		SnapRequest.RequestCoordinateSpace = EToolContextCoordinateSystem::Local;
		SnapRequest.RequestType = ESceneSnapQueryType::Scale;
		SnapRequest.AxisList = InAxisList;
		SnapRequest.Scale = InOutLocalScaleDelta;
		SnapRequest.WorldRay = InRay;

		TArray<FSceneSnapQueryResult> SnapResults;
		if (SnapManager->ExecuteSceneSnapQuery(SnapRequest, SnapResults))
		{
			check(!SnapResults.IsEmpty());

			FVector SnappedScale = SnapResults[0].Scale;

			// TODO 6.0
			// ModifyScaleDeltaForScaleType() bases its percentage calculations off of a `1 + Delta` mechanism
			// so that no movement == 1x 
			// For unsnapped values and snapped values on a snap grid, this works well.
			// For snapped values on a larger grid, this assumption breaks:
			//	the intuitive result of a 2x snap grid is 1 -> 2 -> 4, not 1 -> 3 -> 5
			// This block massages snapped values such that the 1 + Delta calculations result in the target value.
			// This fix is _highly_ coupled to grid snapping.
			// It was chosen as a solution-of-least-impact and should be reevaluated when there is a larger testing runway. 
			if (!GEditor->UsePercentageBasedScaling() && GetScaleType() == EGizmoTransformScaleType::PercentageBased && GEditor->GetScaleGridSize() > 1.0f)
			{
				FVector AxisMultiplier = UE::Editor::GizmoMath::GetAxisMultiplier(GetInteractionAxisList());
				for (int32 Index = 0; Index < 3; ++Index)
				{
					if (AxisMultiplier[Index] < 1.0)
					{
						continue;
					}
						
					if (FMath::Abs(SnappedScale[Index]) > 1.0)
					{
						SnappedScale[Index] -= 1.0;
					}
				}
			}

			InOutLocalScaleDelta = SnappedScale;
		}
	}
}

void UEditorTRSGizmo::OnSelectionChanged(const UTypedElementSelectionSet* InElementSelectionSet)
{
	if (InElementSelectionSet)
	{
		SelectionSetWeak = InElementSelectionSet;

		// Empty selection, terminate interaction
		if (InElementSelectionSet->GetNumSelectedElements() == 0)
		{
			OnTerminateSingleClickAndDragSequence();
		}

		// Update Debug
		if (GizmoLocals::DoDebugDraw())
		{
			DebugData.TransformStart.SetLocation(InteractionPlanarOrigin);
			DebugData.InteractionStart.SetLocation(InteractionPlanarStartPoint);
			DebugData.InteractionCurrent.SetLocation(InteractionPlanarStartPoint);
			DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
		}
	}
}

FTransform UEditorTRSGizmo::GetActiveTransform() const
{
	if (ActiveTarget)
	{
		Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::GetActiveTransformName);
		return ActiveTarget->GetTransform();
	}

	return FTransform();
}

void UEditorTRSGizmo::CacheCursorPosition()
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		if (FViewport* const Viewport = EditorViewportClient->Viewport)
		{
			CachedCursorPosition.X = Viewport->GetMouseX();
			CachedCursorPosition.Y = Viewport->GetMouseY();
		}
	}
}

void UEditorTRSGizmo::RestoreCursorPosition() const
{
	if (const FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		if (FViewport* const Viewport = EditorViewportClient->Viewport)
		{
			Viewport->SetMouse(CachedCursorPosition.X, CachedCursorPosition.Y);
		}
	}
}

void UEditorTRSGizmo::ApplySnappingUpdateDeltas(const FInputDeviceRay& DragPos)
{
	switch (CurrentMode)
	{
	case EGizmoTransformMode::Translate:
	{
		// Rounding to closest snapped value
		FVector RoundedDelta = CumulativeDragDelta;
		SnapTranslate(RoundedDelta, GetInteractionAxisList(), DragPos.WorldRay);
		ApplyTranslateDelta(RoundedDelta - CumulativeDragDelta);
		CumulativeDragDelta = FVector::ZeroVector;

		break;
	}
	case EGizmoTransformMode::Rotate:
	{
		if (bCachedRotationSnap) // Snapping Off --> On
		{
			FQuat SnappedCumulativeRotationDelta = CumulativeRotationDelta;
			SnapRotateDelta(SnappedCumulativeRotationDelta, GetInteractionAxisList());

			// Compute and apply the delta to reach that rotation, so that our object snaps back to closes snapping value
			const FQuat DeltaRotationToSnap = SnappedCumulativeRotationDelta * CumulativeRotationDelta.Inverse();
			ApplyRotateDelta(DeltaRotationToSnap);

			// Update interaction widgets where needed
			if (UE::Editor::InteractiveToolsFramework::Internal::IsAxisSingular(GetInteractionAxisList()))
			{
				const double CumulativeAngle = CumulativeRotationDelta.GetAngle();
				const double SnappedCumulativeAngle = SnappedCumulativeRotationDelta.GetAngle();
				double CorrectionAngle = SnappedCumulativeAngle - CumulativeAngle;

				if (GetInteractionAxisList() == EAxisList::Screen)
				{
					if (InteractionCurrAngle - InteractionStartAngle < 0.0f)
					{
						CorrectionAngle *= -1.0f;
					}

					InteractionCurrAngle += CorrectionAngle;
				}
				else
				{
					if (InteractionDelta.X < 0.0f)
					{
						CorrectionAngle *= -1.0f;
					}

					InteractionDelta.X += CorrectionAngle;
				}
			}

			CumulativeRotationDelta = FQuat::Identity;
			CurrentRotation = StartRotation = CurrentTransform.GetRotation();
			InteractionArcBallStartPoint = InteractionArcBallCurrPoint;
		}
		else // Snapping On --> Off
		{
			CumulativeRotationDelta = FQuat::Identity;
		}

		break;
	}
	case EGizmoTransformMode::Scale:
	{
		// Rounding to closest snapped value
		FVector RoundedDelta = CumulativeDragDelta;
		SnapScaleDelta(RoundedDelta, GetInteractionAxisList(), DragPos.WorldRay);
		ApplyScaleDelta(RoundedDelta - CumulativeDragDelta);
		CumulativeDragDelta = FVector::ZeroVector;

		break;
	}
	default:;
	}
}

void UEditorTRSGizmo::CacheSnappingStates()
{
	if (const ULevelEditorViewportSettings* Settings = GetDefault<ULevelEditorViewportSettings>())
	{
		bCachedTranslationSnap = Settings->GridEnabled;
		bCachedRotationSnap = Settings->RotGridEnabled;
		bCachedScaleSnap = Settings->SnapScaleEnabled;

		bCachedSurfaceSnap = Settings->SnapToSurface.bEnabled;
	}
}

bool UEditorTRSGizmo::HasCurrentTRSSnappingChanged() const
{
	bool bChanged = false;
	if (const ULevelEditorViewportSettings* Settings = GetDefault<ULevelEditorViewportSettings>())
	{
		switch (CurrentMode)
		{
		case EGizmoTransformMode::Translate:
			bChanged = bCachedTranslationSnap != Settings->GridEnabled;
			break;
		case EGizmoTransformMode::Rotate:
			bChanged = bCachedRotationSnap != Settings->RotGridEnabled;
			break;
		case EGizmoTransformMode::Scale:
			bChanged = bCachedScaleSnap != Settings->SnapScaleEnabled;
			break;
		default:;
		}
	}

	return bChanged;
}

bool UEditorTRSGizmo::HasSurfaceSnappingChanged() const
{
	if (const ULevelEditorViewportSettings* Settings = GetDefault<ULevelEditorViewportSettings>())
	{
		return bCachedSurfaceSnap != Settings->SnapToSurface.bEnabled;
	}

	return false;
}


void UEditorTRSGizmo::HandleSnappingChanges(const FInputDeviceRay& DragPos)
{
	if (HasCurrentTRSSnappingChanged())
	{
		CacheSnappingStates();
		ApplySnappingUpdateDeltas(DragPos);
	}

	if (HasSurfaceSnappingChanged())
	{
		CacheSnappingStates();
		ApplySurfaceSnappingUpdateDeltas(DragPos);
	}

	bTRSSnapDirty = false;
	bSurfaceSnapDirty = false;
}

void UEditorTRSGizmo::ApplySurfaceSnappingUpdateDeltas(const FInputDeviceRay& DragPos)
{
	const bool bSurfaceSnapEnabled = GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.bEnabled;
	if (bSurfaceSnapEnabled)
	{
		GUnrealEd->UpdatePivotLocationForSelection();
	}
	else
	{
		GUnrealEd->UpdatePivotLocationForSelection();

		CurrentTransform = GetActiveTransform();

		InteractionPlanarOrigin = CurrentTransform.GetLocation();

		if (CameraFollowsMovement() || bIndirectManipulation)
		{
			InteractionScreenCurrPos = DragPos.ScreenPosition;
		}
		else
		{
			FVector::FReal HitDepth;
			if (GetRayParamIntersectionWithInteractionPlane(DragPos, HitDepth))
			{
				InteractionPlanarCurrPoint = DragPos.WorldRay.Origin + DragPos.WorldRay.Direction * HitDepth;
			}
		}

		FGizmoState State;
		State.TransformMode = EGizmoTransformMode::Translate;
		GetEditorViewportClient()->BeginTransform(State);
	}
}

bool UEditorTRSGizmo::CanRender(IToolsContextRenderAPI* InRenderAPI) const
{
	if (!InRenderAPI || !GizmoElementRoot || !CanInteract())
	{
		return false;
	}

	const FSceneView* SceneView = InRenderAPI->GetSceneView();
	if (!ensure(SceneView))
	{
		return false;
	}
	
	if (SceneView->IsPerspectiveProjection() && FMath::IsNearlyZero(FVector::DistSquared(CurrentTransform.GetTranslation(), SceneView->ViewLocation)))
	{
		return false;
	}

	const FSceneViewFamily* SceneViewFamily = SceneView->Family;
	if (!ensure(SceneViewFamily))
	{
		return false;
	}

	// We ignore game mode, @see: UE-314407
	const bool bEngineShowFlagsModeWidget = SceneViewFamily->EngineShowFlags.ModeWidgets;

	return bEngineShowFlagsModeWidget;
}

bool UEditorTRSGizmo::CanInteract(const EViewportContext InViewportContext) const
{
	if (GizmoViewContext)
	{
		// No interaction if gizmo is at view location
		if (GizmoViewContext->IsPerspectiveProjection() && FMath::IsNearlyZero(FVector::DistSquared(CurrentTransform.GetTranslation(), GizmoViewContext->ViewLocation)))
		{
			return false;
		}
	}

	return Super::CanInteract(InViewportContext);
}

EToolContextCoordinateSystem UEditorTRSGizmo::GetCurrentPartCoordinateSystem() const
{
	// Add exceptions here to the general rule of using the gizmo coordinate system
	if (LastHitPart == ETransformGizmoPartIdentifier::TranslateScreenSpace)
	{
		return EToolContextCoordinateSystem::Screen;
	}

	return GetCoordinateSystem();
}

void UEditorTRSGizmo::OnClickDragTranslateAxis(const FInputDeviceRay& DragPos)
{
	Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DragTranslateAxisName);

	// Drag Translate needs to be reset (e.g. Camera Shift + Drag on/off while dragging)
	if (bResetDragTranslate)
	{
		ResetDragTranslate(DragPos);
		bResetDragTranslate = false;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Internal;
	const FVector LocalAxisDirection = GetAxisVector(EAxis::FromAxisList(GetInteractionAxisList()));

	// indirect manipulation uses a 2D projection approach instead of plane intersection
	if (bIndirectManipulation || CameraFollowsMovement())
	{
		const FVector2D PreviousScreenPos = InteractionScreenCurrPos;
		const FVector2D DragDir = DragPos.ScreenPosition - PreviousScreenPos;

		constexpr bool bUseDPIScaleForPixelToWorld = true;
		const float PixelToWorldRatio = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(GizmoViewContext, CurrentTransform.GetLocation(), bUseDPIScaleForPixelToWorld);
		const double DeltaDistance = PixelToWorldRatio * FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir);

		// Ensure the delta is only applied to the active axis, and signs based on the coordinate system, which affects screen->world
		using namespace UE::Editor::GizmoMath;
		const FVector PerAxisMultiplier = GetAxisMultiplier(GetInteractionAxisList()) * GetAxisCoordinateSystemMultiplier();

		FVector Delta = PerAxisMultiplier * DeltaDistance;
		Delta = CurrentRotation * Delta;

		const FVector PreSnapDelta = Delta;
		SnapTranslate(Delta, GetInteractionAxisList(), DragPos.WorldRay);

		ApplyTranslateDelta(Delta);
		CumulativeDragDelta += Delta;
		
		if (!Delta.IsNearlyZero())
		{
			if (GizmoLocals::CVarUseTranslationSnapRecalibration.GetValueOnAnyThread())
			{
				const FVector DeltaDelta = Delta - PreSnapDelta;
				if (DeltaDelta.IsNearlyZero())
				{
					// No snapping, use the actual screen position
					InteractionScreenCurrPos = DragPos.ScreenPosition;
				}
				else
				{
					// After a snap, the "current position" should be the mouse position that would _exactly_ match the current snap position.
					// This gets there by dead-reckoning the screen position based off the post/pre snap ratio.
					// This makes the assumption that the system has snapped in the same direction as the pre-snap.
					double Ratio = Delta.Length() / PreSnapDelta.Length();
					InteractionScreenCurrPos += DragDir * Ratio;
				}
			}
			else
			{
				InteractionScreenCurrPos = DragPos.ScreenPosition;
			}
		}

		if (CameraFollowsMovement())
		{
			if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
			{
				EditorViewportClient->TranslateViewportCamera(Delta);
			}
		}
	}
	else
	{
		const FVector2D Bidirection2D = FVector2D(-InteractionScreenAxisDirection.Y, InteractionScreenAxisDirection.X).GetSafeNormal();
		if (!Bidirection2D.IsNearlyZero())
		{
			using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;
 
			float DistanceToNearestPoint = 0.0f;
			FVector2D NearestPoint2D = NearestPointOnLine2D(InteractionScreenStartPos, InteractionScreenAxisDirection, DragPos.ScreenPosition, DistanceToNearestPoint);
 
			const FVector AxisOrigin = StartTransform.GetLocation();
			const FVector AxisDirection = GetWorldAxis(LocalAxisDirection);
 
			const FRay3d RayFromScreenPoint = ScreenPointToWorldRay(NearestPoint2D, GizmoViewContext);
			FVector NearestPointOnAxis = FVector::ZeroVector;
			FVector UnusedNearestPointOnRay = FVector::ZeroVector;
			float DistanceAlongAxis = 0.0f;
			float DistanceAlongRay = 0.0f;
			GizmoMath::NearestPointOnLineToRay(
				AxisOrigin, AxisDirection,
				RayFromScreenPoint.Origin, RayFromScreenPoint.Direction,
				NearestPointOnAxis, DistanceAlongAxis,
				UnusedNearestPointOnRay, DistanceAlongRay);
 
			// The cursor is way off the axis, do nothing - in most cases this means the object has been pushed far into the horizon, and we don't want to infinitely push it into space
			if (FMath::IsNearlyZero(DistanceAlongRay))
			{
				return;
			}

			// Compute delta from the parameter change, rather than the HitPoint used in other calculations.
			// This ensures we account for constrained movement of things like Control Rig, where we don't want to assume the translation is exactly what we requested.
			FVector Delta = AxisDirection * (DistanceAlongAxis - InteractionAxisCurrParam);
			
			SnapTranslate(Delta, GetInteractionAxisList(), DragPos.WorldRay);
			ApplyTranslateDelta(Delta);
			CumulativeDragDelta += Delta;
			
			if (!Delta.IsNearlyZero())
			{
				if (GizmoLocals::CVarUseTranslationSnapRecalibration.GetValueOnAnyThread())
				{
					// Updating by the dot product (equivalent of reprojecting and getting length)
					// Allows changes in the delta due to snapping to be taken into account.
					// For example, with a snapping grid of 100, a delta value of 51 will snap to 100.
					// If in the next mouse movement, the mouse did not move along the axis, we want
					// the next delta to be -49 (snapped to 0). 
					InteractionAxisCurrParam += Delta.Dot(AxisDirection);
				}
				else
				{
					InteractionAxisCurrParam = DistanceAlongAxis;
				}
			}
			
			InteractionScreenCurrPos = DragPos.ScreenPosition;
			InteractionPlanarCurrPoint += Delta;

			// Update Debug
			if (GizmoLocals::DoDebugDraw())
			{
				DebugData.InteractionCurrent.SetLocation(InteractionPlanarCurrPoint);
			}
		}
	}

	if (TranslateGroupElement)
	{
		UpdateTranslateDelta();
	}
}

void UEditorTRSGizmo::OnClickReleaseTranslateAxis(const FInputDeviceRay& InReleasePos)
{
	if (TranslateGroupElement)
    {
		// Update Debug
		if (GizmoLocals::DoDebugDraw()
			&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
		{
			DebugData.LastHitDeltaParts.Reset();
			DebugData.LastHitDeltaParts.Emplace(LastHitPart);
		}
		else
		{
			TranslateGroupElement->EndDelta();
		}
    }

	bInInteraction = false;
}

void UEditorTRSGizmo::OnClickPressTranslatePlanar(const FInputDeviceRay& PressPos)
{
	if (CameraFollowsMovement() || bIndirectManipulation)
	{
		InitializeScreenSpaceTranslate(PressPos);
	}
	else
	{
		OnClickPressPlanar(PressPos);
	}

	if (TranslateGroupElement)
	{
		BeginTranslateDelta();
	}

	CumulativeDragDelta = FVector::ZeroVector;
}

void UEditorTRSGizmo::OnClickDragTranslatePlanar(const FInputDeviceRay& DragPos)
{
	Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DragTranslatePlanarName);

	// Drag Translate needs to be reset (e.g. Camera Shift + Drag on/off while dragging)
	if (bResetDragTranslate)
	{
		ResetDragTranslate(DragPos);
		bResetDragTranslate = false;
	}

	const FVector CurrentLocation = CurrentTransform.GetLocation();
	FRay Ray = DragPos.WorldRay;
	
	if (bIndirectManipulation || CameraFollowsMovement())
	{
		// Indirect manipulation moves relative to the target center
		InteractionPlanarCurrPoint = CurrentLocation;
		InteractionPlanarOrigin = CurrentLocation;
		
		const FVector2D CurrentPixels = GizmoViewContext->WorldToPixel(CurrentLocation);
		const FVector2D DragDelta = DragPos.ScreenPosition - InteractionScreenCurrPos;

		Ray = GizmoViewContext->PixelToScreenRay(CurrentPixels + DragDelta, false);
	}
	
	TOptional<FVector> ComputedDelta;
	
	FVector::FReal HitDepth;
	if (GetRayParamIntersectionWithPlane(Ray, InteractionPlanarOrigin, InteractionPlanarNormal, HitDepth))
	{
		const FVector HitPoint = Ray.Origin + Ray.Direction * HitDepth;
		ComputedDelta = ComputePlanarTranslateDelta(InteractionPlanarCurrPoint, HitPoint);
	}
	else
	{
		// Fallback to projecting a guess onto the plane. This allows movements beyond the horizon to still be responsive.
		const float PixelScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(GizmoViewContext, CurrentLocation, true);
		const FVector2D DragDelta = DragPos.ScreenPosition - InteractionScreenCurrPos;
		const FVector CameraDelta = GizmoViewContext->GetViewRight() * DragDelta.X * PixelScale + GizmoViewContext->GetViewUp() * DragDelta.Y * PixelScale;
		const FVector NewLocation = FVector::PointPlaneProject(CurrentLocation + CameraDelta, CurrentLocation, InteractionPlanarNormal);
		ComputedDelta = NewLocation - CurrentLocation;
	}
	
	if (ComputedDelta.IsSet())
	{
		FVector Delta = ComputedDelta.GetValue();
			
		SnapTranslate(Delta, GetInteractionAxisList(), Ray);
		ApplyTranslateDelta(Delta);
		CumulativeDragDelta += Delta;

		// Update cached transform
		CurrentTransform = GetActiveTransform();
				
		if (CameraFollowsMovement())
		{
			if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
			{
				EditorViewportClient->TranslateViewportCamera(Delta);
			}
		}
		
		InteractionPlanarCurrPoint += Delta;

		// Update Debug
		if (GizmoLocals::DoDebugDraw())
		{
			DebugData.InteractionCurrent.SetLocation(InteractionPlanarCurrPoint);
		}
				
		if (!Delta.IsNearlyZero())
		{
			InteractionScreenCurrPos = DragPos.ScreenPosition;
		}
	}

	if (TranslateGroupElement)
	{
		UpdateTranslateDelta();
	}
}

void UEditorTRSGizmo::OnClickReleaseTranslatePlanar(const FInputDeviceRay& InReleasePos)
{
	if (TranslateGroupElement)
	{
		// Update Debug
		if (GizmoLocals::DoDebugDraw()
			&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
		{
			DebugData.LastHitDeltaParts.Reset();
			DebugData.LastHitDeltaParts.Emplace(LastHitPart);
		}
		else
		{
			TranslateGroupElement->EndDelta();
		}
	}

	bInInteraction = false;
}

void UEditorTRSGizmo::OnClickPressScreenSpaceTranslate(const FInputDeviceRay& PressPos)
{
	check(GizmoViewContext);

	InteractionPlanarOrigin = CurrentTransform.GetLocation();
	InteractionPlanarNormal = -GizmoViewContext->GetViewDirection();
	InteractionPlanarAxisX = GizmoViewContext->GetViewUp();
	InteractionPlanarAxisY = GizmoViewContext->GetViewRight();
	SetInteractionAxisList(EAxisList::Screen);

	if (TranslateGroupElement)
	{
		BeginTranslateDelta();
	}

	if (CameraFollowsMovement() || bIndirectManipulation)
	{
		InitializeScreenSpaceTranslate(PressPos);
	}
	else
	{
		OnClickPressPlanar(PressPos);
	}

	CumulativeDragDelta = FVector::ZeroVector;
}

void UEditorTRSGizmo::OnClickDragScreenSpaceTranslate(const FInputDeviceRay& DragPos)
{
	Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DragTranslateUniformName);

	OnClickDragTranslatePlanar(DragPos);

	if (TranslateGroupElement)
	{
		UpdateTranslateDelta();
	}
}

void UEditorTRSGizmo::OnClickReleaseScreenSpaceTranslate(const FInputDeviceRay& InReleasePos)
{
	if (TranslateGroupElement)
	{
		// Update Debug
		if (GizmoLocals::DoDebugDraw()
			&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
		{
			DebugData.LastHitDeltaParts.Reset();
			DebugData.LastHitDeltaParts.Emplace(LastHitPart);
		}
		else
		{
			TranslateGroupElement->EndDelta();
		}
	}

	bIndirectManipulation = false;
	bInInteraction = false;
}

void UEditorTRSGizmo::OnClickPressRotateAxis(const FInputDeviceRay& InPressPos)
{
	const EAxis::Type Axis = GetAxisForPart(LastHitPart);
	if (!ensure(Axis != EAxis::None))
	{
		bTrySwitchingToNormalPull = false;
		bInInteraction = false;
		return;
	}

	DebugData.bDebugRotate = true;

	UGizmoElementRotateAxis* Element = RotateAxisSetElement->GetAxisElement(Axis);
	if (!Element)
	{
		return;
	}

	// Interaction Plane
	{
		// Capture the initial, axis-aligned plane
		InitializeInteractionPlanes(Element->MakePlane(CurrentTransform, GizmoViewContext, GetCoordinateSystem(), GetRotationContext(), EAxisList::All));

		InitializeScreenInteraction(InPressPos);

		// If we're doing a screen-space arc, re-initialize the plane
		if (DefaultRotateMode == EAxisRotateMode::ScreenArc)
		{
			InitializeInteractionPlane(EAxisList::Screen);
		}
	}

	ViewToAlignedSign = UE::Math::TIntVector3<int8>(1);
	InitializeInteractionSign();

	// initialize pull data
	InteractionScreenAxisDirection = GetScreenRotateAxisDir(InPressPos);
	SetInteractionAxisList(EAxisList::FromAxis(Axis, AxisDisplayInfo::GetAxisDisplayCoordinateSystem()));
	ClockwiseCorrectionSignForAxis = UE::Editor::InteractiveToolsFramework::Internal::GetClockwiseAngleSignForAxis(Axis);
	InteractionScreenStartPos = InteractionScreenCurrPos = InPressPos.ScreenPosition;
	InteractionDelta = FVector::ZeroVector;
	AlignedInteractionAngleOffset = InteractionStartAngle = InteractionCurrAngle = 0.0;
	SetRotateMode(EAxisRotateMode::Pull);

	// initialize arc/mixed data
	const bool bRotatePull = bIndirectManipulation || DefaultRotateMode == EAxisRotateMode::Pull;
	const bool bCanRotateArc = OnClickPressRotateArc(InPressPos, AlignedInteractionPlanarNormal, InteractionPlanarAxisX, InteractionPlanarAxisY, Axis);
	if (!bRotatePull)
	{
		if (bCanRotateArc)
		{
			SetRotateMode(DefaultRotateMode);
		}
	}

	if (GizmoLocals::DoDebugDraw())
	{
		UE::Editor::GizmoMath::ComputePointOnCircleFromScreen(
			GizmoViewContext,
			InteractionScreenStartPos,
			1.0f,
			AlignedInteractionPlane,
			DebugData.PointOnCircleDirection);

		DebugData.PointOnCircleDirection -= AlignedInteractionPlane.Origin;
		DebugData.PointOnCircleDirection.Normalize();
	}

	// Begin element deltas
	BeginRotateAxisDelta(Element);

	bTrySwitchingToNormalPull = bIndirectManipulation && RotateMode == EAxisRotateMode::Pull;

	bInInteraction = true;

	if (IsRotationPrecisionMode())
	{
		bRotationPrecisionModeDirty = true;
	}

	SetModeLastHitPart(EGizmoTransformMode::Rotate, LastHitPart);
}

void UEditorTRSGizmo::OnClickPressScaleXAxis(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::X);
	InteractionAxisOrigin = CurrentTransform.GetLocation();
	InteractionAxisDirection = GetWorldAxis(FVector::XAxisVector);
	OnClickPressScale(PressPos);
}

void UEditorTRSGizmo::OnClickPressScaleYAxis(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::Y);
	InteractionAxisOrigin = CurrentTransform.GetLocation();
	InteractionAxisDirection = GetWorldAxis(FVector::YAxisVector);
	OnClickPressScale(PressPos);
}

void UEditorTRSGizmo::OnClickPressScaleZAxis(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::Z);
	InteractionAxisOrigin = CurrentTransform.GetLocation();
	InteractionAxisDirection = GetWorldAxis(FVector::ZAxisVector);

	OnClickPressScale(PressPos);
}

void UEditorTRSGizmo::OnClickPressScaleXYPlanar(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::XY);
	OnClickPressScale(PressPos);
}

void UEditorTRSGizmo::OnClickPressScaleYZPlanar(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::YZ);
	OnClickPressScale(PressPos);
}

void UEditorTRSGizmo::OnClickPressScaleXZPlanar(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::XZ);
	OnClickPressScale(PressPos);
}

void UEditorTRSGizmo::OnClickPressScalePlanar(const FInputDeviceRay& PressPos)
{
	OnClickPressScale(PressPos);
}

void UEditorTRSGizmo::OnClickPressScale(const FInputDeviceRay& PressPos)
{
	if (!ensure(ScaleGroupElement))
	{
		return;
	}

	constexpr bool bLogScale = UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::bLogScale;

	bInInteraction = true;
	constexpr bool bUseScreenSpaceScaling = UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::bUseScreenspaceScaling;

	FVector2D ScreenAxisDirection = FVector2D::ZeroVector;
	FVector AxisDirection = FVector::ZeroVector;

	const bool bEnableX = static_cast<uint8>(GetInteractionAxisList()) & static_cast<uint8>(EAxisList::X);
	const bool bEnableY = static_cast<uint8>(GetInteractionAxisList()) & static_cast<uint8>(EAxisList::Y);
	const bool bEnableZ = static_cast<uint8>(GetInteractionAxisList()) & static_cast<uint8>(EAxisList::Z);
	const bool bEnableAll = bEnableX && bEnableY && bEnableZ;

	if (bEnableAll)
	{
		ScreenAxisDirection = Interaction.ScaleInteraction.Direction * FVector2D(1, - 1);
		AxisDirection = FVector::UpVector;
	}
	else
	{
		using namespace UE::Editor::InteractiveToolsFramework::Internal;
		if (GetInteractionAxisList() & EAxisList::X)
		{
			const FVector LocalAxisDirection = GetAxisVector(EAxis::X);

			ScreenAxisDirection += GetScreenProjectedAxis(GizmoViewContext, LocalAxisDirection, CurrentTransform);
			AxisDirection += GetWorldAxis(LocalAxisDirection);
		}

		if (GetInteractionAxisList() & EAxisList::Y)
		{
			const FVector LocalAxisDirection = GetAxisVector(EAxis::Y);

			ScreenAxisDirection += GetScreenProjectedAxis(GizmoViewContext, LocalAxisDirection, CurrentTransform);
			AxisDirection += GetWorldAxis(LocalAxisDirection);
		}

		if (GetInteractionAxisList() & EAxisList::Z)
		{
			const FVector LocalAxisDirection = GetAxisVector(EAxis::Z);

			ScreenAxisDirection += GetScreenProjectedAxis(GizmoViewContext, LocalAxisDirection, CurrentTransform);
			AxisDirection += GetWorldAxis(LocalAxisDirection);
		}
	}

	InteractionScreenAxisDirection = ScreenAxisDirection.GetSafeNormal();
	InteractionScreenStartPos = InteractionScreenEndPos = InteractionScreenCurrPos = PressPos.ScreenPosition;

	InteractionAxisDirection = AxisDirection.GetSafeNormal();

	InteractionDelta = FVector::ZeroVector;
	InteractionBidirection = FVector::ZeroVector;
	InteractionReferencePointOffsetDistance = 0.0;
	InteractionStartSign = 1.0;

	using namespace UE::Editor::InteractiveToolsFramework::Internal;
	using namespace UE::Editor::InteractiveToolsFramework::Private;

	// Interaction Plane
	{
		UE::Geometry::FFrame3d Plane = ScaleGroupElement->MakePlane(CurrentTransform, GizmoViewContext, GetCoordinateSystem(), GetInteractionAxisList());
		if (IsAxisSingular(GetInteractionAxisList()))
		{
			Plane = ModifyPlaneForView(Plane, GizmoViewContext, NormalToRemove);
		}

		InitializeInteractionPlanes(Plane);
	}

	// Interaction
	InteractionScreenObjectPos2D = GizmoViewContext->WorldToPixel(InteractionPlanarOrigin);

	// Determine bi-direction/mirror line
	if (Interaction.ScaleInteraction.DistanceType == EGizmoElementScaleDistanceType::Directional)
	{
		const FVector2D Bidirection2D = FVector2D(-InteractionScreenAxisDirection.Y, InteractionScreenAxisDirection.X).GetSafeNormal();
		InteractionBidirection2D = Bidirection2D;

		const FVector2D BidirectionPoint2D = InteractionScreenObjectPos2D + Bidirection2D * 100.0f;

		DebugData.Test2 = BidirectionPoint2D;

		// @note: we still need to project this to the interaction plane later
		FVector BidirectionPoint = TransformGizmoLocals::ScreenPointToWorldPoint(BidirectionPoint2D, GizmoViewContext);

		FInputDeviceRay BidirectionIntersectionRay = PressPos;
		BidirectionIntersectionRay.WorldRay.Direction = (BidirectionPoint - BidirectionIntersectionRay.WorldRay.Origin).GetSafeNormal();

		FVector::FReal HitDepth;
		if (GetRayParamIntersectionWithInteractionPlane(BidirectionIntersectionRay, HitDepth))
		{
			const FVector HitPoint = BidirectionIntersectionRay.WorldRay.Origin + BidirectionIntersectionRay.WorldRay.Direction * HitDepth;
			InteractionBidirection = (HitPoint - InteractionPlanarOrigin).GetSafeNormal();
		}
	}

	// @see: UGizmoElementScaleAxis::ApplyStyle
	InteractionDeltaDivisor = Style.ScaleStyle.AxisStyle.AxisOffsetFromCenter + Style.ScaleStyle.AxisStyle.AxisLength * Style.GetModifiedAxisSizeMultiplier() * GetSizeCoefficient();

	FVector::FReal HitDepth;
	if (GetRayParamIntersectionWithInteractionPlane(PressPos, HitDepth))
	{
		const FVector HitPoint = PressPos.WorldRay.Origin + PressPos.WorldRay.Direction * HitDepth;
		InteractionPlanarStartPoint = InteractionPlanarCurrPoint = HitPoint;
		InteractionPlanarStartPoint2D = GizmoViewContext->WorldToPixel(InteractionPlanarStartPoint);

		if (Interaction.ScaleInteraction.DistanceSource == EGizmoElementScaleDistanceSource::FromStart)
		{
			// Distance is from the start, so we init to 0.0
			InteractionPlanarStartDistance = InteractionPlanarCurrDistance = 0.0;
			InteractionReferencePointOffsetDistance = -InteractionDeltaDivisor;
		}
		else // From Origin
		{
			// Otherwise, we use the distance to object origin (which is also the interaction plane origin)

			if constexpr (TransformGizmoLocals::bUseScreenspaceScaling)
			{
				InteractionPlanarStartDistance = InteractionPlanarCurrDistance = FMath::Max(
					UE_DOUBLE_KINDA_SMALL_NUMBER,
					FVector2D::Distance(PressPos.ScreenPosition, InteractionScreenObjectPos2D));
			}
			else // 3D space
			{
				InteractionPlanarStartDistance = InteractionPlanarCurrDistance = FMath::Max(
					UE_DOUBLE_KINDA_SMALL_NUMBER,
					FVector::Distance(InteractionPlanarStartPoint, InteractionPlanarOrigin));
			}

			InteractionReferencePointOffsetDistance = InteractionPlanarStartDistance - InteractionDeltaDivisor;
		}

		UE_CLOGF(bLogScale, LogEditorTRSGizmo, Verbose, "[%lli] BeginCapture: Offset: %.1f", GFrameCounter, InteractionReferencePointOffsetDistance);
		UE_CLOGF(bLogScale, LogEditorTRSGizmo, Verbose, "[%lli] BeginCapture: StartRadius: %.1f | DeltaDivisor: %.1f", GFrameCounter, InteractionPlanarStartDistance, InteractionDeltaDivisor);
	}

	// Set initial screen-space sign
	if (Interaction.ScaleInteraction.DistanceSource == EGizmoElementScaleDistanceSource::FromStart)
	{
		if (bEnableAll && bUseScreenSpaceScaling)
		{
			InteractionStartSign = 1.0;
		}
	}

	BeginScaleDelta();

	SetModeLastHitPart(EGizmoTransformMode::Scale, LastHitPart);

	CumulativeDragDelta = FVector::ZeroVector;
}

void UEditorTRSGizmo::OnClickDragScale(const FInputDeviceRay& DragPos)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private;
	Profiler->BeginScoped(TransformGizmoLocals::DragScaleAxisName);

	constexpr bool bLogScale = TransformGizmoLocals::bLogScale;

	FVector2D ScreenDelta = DragPos.ScreenPosition - InteractionScreenCurrPos;

	InteractionScreenCurrPos = DragPos.ScreenPosition;
	InteractionScreenEndPos += ScreenDelta;

	// We adjust the ref point so that it's pushed back from it's original location, by the InteractionDeltaDivisor, in the opposite direction to the cursor to ref point
	FVector DistanceReferencePoint =
		Interaction.ScaleInteraction.DistanceSource == EGizmoElementScaleDistanceSource::FromStart
		? InteractionPlanarStartPoint
		: InteractionPlanarOrigin;

	FVector2D DistanceReferencePoint2D =
		Interaction.ScaleInteraction.DistanceSource == EGizmoElementScaleDistanceSource::FromStart
		? InteractionScreenStartPos
		: InteractionScreenObjectPos2D;

	FVector2D DirectionAxis2D =
		!bIndirectManipulation
		? InteractionScreenAxisDirection
		: Interaction.ScaleInteraction.Direction * FVector2D(1, -1);

	double DistanceToReferencePoint = 0.0;
	double DistanceToReferencePoint2D = 0.0; // In screen-space
	FVector DirectionToOrigin = FVector::ZeroVector;

	// The distance for delta calculations is based upon an offset so that it's a fraction of a constant length, rather than a variable fraction based on the start cursor distance to some reference point.
	const double OffsetScreenStartDistance = InteractionPlanarStartDistance - InteractionReferencePointOffsetDistance;
	double OffsetScreenCurrDistance = InteractionPlanarCurrDistance - InteractionReferencePointOffsetDistance;

	// Attempt 3D plane intersection
	FVector CursorPlaneHitPos = FVector::ZeroVector;
	FVector::FReal HitDepth;
	const bool bHitPlane = GetRayParamIntersectionWithInteractionPlane(DragPos, HitDepth);
	if (bHitPlane)
	{
		CursorPlaneHitPos = DragPos.WorldRay.Origin + DragPos.WorldRay.Direction * HitDepth;
		InteractionPlanarCurrPoint = CursorPlaneHitPos;
	}

	if (Interaction.ScaleInteraction.DistanceType == EGizmoElementScaleDistanceType::Directional)
	{
		const FVector2D Bidirection2D = FVector2D(-InteractionScreenAxisDirection.Y, InteractionScreenAxisDirection.X).GetSafeNormal();
		if (!Bidirection2D.IsNearlyZero())
		{
			using namespace UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals;

			float DistanceToNearestPoint = 0.0f;
			FVector2D NearestPoint2D = NearestPointOnLine2D(DistanceReferencePoint2D, Bidirection2D, InteractionScreenCurrPos, DistanceToNearestPoint);

			// Distance along direction/axis
			DistanceToReferencePoint = FVector::DotProduct(InteractionAxisDirection, (CursorPlaneHitPos - DistanceReferencePoint));
			DistanceToReferencePoint2D = FVector2D::DotProduct(DirectionAxis2D, (DragPos.ScreenPosition - DistanceReferencePoint2D));

			UE_CLOGF(bLogScale, LogEditorTRSGizmo, Verbose, "DistanceToRef2D: %.1f | DistanceToRef: %.1f", DistanceToReferencePoint2D, DistanceToReferencePoint);

			// Update Debug
			if (GizmoLocals::DoDebugDraw())
			{
				DebugData.Test2 = NearestPoint2D;
			}

			FVector NearestPoint = ScreenPointToWorldPoint(NearestPoint2D, GizmoViewContext);

			FInputDeviceRay BidirectionIntersectionRay = DragPos;
			BidirectionIntersectionRay.WorldRay.Direction = (NearestPoint - BidirectionIntersectionRay.WorldRay.Origin).GetSafeNormal();

			if (bHitPlane && GetRayParamIntersectionWithInteractionPlane(BidirectionIntersectionRay, HitDepth))
			{
				DistanceReferencePoint = NearestPoint;
			}
		}
	}
	else if (Interaction.ScaleInteraction.DistanceType == EGizmoElementScaleDistanceType::Linear)
	{
		// Distance to reference point is the distance to the interaction plane origin
		DistanceToReferencePoint = FVector::Distance(CursorPlaneHitPos, InteractionPlanarOrigin);
		DistanceToReferencePoint2D = FVector2D::Distance(GizmoViewContext->WorldToPixel(CursorPlaneHitPos), InteractionScreenObjectPos2D);
	}

	DirectionToOrigin = (DistanceReferencePoint - InteractionPlanarCurrPoint).GetSafeNormal();

	InteractionPlanarCurrDistance = DistanceToReferencePoint2D;
	OffsetScreenCurrDistance = InteractionPlanarCurrDistance - InteractionReferencePointOffsetDistance;

	// Update Debug
	if (GizmoLocals::DoDebugDraw())
	{
		DebugData.ReferencePoint = DistanceReferencePoint;
	}

	FVector ScaleDelta = FVector::ZeroVector;
	FVector AxisMultiplier = UE::Editor::GizmoMath::GetAxisMultiplier(GetInteractionAxisList());

	double ScaleSign = InteractionStartSign;

	double RadialDeltaFromStart = 0.0;
	if (!FMath::IsNearlyZero(OffsetScreenStartDistance))
	{
		RadialDeltaFromStart = (OffsetScreenCurrDistance / OffsetScreenStartDistance) - ScaleSign;
		RadialDeltaFromStart *= ScaleSign; // Flips the sign if the interaction is in the opposite direction
	}

	ScaleDelta = RadialDeltaFromStart * AxisMultiplier;
	FVector PreSnapScaleDelta = ScaleDelta;

	SnapScaleDelta(ScaleDelta, GetInteractionAxisList());

	const FVector PreviousAppliedDeltaScale = InteractionDelta;
	InteractionDelta = ScaleDelta;

	ModifyScaleDeltaForScaleType(GetScaleType(), PreviousAppliedDeltaScale, ScaleDelta);

	UE_CLOGF(bLogScale, LogEditorTRSGizmo, Verbose, "PreSnapDelta: %ls | Delta: %ls",
		*PreSnapScaleDelta.ToString(),
		*ScaleDelta.ToString());

	UE_CLOGF(bLogScale, LogEditorTRSGizmo, Verbose, "ScaleSign: %.3f | StartR: %.3f | CurrentR: %.3f | R: %.3f | InteractionD: %ls | Divisor: %.3f | ScaleD: %ls",
		ScaleSign,
		OffsetScreenStartDistance,
		OffsetScreenCurrDistance,
		RadialDeltaFromStart,
		*InteractionDelta.ToString(),
		InteractionDeltaDivisor,
		*ScaleDelta.ToString());


	if (ScaleDelta.X != 0.0 || ScaleDelta.Y != 0.0 || ScaleDelta.Z != 0.0)
	{
		FVector PreDeltaScale = GetActiveTransform().GetScale3D();

		ApplyScaleDelta(ScaleDelta);

		CumulativeDragDelta += ScaleDelta;

		FVector PostDeltaScale = GetActiveTransform().GetScale3D();
		
		if (FMath::IsNearlyEqual((PreDeltaScale - PostDeltaScale).SquaredLength(), 0.0))
		{
			// The attempt at scaling appeared to have no effect.
			// The reported scale cannot be trusted for the purposes of feeding percentage-based deltas.
			UE_CLOGF(bLogScale, LogEditorTRSGizmo, Verbose, "Non standard scaling detected. Sent (%ls), got %ls vs %ls", *ScaleDelta.ToCompactString(), *PreDeltaScale.ToCompactString(), *PostDeltaScale.ToCompactString());
			bDetectedNonStandardScaling = true;
		}

		UE_CLOGF(bLogScale, LogEditorTRSGizmo, Verbose, "[%lli] UpdateCapture: ScaleDelta: %ls | Pre: %ls | Post: %ls", GFrameCounter, *ScaleDelta.ToString(), *PreDeltaScale.ToString(), *PostDeltaScale.ToString());
	}

	UpdateScaleDelta();

	UE_CLOGF(bLogScale, LogEditorTRSGizmo, Verbose, "[%lli] UpdateCapture: ScreenToWorldDistanceMultiplier: %.1f", GFrameCounter, DistanceToReferencePoint);

	// Update Debug
	if (GizmoLocals::DoDebugDraw())
	{
		DebugData.InteractionCurrent.SetLocation(CursorPlaneHitPos);
		DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
	}
}

void UEditorTRSGizmo::OnClickReleaseScaleAxis(const FInputDeviceRay& ReleasePos)
{
	UpdateAllScaleAxis();

	bInInteraction = false;

	if (ScaleGroupElement)
	{
		// Update Debug
		if (GizmoLocals::DoDebugDraw()
			&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
		{
			DebugData.LastHitDeltaParts.Reset();
			DebugData.LastHitDeltaParts.Emplace(LastHitPart);
		}
		else
		{
			ScaleGroupElement->EndDelta();
		}
	}
}

void UEditorTRSGizmo::OnClickReleaseScalePlanar(const FInputDeviceRay& ReleasePos)
{
	UpdateAllScaleAxis();

	bInInteraction = false;

	if (ScaleGroupElement)
	{
		// Update Debug
		if (GizmoLocals::DoDebugDraw()
			&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
		{
			DebugData.LastHitDeltaParts.Reset();
			DebugData.LastHitDeltaParts.Emplace(LastHitPart);
		}
		else
		{
			ScaleGroupElement->EndDelta();
		}
	}
}

void UEditorTRSGizmo::OnClickReleaseScaleXYZ(const FInputDeviceRay& ReleasePos)
{
	UpdateAllScaleAxis();

	bInInteraction = false;

	if (ScaleGroupElement)
	{
		// Update Debug
		if (GizmoLocals::DoDebugDraw()
			&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
		{
			DebugData.LastHitDeltaParts.Reset();
			DebugData.LastHitDeltaParts.Emplace(LastHitPart);
		}
		else
		{
			ScaleGroupElement->EndDelta();
		}
	}
}

FVector UEditorTRSGizmo::ComputeScaleDelta(const FVector2D& InStartPos, const FVector2D& InEndPos, FVector2D& OutScreenDelta)
{
	const FVector2D DragDir = InEndPos - InStartPos;
	const double ScaleDelta = FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir);

	const FVector Scale(
		(GetInteractionAxisList() & EAxisList::X) ? ScaleDelta : 0.0f,
		(GetInteractionAxisList() & EAxisList::Y) ? ScaleDelta : 0.0f,
		(GetInteractionAxisList() & EAxisList::Z) ? ScaleDelta : 0.0f);

	const FVector::FReal ScaleMax = Scale.GetMax();
	const FVector::FReal ScaleMin = Scale.GetMin();
	const FVector::FReal ScaleApplied = (ScaleMax > -ScaleMin) ? ScaleMax : ScaleMin;

	OutScreenDelta = InteractionScreenAxisDirection * ScaleApplied;

	return Scale;
}

void UEditorTRSGizmo::ModifyScaleDeltaForScaleType(const EGizmoTransformScaleType InScaleType, const FVector& InPreviousDelta, FVector& InOutScaleDelta)
{
	check(ActiveTarget);

	const FVector& StartObjectScale = StartTransform.GetScale3D();
	const FVector& CurrentObjectScale = CurrentTransform.GetScale3D();

	// Start and current scale info for current selection might be unavailable/unreliable.
	// e.g. some editors might empty current selection when interacting with certain elements (e.g. Control Rig).
	// Fallback is "Default" scaling (see bIsPercentageBasedScale below)
	const bool bScaleInfoAvailable = !bDetectedNonStandardScaling && !(StartObjectScale.IsZero() && CurrentObjectScale.IsZero());

	// Get the global setting so we don't double-up any calculations, otherwise we can use custom logic for user-overridden percentage based scaling
	// The global percentage based scale mechanism should be considered to be obsolete
	const bool bIsGlobalPercentageBasedScale = GEditor->UsePercentageBasedScaling();

	// When StartObjectScale has at least one axis close to zero, we will not perform percentage based scale
	const bool bStartsFromZeroedAxis = FMath::IsNearlyZero(StartObjectScale.X) || FMath::IsNearlyZero(StartObjectScale.Y) ||FMath::IsNearlyZero(StartObjectScale.Z);

	const bool bIsPercentageBasedScale = InScaleType == EGizmoTransformScaleType::PercentageBased && bScaleInfoAvailable && !bStartsFromZeroedAxis;

	float AdjustedScaleMultiplier = 1.0f;
	if (bIsPercentageBasedScale && bIsGlobalPercentageBasedScale)
	{
		AdjustedScaleMultiplier = 100.0f;
	}

	if (bIsPercentageBasedScale && !bIsGlobalPercentageBasedScale)
	{
		const FVector& DeltaFromStart = InteractionDelta;

		// Work out the *intended* scale, assumes UTransformProxy returns a correct Transform Scale
		const FVector TargetScale = StartObjectScale * (FVector::OneVector + DeltaFromStart);

		InOutScaleDelta = TargetScale - CurrentObjectScale;

		UE_CLOGF(
			UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::bLogScale,
			LogEditorTRSGizmo, Verbose, "DeltaFromStart: %ls | StartObjScale: %ls | CurrentObjScale: %ls | TargetScale: %ls | ThisDelta: %ls",
				*DeltaFromStart.ToCompactString(),
				*StartObjectScale.ToCompactString(),
				*CurrentObjectScale.ToCompactString(),
				*TargetScale.ToCompactString(),
				*InOutScaleDelta.ToCompactString());
	}
	// Percentage mode is based on a fraction of the already-applied scale
	else if (bIsPercentageBasedScale)
	{
		InOutScaleDelta = (FVector::OneVector + InOutScaleDelta) / (FVector::OneVector + InPreviousDelta);
		InOutScaleDelta -= FVector::OneVector;
	}
	// Offset mode is based on a fraction of the divisor
	else
	{
		// Modifier because it may flip existing signs, rather than just setting to this value (ie. it doesn't pre-abs the value being affected)
		FVector ScaleSignModifier = FVector::OneVector;
	
		const FVector OriginalScale = StartTransform.GetScale3D();
		const FVector OriginalScaleSign = OriginalScale.GetSignVector();
	
		int8 UniformSign = 1;
		bool bIsUniformSign = UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::GetEffectiveSignForAxisList(GetInteractionAxisList(), OriginalScaleSign, UniformSign);

		// @todo: calculate once on begin transform
		// Compensate for scale sign for directional scale, where scaling "up" effectively means "more negative" if the original scale is negative
		constexpr bool bApplyDirectionalScale = true;
		if constexpr (bApplyDirectionalScale)
		{
			if (bIsUniformSign)
			{
				ScaleSignModifier *= UniformSign;
			}
			else
			{
				ScaleSignModifier = OriginalScaleSign;
			}
		}
	
		InOutScaleDelta -= InPreviousDelta;
		InOutScaleDelta *= Interaction.ScaleInteraction.Multiplier; // Multiplier only applies to offset scale type
		InOutScaleDelta *= ScaleSignModifier;
	}

	InOutScaleDelta *= AdjustedScaleMultiplier;
}

void UEditorTRSGizmo::NudgeSelection(const FKey& InCurrentNudgeKey)
{
	using namespace UE::Editor::InteractiveToolsFramework;

	if (InCurrentNudgeKey == EKeys::AnyKey)
	{
		return;
	}

	if (FEditorViewportClient* const ViewportClient = GetEditorViewportClient())
	{
		EAxisList::Type AxesToHighlight = EAxisList::None;
	
		if (CurrentMode == EGizmoTransformMode::Translate)
		{
			SetInteractionAxisList(EAxisList::All);
			const TNudgeDelta<FVector> TranslationNudge = GetTranslationNudge();
			ApplyTranslateDelta(TranslationNudge.Delta);

			// Shift Drag, to move camera while nudging
			if (CurrentMode == EGizmoTransformMode::Translate && bCameraFollowsMovement)
			{
				ViewportClient->TranslateViewportCamera(TranslationNudge.Delta);
			}
			
			AxesToHighlight = TranslationNudge.Axis;
		}
		else if (CurrentMode == EGizmoTransformMode::Rotate)
		{
			SetInteractionAxisList(EAxisList::All);
			const TNudgeDelta<FQuat> RotationNudge = GetRotationNudge();
			ApplyRotateDelta(RotationNudge.Delta);
			
			AxesToHighlight = RotationNudge.Axis;
		}
		else if (CurrentMode == EGizmoTransformMode::Scale)
		{
			const TNudgeDelta<FVector> ScaleNudge = GetScaleNudge();
			SetInteractionAxisList(ScaleNudge.Axis);
			ApplyScaleDelta(ScaleNudge.Delta);
			
			AxesToHighlight = ScaleNudge.Axis;
		}

		SetInteractionAxisList(EAxisList::None);
		CurrentTransform = GetActiveTransform();
		ViewportClient->Invalidate();
		
		if (AxesToHighlight != EAxisList::None)
		{
			const ETransformGizmoPartIdentifier PartToHighlight = GetPartForAxisList(AxesToHighlight);
			const ETransformGizmoPartIdentifier ModeLastHitPart = GetCurrentModeLastHitPart();
			
			ResetSelectedStates(CurrentMode);
			SetModeLastHitPart(CurrentMode, PartToHighlight);
			UpdateInteractingState(false, ModeLastHitPart);
			
			if (PartToHighlight != ETransformGizmoPartIdentifier::Default)
			{
				UpdateInteractingState(true, PartToHighlight, true);
				UpdateSelectedState(true, PartToHighlight, true);
			}
		}
	}
}

UE::Editor::InteractiveToolsFramework::TNudgeDelta<FVector> UEditorTRSGizmo::GetTranslationNudge() const
{
	using namespace UE::Editor::InteractiveToolsFramework;

	if (!ActiveTarget)
	{
		return { FVector::ZeroVector, EAxisList::None };
	}

	const FTransform TargetTransform = ActiveTarget->GetTransform();

	const FKey& CurrentNudgeKey = NudgeData.GetCurrentKey();

	TNudgeDelta<FVector> TranslateNudge;
	if (GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.bScreenSpaceNudge)
	{
		TranslateNudge = GetScreenSpaceNudgeTransformAxis(GizmoViewContext, TargetTransform, GetNudgeDirection(CurrentNudgeKey), CurrentNudgeKey);
	}
	else
	{
		TranslateNudge = GetAbsoluteNudgeTransformAxis(GetNudgeDirection(CurrentNudgeKey), CurrentNudgeKey);
	}

	TranslateNudge.Delta = TargetTransform.GetRotation().RotateVector(TranslateNudge.Delta);
	
	// TODO: Could be better to provide snapping/grid values through a Context Object (e.g. see SnapRotateAngleDelta using USceneSnappingManager)
	TranslateNudge.Delta *= GEditor->GetGridSize();

	return TranslateNudge;
}

UE::Editor::InteractiveToolsFramework::TNudgeDelta<FQuat> UEditorTRSGizmo::GetRotationNudge() const
{
	using namespace UE::Editor::InteractiveToolsFramework;

	TNudgeDelta<FVector> VectorRotationNudge;
	
	const FKey CurrentNudgeKey = NudgeData.GetCurrentKey();
	const ENudgeDirection NudgeDirection = GetNudgeDirection(CurrentNudgeKey);
	const FTransform ActiveTransform = GetActiveTransform();
	const EToolContextCoordinateSystem CoordinateSystem = GetCoordinateSystem();

	// Retrieve current rotation snapping value
	// TODO: Could be better to provide snapping/grid values through a Context Object (e.g. see SnapRotateAngleDelta using USceneSnappingManager)
	const TArray<float>& RotGridSizes = GEditor->GetCurrentRotationGridArray();
	const int32 CurrentRotGridSize = GetDefault<ULevelEditorViewportSettings>()->CurrentRotGridSize;
	float RotationAmount = 0.0001f;
	if (RotGridSizes.IsValidIndex(CurrentRotGridSize))
	{
		RotationAmount = RotGridSizes[CurrentRotGridSize];
	}

	if (GetRotationContext().bUseExplicitRotator)
	{
		// Explicit assigns deltas directly to Roll, Pitch or Yaw
		FRotator ExplicitRotator = FRotator::ZeroRotator;
		const float VerticalAxisMultiplier = CurrentNudgeKey == EKeys::Down ? -1.0f : CurrentNudgeKey == EKeys::Up ? 1.0f : 0.0f;

		if (NudgeDirection == Vertical)
		{
			VectorRotationNudge.Axis = EAxisList::Y;
			ExplicitRotator.Pitch = RotationAmount * VerticalAxisMultiplier;
		}
		else if (NudgeDirection == Secondary)
		{
			VectorRotationNudge.Axis = EAxisList::Z;
			ExplicitRotator.Yaw = RotationAmount * VerticalAxisMultiplier;
		}
		else if (NudgeDirection == Horizontal)
		{
			VectorRotationNudge.Axis = EAxisList::X;
			const float HorizontalAxisMultiplier = CurrentNudgeKey == EKeys::Right ? 1.0f : CurrentNudgeKey == EKeys::Left ? -1.0f : 0.0f;
			ExplicitRotator.Roll = RotationAmount * HorizontalAxisMultiplier;
		}

		return { ExplicitRotator.Quaternion(), VectorRotationNudge.Axis };
	}
	
	// World and Local
	if (CoordinateSystem < EToolContextCoordinateSystem::Screen)
	{
		VectorRotationNudge = GetAbsoluteNudgeTransformAxis(NudgeDirection, CurrentNudgeKey);
		VectorRotationNudge.Delta *= -1.0f;
		VectorRotationNudge.Delta.Z *= -1.0f;

		if (CoordinateSystem == EToolContextCoordinateSystem::Local)
		{
			VectorRotationNudge.Delta = ActiveTransform.GetRotation().RotateVector(VectorRotationNudge.Delta);
		}
	}

	return { FQuat(VectorRotationNudge.Delta, FMath::DegreesToRadians(RotationAmount)), VectorRotationNudge.Axis };
}

UE::Editor::InteractiveToolsFramework::TNudgeDelta<FVector> UEditorTRSGizmo::GetScaleNudge() const
{
	using namespace UE::Editor::InteractiveToolsFramework;

	if (!ActiveTarget)
	{
		return { FVector::ZeroVector, EAxisList::None };
	}

	TNudgeDelta<FVector> ScaleNudge;

	const FKey& CurrentNudgeKey = NudgeData.GetCurrentKey();
	
	if (GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.bScreenSpaceNudge)
	{
		ScaleNudge = GetScreenSpaceNudgeTransformAxis(GizmoViewContext, ActiveTarget->GetTransform(), GetNudgeDirection(CurrentNudgeKey), CurrentNudgeKey);
	}
	else
	{
		ScaleNudge = GetAbsoluteNudgeTransformAxis(GetNudgeDirection(CurrentNudgeKey), CurrentNudgeKey);
	}

	// TODO: Could be better to provide snapping/grid values through a Context Object (e.g. see SnapRotateAngleDelta using USceneSnappingManager)
	ScaleNudge.Delta *= GEditor->GetScaleGridSize();

	return ScaleNudge;
}

UEditorTRSGizmo::ENudgeDirection UEditorTRSGizmo::GetNudgeDirection(const FKey& InCurrentNudgeKey) const
{
	if (InCurrentNudgeKey == EKeys::Left || InCurrentNudgeKey == EKeys::Right)
	{
		return ENudgeDirection::Horizontal;
	}
	else if (InCurrentNudgeKey == EKeys::Up || InCurrentNudgeKey == EKeys::Down)
	{
		return bCtrlKeyDown ? ENudgeDirection::Secondary : ENudgeDirection::Vertical;
	}

	return ENudgeDirection::None;
}

void UEditorTRSGizmo::ApplyScaleDelta(const FVector& InScaleDelta)
{
	check(ActiveTarget);

	if (UEditorTransformProxy* EditorTransformProxy = Cast<UEditorTransformProxy>(ActiveTarget))
	{
		EditorTransformProxy->InputScaleDelta(InScaleDelta, GetInteractionAxisList());

		// Pull the scale directly from the transformed proxy.
		// The actual applied scale can be post-processed, making the delta not match the actual result.
		CurrentTransform = EditorTransformProxy->GetTransform();
	}
	else
	{
		UTransformGizmo::ApplyScaleDelta(InScaleDelta);
	}
}

bool UEditorTRSGizmo::OnClickPressRotateArc(
	const FInputDeviceRay& InPressPos, const FVector& InPlaneNormal,
	const FVector& InPlaneAxis1, const FVector& InPlaneAxis2,
	const EAxis::Type InAxis)
{
	using namespace UE::Editor::GizmoMath;

	constexpr bool bLogRotation = UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::bLogRotation;

	const FRay& Ray = InPressPos.WorldRay;

	// compute axis / view direction projection: is the rotation plane nearly perpendicular to the view plane?
	const FVector WorldOrigin = CurrentTransform.GetLocation();
	const FVector ViewDirection = GizmoViewContext->IsPerspectiveProjection() ?
		(WorldOrigin - GizmoViewContext->ViewLocation).GetSafeNormal() :
		GizmoViewContext->GetViewDirection();

	constexpr double DotThreshold = 0.2;
	const bool bAxisPerpendicularToView = FMath::Abs(FVector::DotProduct(InPlaneNormal, ViewDirection)) < DotThreshold;
	const bool bRayPerpendicularToAxis = FMath::IsNearlyZero(FVector::DotProduct(InPlaneNormal, Ray.Direction));

	// Can we project?
	const bool bCanProjectOnPlane = !bAxisPerpendicularToView && !bRayPerpendicularToAxis;

	FVector HitPoint = FVector::ZeroVector;
	double HitAngle = 0.0;

	const bool bDidProject = ComputeAngleInPlaneFromScreen(GizmoViewContext, InPressPos.ScreenPosition, InteractionPlane, HitPoint, HitAngle);
	if (bDidProject)
	{
		InteractionPlanarCurrPoint = InteractionPlanarStartPoint = HitPoint;
		InteractionCurrAngle = InteractionStartAngle = HitAngle;

		AlignedInteractionAngleOffset = 0.0; // No offset in this case

		// Update Debug
		if (GizmoLocals::DoDebugDraw())
		{
			DebugData.TransformStart.SetLocation(InteractionPlanarOrigin);
			DebugData.InteractionStart.SetLocation(HitPoint);
			DebugData.InteractionAngleStart = InteractionStartAngle;
			DebugData.InteractionAngleCurrent = InteractionCurrAngle;
			DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionStart.GetLocation());
			DebugData.InteractionPlaneNormal = InPlaneNormal;
			DebugData.Test = HitPoint;
		}
	}

	// Pull or ScreenArc need to map Screen->World Aligned Plane
	if (DefaultRotateMode != EAxisRotateMode::Arc)
	{
		ComputeAngleInPlaneFromScreen(GizmoViewContext, InPressPos.ScreenPosition, AlignedInteractionPlane, HitPoint, HitAngle);

		AlignedInteractionAngleOffset = InteractionStartAngle - HitAngle;
	}

	// Log StartAngle, InteractionStartAngle, AlignedInteractionAngleOffset
	UE_CLOGF(bLogRotation, LogEditorTRSGizmo, Verbose, "[%lli] StartAngle: %.1f | AlignedStartAngle: %.1f | Delta: %.1f",
		GFrameCounter,
		FMath::RadiansToDegrees(InteractionStartAngle),
		FMath::RadiansToDegrees(HitAngle),
		FMath::RadiansToDegrees(AlignedInteractionAngleOffset));

	const FVector2D HitCoord = GizmoMath::ComputeCoordinatesInPlane(HitPoint, InteractionPlanarOrigin, InteractionPlanarNormal,
		InteractionPlanarAxisX, InteractionPlanarAxisY);

	InteractionPlanarCurrPoint2D = InteractionPlanarStartPoint2D = HitCoord;
	InteractionPlanarCurrPoint = InteractionPlanarStartPoint = HitPoint;
	InteractionCurrAngle = InteractionStartAngle;
	bInInteraction = true;

	// Update Debug
	if (GizmoLocals::DoDebugDraw())
	{
		DebugData.TransformStart.SetLocation(InteractionPlanarOrigin);
		DebugData.InteractionStart.SetLocation(HitPoint);
		DebugData.InteractionAngleStart = InteractionStartAngle;
		DebugData.InteractionAngleCurrent = InteractionCurrAngle;
		DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionStart.GetLocation());
		DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
	}

	return bCanProjectOnPlane;
}

double UEditorTRSGizmo::ComputeAxisRotateDeltaAngle(const FVector2D& InStartPos, const FInputDeviceRay& InDragPos)
{
	FVector2D DragDir = InDragPos.ScreenPosition - InStartPos;

	if (bTrySwitchingToNormalPull)
	{
		const double DotTangent = FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir);
		const double DotNormal = FVector2D::DotProduct(NormalProjectionToRemove, DragDir);
		if (FMath::Abs(DotNormal) > FMath::Abs(DotTangent))
		{
			::Swap(NormalProjectionToRemove, InteractionScreenAxisDirection);
			::Swap(DebugData.DebugDirection, DebugData.DebugNormalRemoved);
			InteractionScreenAxisDirection *= FMath::Sign(DotTangent) * FMath::Sign(DotNormal);
		}
		bTrySwitchingToNormalPull = false;
	}

	const FVector2D DragDirToRemove = NormalProjectionToRemove * FVector2D::DotProduct(DragDir, NormalProjectionToRemove);
	DragDir -= DragDirToRemove;

	return FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir) * Interaction.RotateInteraction.PullMultiplier;
}

void UEditorTRSGizmo::OnClickDragRotateAxis(const FInputDeviceRay& InDragPos)
{
	Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DragRotateAxisName);

	UGizmoElementRotateAxis* Element = RotateAxisSetElement->GetAxisElement(EAxis::FromAxisList(GetInteractionAxisList()));

	OnClickDragRotateAxisInternal(
		InDragPos,
		Element,
		[this](const double& InDeltaAngle, const double& InAngleSign, const FVector2d&, const FVector2d&)
		{
			const double DeltaAngle = InDeltaAngle * InAngleSign;
			const FQuat Delta = ComputeAxisRotateDelta(FMath::RadiansToDegrees(DeltaAngle));

			return Delta;
		},
		[this](const double& InDeltaAngle, const double& InAngleSign)
		{
			return FQuat(AlignedInteractionPlanarNormal, InDeltaAngle * InAngleSign);
		});
}

void UEditorTRSGizmo::OnClickReleaseRotateAxis(const FInputDeviceRay& InReleasePos)
{
	// End element deltas
	{
		// Update Debug
		if (GizmoLocals::DoDebugDraw()
			&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
		{
			DebugData.LastHitDeltaParts.Reset();
		}

		auto EndRotateAxisDelta = [&](UGizmoElementRotateAxis* InElement)
		{
			if (InElement)
			{
				// Update Debug
				if (GizmoLocals::DoDebugDraw()
					&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
				{
					DebugData.LastHitDeltaParts.Emplace(LastHitPart);
				}
				else
				{
					InElement->EndDelta();
				}
			}
		};

		if (bGimbalRotationMode)
		{
			EndRotateAxisDelta(RotateXGimbalElement2);
			EndRotateAxisDelta(RotateYGimbalElement2);
			EndRotateAxisDelta(RotateZGimbalElement2);
		}
		else
		{
			EndRotateAxisDelta(RotateAxisSetElement->GetAxisElement(EAxis::X));
			EndRotateAxisDelta(RotateAxisSetElement->GetAxisElement(EAxis::Y));
			EndRotateAxisDelta(RotateAxisSetElement->GetAxisElement(EAxis::Z));
		}
	}

	bInInteraction = false;
	DebugData.bDebugRotate = false;
	bTrySwitchingToNormalPull = false;
}

void UEditorTRSGizmo::OnClickPressGimbalRotateAxis(const FInputDeviceRay& InPressPos)
{
	const int32 RotateID = GetRotateAxisIndexForPart(LastHitPart);
	if (!ensure(RotateID != INDEX_NONE))
	{
		bTrySwitchingToNormalPull = false;
		bInInteraction = false;
		return;
	}

	const EAxis::Type Axis = GetAxisForIndex(RotateID);

	DebugData.bDebugRotate = true;

	UGizmoElementRotateAxis* Element = RotateGimbalElement->GetAxisElement<UGizmoElementRotateAxis>(Axis);
	if (!Element)
	{
		return;
	}

	// Interaction Plane
	{
		// Capture the initial, axis-aligned plane
		const UE::Geometry::FFrame3d Plane = Element->MakePlane(CurrentTransform, GizmoViewContext, GetCoordinateSystem(), GetRotationContext());
		InitializeInteractionPlanes(Plane);

		// If we're doing a screen-space arc, re-initialize the plane
		if (DefaultRotateMode == EAxisRotateMode::ScreenArc)
		{
			InitializeInteractionPlane(EAxisList::Screen);
		}
	}

	ViewToAlignedSign = UE::Math::TIntVector3<int8>(1);
	InitializeInteractionSign();

	// initialize pull data
	InteractionScreenAxisDirection = GetScreenGimbalRotateAxisDir(InPressPos);
	InteractionAxisDirection = GetGimbalRotationAxis(RotateID);
	SetInteractionAxisList(EAxisList::FromAxis(Axis, AxisDisplayInfo::GetAxisDisplayCoordinateSystem()));
	ClockwiseCorrectionSignForAxis = UE::Editor::InteractiveToolsFramework::Internal::GetClockwiseAngleSignForAxis(Axis);
	InteractionScreenStartPos = InteractionScreenCurrPos = InPressPos.ScreenPosition;
	InteractionDelta = FVector::ZeroVector;
	AlignedInteractionAngleOffset = InteractionStartAngle = InteractionCurrAngle = 0.0;
	SetRotateMode(EAxisRotateMode::Pull);

	// initialize arc/mixed data
	const bool bRotatePull = bIndirectManipulation || DefaultRotateMode == EAxisRotateMode::Pull;

	// initialize plane intersection even if we're not in arc mode (for some other required visuals)
	const bool bCanRotateArc = OnClickPressRotateArc(InPressPos, AlignedInteractionPlanarNormal, InteractionPlanarAxisX, InteractionPlanarAxisY, Axis);
	if (!bRotatePull)
	{
		if (bCanRotateArc)
		{
			// Arc or ScreenArc
			SetRotateMode(DefaultRotateMode);
		}
	}

	// Begin element deltas
	if (Element)
	{
		BeginRotateAxisDelta(Element);
	}

	bTrySwitchingToNormalPull = bIndirectManipulation && RotateMode == EAxisRotateMode::Pull;

	bInInteraction = true;
	SetModeLastHitPart(EGizmoTransformMode::Rotate, LastHitPart);
}

void UEditorTRSGizmo::OnClickDragGimbalRotateAxis(const FInputDeviceRay& InDragPos)
{
	Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DragGimbalRotateAxisName);

	UGizmoElementRotateAxis* Element = RotateGimbalElement->GetAxisElement<UGizmoElementRotateAxis>(EAxis::FromAxisList(GetInteractionAxisList()));

	OnClickDragRotateAxisInternal(
		InDragPos,
		Element,
		[this](const double&, const double& InAngleSign, const FVector2d& InScreenPosStart, const FVector2d& InScreenPosEnd)
		{
			return ComputeGimbalRotateDelta(InScreenPosStart, InScreenPosEnd, InAngleSign);
		},
		[this](const double& InDeltaAngle, const double& InAngleSign)
		{
			const double DeltaAngle = FMath::RadiansToDegrees(InDeltaAngle) * InAngleSign;

			const EAxisList::Type InteractedAxisList = GetInteractionAxisList();

			// @todo: we can clean this and it's friends up a bit
			const FVector DeltaRot = FVector(
				InteractedAxisList == EAxisList::X || InteractedAxisList == EAxisList::Forward ? DeltaAngle : 0.0,
				InteractedAxisList == EAxisList::Y || InteractedAxisList == EAxisList::Left ? DeltaAngle : 0.0,
				InteractedAxisList == EAxisList::Z || InteractedAxisList == EAxisList::Up ? DeltaAngle : 0.0);

			// @note: Using UE handyness false here because we've already accounted for it with ClockwiseCorrectionSignForAxis
			constexpr bool bUseUEHandyness = false;
			const FQuat Delta = AnimationCore::QuatFromEuler(DeltaRot * UE::Editor::GizmoMath::GetAxisCoordinateSystemMultiplier(), GetRotationContext().RotationOrder, bUseUEHandyness);

			return Delta;
		});
}

FQuat UEditorTRSGizmo::ComputeGimbalRotateDelta(const FVector2D& InStartPos, const FVector2D& InEndPos, const double InAngleSign)
{
	FVector2D DragDir = InEndPos - InStartPos;

	if (bTrySwitchingToNormalPull)
	{
		const double DotTangent = FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir);
		const double DotNormal = FVector2D::DotProduct(NormalProjectionToRemove, DragDir);
		if (FMath::Abs(DotNormal) > FMath::Abs(DotTangent))
		{
			::Swap(NormalProjectionToRemove, InteractionScreenAxisDirection);
			::Swap(DebugData.DebugDirection, DebugData.DebugNormalRemoved);
			InteractionScreenAxisDirection *= FMath::Sign(DotTangent) * FMath::Sign(DotNormal);
		}
		bTrySwitchingToNormalPull = false;
	}

	const FVector2D DragDirToRemove = NormalProjectionToRemove * FVector2D::DotProduct(DragDir, NormalProjectionToRemove);
	DragDir -= DragDirToRemove;

	const EAxisList::Type InteractedAxisList = GetInteractionAxisList();

	const double Delta = FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir) * Interaction.RotateInteraction.PullMultiplier * InAngleSign;
	const FVector DeltaRot = FVector(
			InteractedAxisList == EAxisList::X || InteractedAxisList == EAxisList::Forward ? Delta : 0.0,
			InteractedAxisList == EAxisList::Y || InteractedAxisList == EAxisList::Left ? Delta : 0.0,
			InteractedAxisList == EAxisList::Z || InteractedAxisList == EAxisList::Up ? Delta : 0.0);

	// @note: Using UE handyness flips the X and Y axis, but we do the opposite with ClockwiseCorrectionSignForAxis before it's applied here.
	// Instead, we don't use UE handyness - we invert the entire input rotation
	constexpr bool bUseUEHandyness = false;
	return AnimationCore::QuatFromEuler(DeltaRot * UE::Editor::GizmoMath::GetAxisCoordinateSystemMultiplier() * -1, GetRotationContext().RotationOrder, bUseUEHandyness);
}

FVector UEditorTRSGizmo::GetGimbalRotationAxis(const int32 InAxis) const
{
	FVector Result = UTransformGizmo::GetGimbalRotationAxis(InAxis);

	if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward && InAxis == 1) // Y is inverted
	{
		Result *= -1.0f;
	}

	// @todo: correct in GizmoRotationUtil!!
	if (InAxis == 2) // Z
	{
		Result *= -1.0f;
	}

	return Result;
}

FVector2D UEditorTRSGizmo::GetScreenGimbalRotateAxisDir(const FInputDeviceRay& InPressPos)
{
	if (Interaction.RotateInteraction.Direction.IsSet())
	{
		// Invert the Y axis, which for the user is 1 = top, but in UE is 1 = bottom
		// Invert the entire result to correct for CCW -> CW
		const FVector2D Direction = (Interaction.RotateInteraction.Direction.GetValue() * FVector2D(1, -1)).GetSafeNormal();
		NormalProjectionToRemove = FVector2D::ZeroVector;
		return Direction;
	}

	FVector WorldAxis = FVector::ZeroVector;
	if(GetRotateAxisVectorForPart(LastHitPart, WorldAxis))
	{
		// Flip the tested plane axis if needed
		WorldAxis *= ViewToAlignedSign.X;
	}

	return GetWorldToScreenRotateAxisDir(InPressPos, WorldAxis);
}

FVector2D UEditorTRSGizmo::GetScreenRotateAxisDir(const FInputDeviceRay& InPressPos)
{
	if (Interaction.RotateInteraction.Direction.IsSet())
	{
		FVector2D Direction = (Interaction.RotateInteraction.Direction.GetValue() * FVector2D(1, -1) * ViewToAlignedSign.X).GetSafeNormal();
		NormalProjectionToRemove = FVector2D::ZeroVector;
		return Direction;
	}

	// store world origin and axis
	FVector WorldAxis = FVector::ZeroVector;
	if (GetRotateAxisVectorForPart(LastHitPart, WorldAxis))
	{
		// Flip the tested plane axis if needed
		WorldAxis *= ViewToAlignedSign.X;
	}

	// Don't account for ViewToAlignedSign here, this already does it
	return GetWorldToScreenRotateAxisDir(InPressPos, WorldAxis);
}

void UEditorTRSGizmo::OnClickPressScreenSpaceRotate(const FInputDeviceRay& InPressPos)
{
	check(GizmoViewContext);

	SetInteractionAxisList(EAxisList::Screen);

	// Interaction Plane
	if (const UGizmoElementRotateAxis* Element = RotateScreenSpaceElement2)
	{
		// Capture the initial, axis-aligned plane
		InitializeInteractionPlanes(Element->MakePlane(CurrentTransform,  GizmoViewContext, GetCoordinateSystem(), GetRotationContext(), EAxisList::All));

		InitializeScreenInteraction(InPressPos);

		// If we're doing a screen-space arc, re-initialize the plane
		if (DefaultRotateMode == EAxisRotateMode::ScreenArc)
		{
			InitializeInteractionPlane(EAxisList::Screen);
		}
	}

	bTrySwitchingToNormalPull = false;
	bInInteraction = false;
	DebugData.bDebugRotate = false;
	ViewToAlignedSign = UE::Math::TIntVector3<int8>(1);
	ClockwiseCorrectionSignForAxis = 1;
	AlignedInteractionAngleOffset = 0.0;
	SetRotateMode(EAxisRotateMode::Pull);

	// initialize arc/mixed data
	const bool bRotatePull = bIndirectManipulation || DefaultRotateMode == EAxisRotateMode::Pull;
	if (!bRotatePull)
	{
		SetRotateMode(DefaultRotateMode);
	}

	// Always perform ray/plane intersection
	FVector HitPoint = FVector::ZeroVector;
	double HitAngle = 0.0;
	if (UE::Editor::GizmoMath::ComputeAngleInPlaneFromScreen(GizmoViewContext, InPressPos.ScreenPosition, InteractionPlane, HitPoint, HitAngle))
	{
		InteractionPlanarCurrPoint = InteractionPlanarStartPoint = HitPoint;
		InteractionCurrAngle = InteractionStartAngle = HitAngle;
	}

	if (bIndirectManipulation)
	{
		InteractionScreenAxisDirection = GetScreenRotateAxisDir(InPressPos);
		InteractionScreenStartPos = InteractionScreenCurrPos = InPressPos.ScreenPosition;
		bTrySwitchingToNormalPull = true;
		DebugData.bDebugRotate = true;
	}

	// Update Debug
	if (GizmoLocals::DoDebugDraw())
	{
		DebugData.InteractionStart.SetLocation(HitPoint);
		DebugData.InteractionAngleStart = InteractionStartAngle;
		DebugData.InteractionAngleCurrent = InteractionCurrAngle;
		DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
		DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
	}

	if (RotateScreenSpaceElement2)
	{
		// Compensate for GizmoElement screen alignment
		const double AngleForDelta = InteractionCurrAngle * -1;

		UGizmoElementRotateAxis::FDeltaParameters DeltaParameters;
		DeltaParameters.bIsIndirectManipulation = bIndirectManipulation;
		DeltaParameters.Transform = CurrentTransform;
		DeltaParameters.RotationContext = GetRotationContext();
		DeltaParameters.Angle = AngleForDelta;
		DeltaParameters.DisplaySign = -1; // Incoming angle is CCW/negative, but for screen-space rotation, treat as CW/positive, so invert
		DeltaParameters.PlaneNormal = InteractionPlanarNormal;
		DeltaParameters.CoordinateSystem = GetCoordinateSystem();
		DeltaParameters.CursorLocation = InteractionPlanarStartPoint;
		DeltaParameters.CursorLocation2D = InteractionPlanarStartPoint2D;
		DeltaParameters.RotateMode = RotateMode;

		RotateScreenSpaceElement2->BeginDelta(DeltaParameters);
	}

	bInInteraction = true;

	if (IsRotationPrecisionMode())
	{
		bRotationPrecisionModeDirty = true;
	}

	ScreenSpacePrecisionVisualOffset = -FMath::FindDeltaAngleRadians(InteractionCurrAngle, HitAngle);

	SetModeLastHitPart(EGizmoTransformMode::Rotate, LastHitPart);
}

void UEditorTRSGizmo::OnClickDragScreenSpaceRotate(const FInputDeviceRay& InDragPos)
{
	Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DragRotateScreenName);

	// Making sure current interaction angle and visual display are correct when switching precision mode On/Off
	// This is required since precision mode has the gizmo lose the 1:1 mapping between cursor movements and actual rotation
	if (bRotationPrecisionModeDirty)
	{
		InteractionScreenAxisDirection = GetScreenRotateAxisDir(InDragPos);
		InteractionScreenCurrPos = InDragPos.ScreenPosition;

		double HitAngle = 0.0;
		FVector HitPoint = FVector::ZeroVector;
		if (UE::Editor::GizmoMath::ComputeAngleInPlaneFromScreen(GizmoViewContext, InDragPos.ScreenPosition, InteractionPlane, HitPoint, HitAngle))
		{
			const double AngleOffset = FMath::FindDeltaAngleRadians(InteractionCurrAngle, HitAngle);

			InteractionCurrAngle += AngleOffset;
			ScreenSpacePrecisionVisualOffset -= AngleOffset;
		}

		bRotationPrecisionModeDirty = false;
	}

	check(GizmoViewContext);

	const double PreviousAngle = InteractionCurrAngle;
	const FVector2D PreviousScreenPos = InteractionScreenCurrPos;
	
	using namespace UE::Editor::GizmoMath;

	// Always perform ray/plane intersection
	FVector HitPoint = FVector::ZeroVector;
	double HitAngle = 0.0;
	bool bHitPlane = ComputeAngleInPlaneFromScreen(GizmoViewContext, InDragPos.ScreenPosition, InteractionPlane, HitPoint, HitAngle);
	if (bHitPlane)
	{
		// Correct to avoid wrapping etc.
		HitAngle = PreviousAngle + FMath::FindDeltaAngleRadians(PreviousAngle, HitAngle);

		InteractionPlanarCurrPoint = HitPoint;
		InteractionCurrAngle = HitAngle;
	}

	if (bIndirectManipulation || IsRotationPrecisionMode())
	{
		FVector2D DragDir = InDragPos.ScreenPosition - PreviousScreenPos;

		if (bTrySwitchingToNormalPull)
		{
			const double DotTangent = FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir);
			const double DotNormal = FVector2D::DotProduct(NormalProjectionToRemove, DragDir);
			if (FMath::Abs(DotNormal) > FMath::Abs(DotTangent))
			{
				::Swap(NormalProjectionToRemove, InteractionScreenAxisDirection);
				::Swap(DebugData.DebugDirection, DebugData.DebugNormalRemoved);
				InteractionScreenAxisDirection *= FMath::Sign(DotTangent) * FMath::Sign(DotNormal);
			}
			bTrySwitchingToNormalPull = false;
		}

		const FVector2D DragDirToRemove = NormalProjectionToRemove * FVector2D::DotProduct(DragDir, NormalProjectionToRemove);
		DragDir -= DragDirToRemove;

		double DeltaAngle = FMath::DegreesToRadians(FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir) * Interaction.RotateInteraction.PullMultiplier);

		if (IsRotationPrecisionMode())
		{
			DeltaAngle *= FMath::Abs(GetRotationPrecisionModeMultiplier());
		}

		SnapRotateAngleDelta(DeltaAngle, EAxisList::X); // Only snapping a single angle, so axis list doesn't matter

		// Reapply the (potentially) snapped delta angle
		HitAngle = PreviousAngle + DeltaAngle;

		const FQuat Delta(InteractionPlanarNormal, DeltaAngle);
		ApplyRotateDelta(Delta);

		CumulativeRotationDelta = CumulativeRotationDelta * Delta;

		InteractionCurrAngle = HitAngle;
		InteractionScreenCurrPos = FMath::IsNearlyZero(DeltaAngle) ? PreviousScreenPos : InDragPos.ScreenPosition;
	}
	else
	{
		if (bHitPlane)
		{
			InteractionScreenCurrPos = InDragPos.ScreenPosition;

			double HitAngleDelta = FMath::FindDeltaAngleRadians(PreviousAngle, HitAngle);
			SnapRotateAngleDelta(HitAngleDelta, EAxisList::X); // Only snapping a single angle, so axis list doesn't matter

			// Reapply the (potentially) snapped delta angle
			HitAngle = PreviousAngle + HitAngleDelta;

			const FQuat Delta = FQuat(InteractionPlanarNormal, HitAngleDelta);
			ApplyRotateDelta(Delta);

			CumulativeRotationDelta = CumulativeRotationDelta * Delta;

			InteractionCurrAngle = HitAngle;

			// Update Debug
			if (GizmoLocals::DoDebugDraw())
			{
				//DebugData.TransformStart.SetLocation(HitPoint);
				DebugData.InteractionCurrent.SetLocation(HitPoint);
				DebugData.InteractionAngleCurrent += FMath::FindDeltaAngleRadians(DebugData.InteractionAngleCurrent, HitAngle);
				DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
			}
		}
	}

	if (RotateScreenSpaceElement2)
	{
		// Compensate for GizmoElement screen alignment
		const double AngleForDelta = (InteractionCurrAngle + ScreenSpacePrecisionVisualOffset) * -1;

		// Re-calculate based on screen plane to avoid plane intersection failures
		if (ComputePointOnPlaneFromScreen(GizmoViewContext, InDragPos.WorldRay, InDragPos.ScreenPosition, ScreenPlane, HitPoint))
		{
			InteractionPlanarCurrPoint = HitPoint;
		}

		UGizmoElementRotateAxis::FDeltaParameters DeltaParameters;
		DeltaParameters.bIsIndirectManipulation = bIndirectManipulation;
		DeltaParameters.Transform = GetActiveTransform();
		DeltaParameters.Angle = AngleForDelta;
		DeltaParameters.CursorLocation = InteractionPlanarCurrPoint;
		DeltaParameters.CursorLocation2D = InteractionPlanarCurrPoint2D;
		DeltaParameters.RotateMode = RotateMode;

		RotateScreenSpaceElement2->UpdateDelta(DeltaParameters);
	}
}

void UEditorTRSGizmo::OnClickReleaseScreenSpaceRotate(const FInputDeviceRay& InReleasePos)
{
	if (RotateScreenSpaceElement2)
	{
		// Update Debug
		if (GizmoLocals::DoDebugDraw()
			&& GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters.Debug.bFreezeDelta)
		{
			DebugData.LastHitDeltaParts.Reset();
			DebugData.LastHitDeltaParts.Emplace(LastHitPart);
		}
		else
		{
			RotateScreenSpaceElement2->EndDelta();
		}
	}

	bInInteraction = false;
	DebugData.bDebugRotate = false;
	bTrySwitchingToNormalPull = false;
}

// @todo: Temporary to avoid naming clash, re-merge later
namespace ArcBallLocals2
{
	/* See Holroyd's implementation that mixes a sphere + and hyperbola to avoid popping
	 * Knud Henriksen, Jon Sporring, and Kasper Hornbaek, “Virtual trackballs revisited”,
	 * IEEE Transactions on Visualization and Computer Graphics, vol. 10, no. 2, pp. 206–216, 2004.
	 */
	bool GetSphereAndHyperbolicProjection(
		const FVector& SphereOrigin,
		const double SphereRadius,
		const FVector& RayOrigin,
		const FVector& RayDirection,
		const UGizmoViewContext& ViewContext,
		FVector& OutProjection)
	{
		const FVector CircleNormal = -ViewContext.GetViewDirection();

		// if ray is parallel to circle, no hit
		if (FMath::IsNearlyZero(FVector::DotProduct(CircleNormal, RayDirection)))
		{
			return false;
		}

		const FPlane Plane(SphereOrigin, CircleNormal);
		const double Param = FMath::RayPlaneIntersectionParam(RayOrigin, RayDirection, Plane);

		if (Param < 0)
		{
			return false;
		}

		const FVector HitPoint = RayOrigin + RayDirection * Param;
		FVector Offset = HitPoint - SphereOrigin;

		// switch to screen space
		FVector OffsetProjection = ViewContext.ViewMatrices.GetInvViewMatrix().InverseTransformVector(Offset);

		const double OffsetSquared = Offset.SizeSquared();
		const double CircleRadiusSquared = SphereRadius * SphereRadius;

		OffsetProjection.Z = (OffsetSquared <= CircleRadiusSquared * 0.5) ?
			-FMath::Sqrt(CircleRadiusSquared - OffsetSquared) : // spherical projection
			-CircleRadiusSquared * 0.5 / Offset.Length(); // hyperbolic projection

		// switch back to world space
		Offset = ViewContext.ViewMatrices.GetInvViewMatrix().TransformVector(OffsetProjection);
		OutProjection = SphereOrigin + Offset;

		return true;
	}
}

void UEditorTRSGizmo::OnClickPressArcBallRotate(const FInputDeviceRay& PressPos)
{
	if (!CanInteractWithPart(ETransformGizmoPartIdentifier::RotateArcball))
	{
		// arc ball rotation is disabled in gimbal mode but is still rendered to ease the visualization
		SetModeLastHitPart(EGizmoTransformMode::Rotate, LastHitPart);
		bInInteraction = false;
		return;
	}

	check(GizmoViewContext);

	const FVector& RayOrigin = PressPos.WorldRay.Origin;
	const FVector& RayDir = PressPos.WorldRay.Direction;
	const float GizmoScale = static_cast<float>(GetGizmoTransform().GetScale3D().X);
	const double SphereRadius = GetWorldRadius(Style.RotateStyle.ArcballStyle.Radius * GizmoScale * Style.GetModifiedAxisSizeMultiplier()) * GetRotationPrecisionModeMultiplier();

	StartRotation = CurrentRotation = CurrentTransform.GetRotation();
	InteractionPlanarOrigin = CurrentTransform.GetLocation();
	InteractionPlanarNormal = -GizmoViewContext->GetViewDirection();
	InteractionPlanarAxisX = GizmoViewContext->GetViewUp();
	InteractionPlanarAxisY = GizmoViewContext->GetViewRight();
	SetInteractionAxisList(EAxisList::XYZ);

	auto NeedsInteraction = [&]()
	{
		const bool bIntersect = IntersectionUtil::RaySphereTest(RayOrigin, RayDir, InteractionPlanarOrigin, SphereRadius);
		if (bIntersect)
		{
			return true;
		}

		if (bIndirectManipulation)
		{
			// change the arc ball center in indirect manipulation if we didn't hit the sphere
			FVector::FReal HitDepth;
			if (GetRayParamIntersectionWithInteractionPlane(PressPos, HitDepth))
			{
				InteractionPlanarOrigin = PressPos.WorldRay.Origin + PressPos.WorldRay.Direction * HitDepth;
				return true;
			}
		}

		return false;
	};

	if (NeedsInteraction())
	{
		// project on sphere
		ArcBallLocals2::GetSphereAndHyperbolicProjection(
			InteractionPlanarOrigin, SphereRadius, RayOrigin, RayDir, *GizmoViewContext,
			InteractionArcBallStartPoint);

		// Update Debug
		if (GizmoLocals::DoDebugDraw())
		{
			DebugData.TransformStart.SetLocation(InteractionPlanarOrigin);
			DebugData.InteractionStart.SetLocation(InteractionArcBallStartPoint);
			DebugData.InteractionCurrent.SetLocation(InteractionArcBallStartPoint);
			DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
			DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
		}

		bInInteraction = true;

		SetModeLastHitPart(EGizmoTransformMode::Rotate, LastHitPart);
	}
}

void UEditorTRSGizmo::OnClickDragArcBallRotate(const FInputDeviceRay& DragPos)
{
	Profiler->BeginScoped(UE::Editor::InteractiveToolsFramework::Private::TransformGizmoLocals::DragRotateArcBallName);

	const FVector& RayOrigin = DragPos.WorldRay.Origin;
	const FVector& RayDir = DragPos.WorldRay.Direction;

	const float GizmoScale = static_cast<float>(GetGizmoTransform().GetScale3D().X);
	const double SphereRadius = GetWorldRadius(Style.RotateStyle.ArcballStyle.Radius * GizmoScale * Style.GetModifiedAxisSizeMultiplier()) * GetRotationPrecisionModeMultiplier();

	// Precision mode: make sure rotation keeps working when entering/leaving precision mode
	if (bRotationPrecisionModeDirty)
	{
		// Reproject cursor onto the new sphere to prevent a rotation jump
		FVector TransitionCurrPoint;
		if (ArcBallLocals2::GetSphereAndHyperbolicProjection(
				InteractionPlanarOrigin, SphereRadius, RayOrigin, RayDir, *GizmoViewContext, TransitionCurrPoint
			))
		{
			InteractionArcBallStartPoint = TransitionCurrPoint;
			StartRotation = CurrentRotation;
		}

		InteractionScreenCurrPos = DragPos.ScreenPosition;
		bRotationPrecisionModeDirty = false;
	}

	// compute projection
	ArcBallLocals2::GetSphereAndHyperbolicProjection(
		InteractionPlanarOrigin, SphereRadius, RayOrigin, RayDir, *GizmoViewContext,
		InteractionArcBallCurrPoint);

	if ((InteractionArcBallCurrPoint-InteractionArcBallStartPoint).Length() <= 0.0)
	{
		return;
	}

	// compute rotation
	const FVector Axis1 = (InteractionArcBallCurrPoint - InteractionPlanarOrigin).GetSafeNormal();
	const FVector Axis0 = (InteractionArcBallStartPoint - InteractionPlanarOrigin).GetSafeNormal();

	const FQuat DeltaQ = FQuat::FindBetweenNormals(Axis0, Axis1);

	// apply rotation
	const FQuat FinalRotation = DeltaQ * StartRotation;
	const FQuat InvCurrentRot = CurrentRotation.Inverse();

	FQuat DeltaRot = (FinalRotation * InvCurrentRot).GetNormalized();
	if (!DeltaRot.IsIdentity())
	{
		SnapRotateDelta(DeltaRot, GetInteractionAxisList());
		ApplyRotateDelta(DeltaRot);
		CumulativeRotationDelta = DeltaRot * CumulativeRotationDelta;
		CurrentRotation = DeltaRot * CurrentRotation;
	}

	// Update Debug
	if (GizmoLocals::DoDebugDraw())
	{
		DebugData.InteractionCurrent.SetLocation(InteractionArcBallCurrPoint);
		DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
		DebugData.InteractionPlaneNormal = FVector::CrossProduct(Axis1, Axis0);
	}
}

void UEditorTRSGizmo::ApplyTranslate(const FVector& InTranslate)
{
	ApplyTranslateDelta(InTranslate - CurrentTransform.GetLocation());
}

void UEditorTRSGizmo::FGizmoElementInteraction::FPartSet::ForEachPart(const TFunctionRef<void(const ETransformGizmoPartIdentifier)>& InFunc)
{
	for (const ETransformGizmoPartIdentifier PartId : Parts)
	{
		InFunc(PartId);
	}

	if (PartRange.IsSet())
	{
		FGizmoElementInteraction::FPartSet::FPartRange Range = PartRange.GetValue();
		if (Range.Exclude.IsSet())
		{
			const ETransformGizmoPartIdentifier PartToExcludeFromRange = Range.Exclude.GetValue();
			for (const ETransformGizmoPartIdentifier PartId : TEnumRange<ETransformGizmoPartIdentifier>(Range.Start, Range.End))
			{
				if (PartId == PartToExcludeFromRange)
				{
					continue;
				}

				InFunc(PartId);
			}
		}
		else
		{
			for (const ETransformGizmoPartIdentifier PartId : TEnumRange<ETransformGizmoPartIdentifier>(Range.Start, Range.End))
			{
				InFunc(PartId);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
