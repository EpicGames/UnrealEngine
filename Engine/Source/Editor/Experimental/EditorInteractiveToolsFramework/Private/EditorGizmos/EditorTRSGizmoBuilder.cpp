// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTRSGizmoBuilder.h"

#include "BaseGizmos/GizmoElementGroup.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "ContextObjectStore.h"
#include "EditorGizmoElementHitMultiTarget.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.h"
#include "EditorGizmos/EditorTransformGizmoSource.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorGizmos/EditorTransformProxy.h"
#include "EditorModeManager.h"
#include "TransformGizmoEditorSettings.h"
#include "EditorGizmos/EditorTRSGizmo.h"
#include "Tools/AssetEditorContextInterface.h"

UInteractiveGizmo* UEditorTRSGizmoBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	if (!GetDefault<UTransformGizmoEditorSettings>()->UsesNewTRSGizmo())
	{
		// Fall back to the old implementation
		return Super::BuildGizmo(SceneState);
	}

	UGizmoViewContext* GizmoViewContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGizmoViewContext>();
	check(GizmoViewContext && GizmoViewContext->IsValidLowLevel());

	IToolsContextQueriesAPI* QueriesAPI = SceneState.GizmoManager->GetContextQueriesAPI();
	ensureAlways(QueriesAPI);

	UEditorTRSGizmo* TransformGizmo = NewObject<UEditorTRSGizmo>(SceneState.GizmoManager);
	TransformGizmo->SetCustomizationFunction(CustomizationFunction);
	TransformGizmo->Setup();
	TransformGizmo->TransformGizmoSource = UEditorTransformGizmoSource::CreateNew(TransformGizmo, GetTransformGizmoContext(SceneState));
	TransformGizmo->TransformAdjuster = MakeShared<UE::Editor::InteractiveToolsFramework::Internal::FTransformGizmoScaleAdjuster>(QueriesAPI, TransformGizmo->TransformGizmoSource);
	TransformGizmo->GizmoViewContext = GizmoViewContext;
	// TransformGizmo->AssetEditorContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>();

	// A UGizmoElementHitMultiTarget will be constructed and both the
	// render and hit target's Construct methods will take the gizmo element root as input.
	TransformGizmo->HitTarget = UEditorGizmoElementHitMultiTarget::Construct(TransformGizmo->GizmoElementRoot, GizmoViewContext);
	TransformGizmo->HitTarget->SetTransformAdjuster(TransformGizmo->TransformAdjuster);

	return TransformGizmo;
}

void UEditorTRSGizmoBuilder::UpdateGizmoForSelection(UInteractiveGizmo* Gizmo, const FToolBuilderState& SceneState)
{
	if (!GetDefault<UTransformGizmoEditorSettings>()->UsesNewTRSGizmo())
	{
		// Fall back to the old implementation
		Super::UpdateGizmoForSelection(Gizmo, SceneState);
		return;
	}

	if (UTransformGizmo* TransformGizmo = Cast<UTransformGizmo>(Gizmo))
	{
		// UE_LOGF(LogTemp, Warning, "UEditorTransformGizmoBuilder::UpdateGizmoForSelection");

		const UEditorTransformGizmoContextObject* GizmoContextObject = GetTransformGizmoContext(SceneState);
		FEditorModeTools* ModeTools = GizmoContextObject ? GizmoContextObject->GetModeTools() : nullptr;
		ensure(ModeTools);

		// Used to retrieve type elements
		IAssetEditorContextInterface* AssetEditorContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>();

		UEditorTransformProxy* TransformProxy = UEditorTransformProxy::CreateNew(GizmoContextObject, AssetEditorContext);
		TransformProxy->SetUseLegacyWidgetScale(false);
		
		TransformGizmo->SetActiveTarget(TransformProxy, nullptr, ModeTools->GetGizmoStateTarget());
		TransformGizmo->SetVisibility(true);

		if (UEditorGizmoElementHitMultiTarget* HitMultiTarget = Cast< UEditorGizmoElementHitMultiTarget>(TransformGizmo->HitTarget))
		{
			HitMultiTarget->GizmoTransformProxy = TransformProxy;
		}
	}
}
