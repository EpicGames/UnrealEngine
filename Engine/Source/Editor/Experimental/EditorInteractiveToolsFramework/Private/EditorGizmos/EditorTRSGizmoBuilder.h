// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EditorInteractiveGizmoSelectionBuilder.h"
#include "InteractiveGizmoBuilder.h"
#include "EditorGizmos/EditorTransformGizmoBuilder.h"

#include "EditorTRSGizmoBuilder.generated.h"

class UEditorTRSGizmo;
class UEditorTransformGizmoContextObject;
struct FGizmoCustomization;

/**
 * Gizmo builder for the TRS (Translate/Rotate/Scale) transform gizmo.
 * Creates and configures UEditorTRSGizmo instances based on the current scene selection.
 */
UCLASS(MinimalAPI)
class UEditorTRSGizmoBuilder : public UEditorTransformGizmoBuilder
{
	GENERATED_BODY()

public:
	//~ Begin UEditorInteractiveGizmoSelectionBuilder Interface

	/** Creates a new TRS gizmo instance configured for the current scene state. */
	virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;

	/** Updates the given TRS gizmo to reflect changes in the current scene selection. */
	virtual void UpdateGizmoForSelection(UInteractiveGizmo* Gizmo, const FToolBuilderState& SceneState) override;

	//~ End UEditorInteractiveGizmoSelectionBuilder Interface
};
