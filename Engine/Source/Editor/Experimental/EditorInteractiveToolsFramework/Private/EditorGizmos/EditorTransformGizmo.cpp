// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/EditorTransformGizmo.h"

#include "Containers/EnumAsByte.h"
#include "EditorGizmos/EditorTransformProxy.h"
#include "Logging/LogMacros.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorTransformGizmo)

#define LOCTEXT_NAMESPACE "UEditorTransformGizmo"

DEFINE_LOG_CATEGORY_STATIC(LogEditorTransformGizmo, Log, All);

void UEditorTransformGizmo::Render(IToolsContextRenderAPI* RenderAPI)
{
	const FSceneView* SceneView = RenderAPI ? RenderAPI->GetSceneView() : nullptr;
	const FSceneViewFamily* SceneViewFamily = SceneView ? SceneView->Family : nullptr;
	if (!SceneViewFamily)
	{
		return;
	}

	const bool bEngineShowFlagsModeWidget = SceneViewFamily->EngineShowFlags.ModeWidgets && !SceneViewFamily->EngineShowFlags.Game;
	if (bEngineShowFlagsModeWidget)
	{
		Super::Render(RenderAPI);
	}
}

void UEditorTransformGizmo::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	const FSceneView* SceneView = RenderAPI ? RenderAPI->GetSceneView() : nullptr;
	const FSceneViewFamily* SceneViewFamily = SceneView ? SceneView->Family : nullptr;
	if (!SceneViewFamily)
	{
		return;
	}

	const bool bEngineShowFlagsModeWidget = SceneViewFamily->EngineShowFlags.ModeWidgets && !SceneViewFamily->EngineShowFlags.Game;
	if (bEngineShowFlagsModeWidget)
	{
		Super::DrawHUD(Canvas, RenderAPI);
	}
}

void UEditorTransformGizmo::ApplyTranslateDelta(const FVector& InTranslateDelta)
{
	check(ActiveTarget);

	if (UEditorTransformProxy* EditorTransformProxy = Cast<UEditorTransformProxy>(ActiveTarget))
	{
		EditorTransformProxy->InputTranslateDelta(InTranslateDelta, GetInteractionAxisList());

		// Update the cached current transform
		CurrentTransform.AddToTranslation(InTranslateDelta);
	}
	else
	{
		Super::ApplyTranslateDelta(InTranslateDelta);
	}
}

void UEditorTransformGizmo::ApplyRotateDelta(const FQuat& InRotateDelta)
{
	check(ActiveTarget);

	if (UEditorTransformProxy* EditorTransformProxy = Cast<UEditorTransformProxy>(ActiveTarget))
	{
		// Update the cached current delta.
		// Applies rot delta after the current rotation.
		FQuat NewRotation = InRotateDelta * CurrentTransform.GetRotation();
		CurrentTransform.SetRotation(NewRotation);

		EditorTransformProxy->InputRotateDelta(InRotateDelta.Rotator(), GetInteractionAxisList());
	}
	else
	{
		Super::ApplyRotateDelta(InRotateDelta);
	}
}

void UEditorTransformGizmo::ApplyScaleDelta(const FVector& InScaleDelta)
{
	check(ActiveTarget);

	if (UEditorTransformProxy* EditorTransformProxy = Cast<UEditorTransformProxy>(ActiveTarget))
	{
		FVector StartScale = CurrentTransform.GetScale3D();

		EditorTransformProxy->InputScaleDelta(InScaleDelta, GetInteractionAxisList());

		// Update the cached current transform
		FVector NewScale = StartScale + InScaleDelta;
		CurrentTransform.SetScale3D(NewScale);
	}
	else
	{
		Super::ApplyScaleDelta(InScaleDelta);
	}
}

void UEditorTransformGizmo::SetActiveTarget(UTransformProxy* Target, IToolContextTransactionProvider* TransactionProvider, IGizmoStateTarget* InStateTarget)
{
	Super::SetActiveTarget(Target, TransactionProvider, InStateTarget);

	Target->OnBeginTransformEdit.AddUObject(this, &UEditorTransformGizmo::OnGizmoTransformBegin);
	Target->OnEndTransformEdit.AddUObject(this, &UEditorTransformGizmo::OnGizmoTransformEnd);
}

void UEditorTransformGizmo::OnGizmoTransformBegin(UTransformProxy* InTransformProxy) const
{
	const UEditorTransformProxy* EditorTransformProxy = Cast<UEditorTransformProxy>(ActiveTarget);
	if (InTransformProxy && InTransformProxy == EditorTransformProxy)
	{
		// FIXME this is need for some modes (including IKRig) but has side effects for now
		// has it sets dragging to true. I'll fix this in a next CL
		
		// Set legacy widget axis temporarily because FEditorViewportClient overrides may expect it
		// EditorTransformProxy->SetCurrentAxis(InteractionAxisList);
	}
}

void UEditorTransformGizmo::OnGizmoTransformEnd(UTransformProxy* InTransformProxy) const
{
	const UEditorTransformProxy* EditorTransformProxy = Cast<UEditorTransformProxy>(ActiveTarget);
	if (InTransformProxy && InTransformProxy == EditorTransformProxy)
	{
		EditorTransformProxy->SetCurrentAxis(EAxisList::None);
	}
}

#undef LOCTEXT_NAMESPACE
