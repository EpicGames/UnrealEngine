// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/EditorTransformGizmoBuilder.h"

#include "BaseGizmos/GizmoElementGroup.h"
#include "BaseGizmos/GizmoElementHitTargets.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "EditorGizmos/EditorTransformGizmo.h"
#include "EditorGizmos/EditorTransformGizmoSource.h"
#include "EditorGizmos/EditorTransformProxy.h"
#include "EditorGizmos/TransformGizmo.h"
#include "InteractiveGizmo.h"
#include "InteractiveGizmoManager.h"
#include "Templates/Casts.h"
#include "ToolContextInterfaces.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptInterface.h"
#include "Tools/AssetEditorContextInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorTransformGizmoBuilder)

UInteractiveGizmo* UEditorTransformGizmoBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	UGizmoViewContext* GizmoViewContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGizmoViewContext>();
	check(GizmoViewContext && GizmoViewContext->IsValidLowLevel());

	UEditorTransformGizmo* TransformGizmo = NewObject<UEditorTransformGizmo>(SceneState.GizmoManager);
	TransformGizmo->SetCustomizationFunction(CustomizationFunction);
	TransformGizmo->Setup();
	TransformGizmo->TransformGizmoSource = UEditorTransformGizmoSource::CreateNew(TransformGizmo, GetTransformGizmoContext(SceneState));
	TransformGizmo->GizmoViewContext = GizmoViewContext;

	// @todo: Gizmo element construction to be moved here from UTransformGizmo.
	// A UGizmoElementRenderMultiTarget will be constructed and both the
	// render and hit target's Construct methods will take the gizmo element root as input.
	TransformGizmo->HitTarget = UGizmoElementHitMultiTarget::Construct(TransformGizmo->GizmoElementRoot, GizmoViewContext);

	return TransformGizmo;
}

void UEditorTransformGizmoBuilder::UpdateGizmoForSelection(UInteractiveGizmo* Gizmo, const FToolBuilderState& SceneState)
{
	if (UTransformGizmo* TransformGizmo = Cast<UTransformGizmo>(Gizmo))
	{
		const UEditorTransformGizmoContextObject* GizmoContextObject = GetTransformGizmoContext(SceneState);

		FEditorModeTools* ModeTools = GizmoContextObject ? GizmoContextObject->GetModeTools() : nullptr;
		ensure(ModeTools);

		IAssetEditorContextInterface* AssetEditorContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>();

		UEditorTransformProxy* TransformProxy = UEditorTransformProxy::CreateNew(GizmoContextObject, AssetEditorContext);
		TransformGizmo->SetActiveTarget(TransformProxy, nullptr, ModeTools->GetGizmoStateTarget());
		TransformGizmo->SetVisibility(true);
		
		if (UGizmoElementHitMultiTarget* HitMultiTarget = Cast< UGizmoElementHitMultiTarget>(TransformGizmo->HitTarget))
		{
			HitMultiTarget->GizmoTransformProxy = TransformProxy;
		}
	}
}

const UEditorTransformGizmoContextObject* UEditorTransformGizmoBuilder::GetTransformGizmoContext(const FToolBuilderState& InSceneState)
{
	const UContextObjectStore* ContextStore = InSceneState.GizmoManager ? InSceneState.GizmoManager->GetContextObjectStore() : nullptr;
	return ContextStore ? ContextStore->FindContext<UEditorTransformGizmoContextObject>() : nullptr;
}
