// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditorGizmos/EditorTransformGizmoDataBinder.h"

#include "EditorModeManager.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorGizmos/TransformGizmo.h"
#include "EditorViewportClient.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"

FEditorTransformGizmoDataBinder::~FEditorTransformGizmoDataBinder()
{
	const TSet<TWeakObjectPtr<UTransformGizmo>> Gizmos(BoundGizmos);
	for (const TWeakObjectPtr<UTransformGizmo> Gizmo : Gizmos)
	{
		if (Gizmo.IsValid())
		{
			UnbindFromGizmo(Gizmo.Get(), Gizmo->ActiveTarget);
			Gizmo->OnSetActiveTarget.RemoveAll(this);
			Gizmo->OnAboutToClearActiveTarget.RemoveAll(this);
		}
	}
	BoundGizmos.Empty();

	if (WeakContext.IsValid())
	{
		WeakContext->OnGizmoCreatedDelegate().RemoveAll(this);
	}
	WeakContext.Reset();
}

void FEditorTransformGizmoDataBinder::BindToGizmoContextObject(UEditorTransformGizmoContextObject* InContextObject)
{
	if (!ensure(IsValid(InContextObject)))
	{
		return;
	}

	InContextObject->OnGizmoCreatedDelegate().AddSP(this, &FEditorTransformGizmoDataBinder::BindToUninitializedGizmo);
	
	WeakContext = InContextObject;
}

void FEditorTransformGizmoDataBinder::BindToUninitializedGizmo(UTransformGizmo* InGizmo)
{
	FEditorModeTools* ModeTools = WeakContext.IsValid() ? WeakContext->GetModeTools() : nullptr;
	if (ensure(ModeTools))
	{
		if (!ModeTools->OnWidgetModeChanged().IsBoundToObject(InGizmo))
		{
			ModeTools->OnWidgetModeChanged().AddUObject(InGizmo, &UTransformGizmo::HandleWidgetModeChanged);
		}
	}

	const bool bSetBound = InGizmo->OnSetActiveTarget.IsBoundToObject(this);
	const bool bClearBound = InGizmo->OnAboutToClearActiveTarget.IsBoundToObject(this);
	if ( ensure(!bSetBound && !bClearBound))
	{
		InGizmo->OnSetActiveTarget.AddSP(this, &FEditorTransformGizmoDataBinder::BindToInitializedGizmo);
		if (InGizmo->ActiveTarget)
		{
			BindToInitializedGizmo(InGizmo, InGizmo->ActiveTarget);
		}
		InGizmo->OnAboutToClearActiveTarget.AddSP(this, &FEditorTransformGizmoDataBinder::UnbindFromGizmo);
	}
}

void FEditorTransformGizmoDataBinder::BindToInitializedGizmo(UTransformGizmo* InGizmo, UTransformProxy* InProxy)
{
	const bool bHaveValidInput = InGizmo && InGizmo->ActiveTarget && InGizmo->ActiveTarget == InProxy;
	if (!ensure(bHaveValidInput && !BoundGizmos.Contains(InGizmo)))
	{
		return;
	}

	// bind InProxy OnBeginTransformEdit/OnTransformChanged/OnEndTransformEdit etc here if needed
	if (!InProxy->OnBeginTransformEdit.IsBoundToObject(this))
	{
		InProxy->OnBeginTransformEdit.AddSP(this, &FEditorTransformGizmoDataBinder::OnProxyBeginTransformEdit);
	}

	if (!InProxy->OnTransformChanged.IsBoundToObject(this))
	{
		InProxy->OnTransformChanged.AddSP(this, &FEditorTransformGizmoDataBinder::OnProxyTransformChanged);
	}
	
	if (!InProxy->OnEndTransformEdit.IsBoundToObject(this))
	{
		InProxy->OnEndTransformEdit.AddSP(this, &FEditorTransformGizmoDataBinder::OnProxyEndTransformEdit);
	}
	
	if (!InProxy->OnBeginPivotEdit.IsBoundToObject(this))
	{
		InProxy->OnBeginPivotEdit.AddSP(this, &FEditorTransformGizmoDataBinder::OnProxyBeginPivotEdit);
	}
	
	if (!InProxy->OnPivotChanged.IsBoundToObject(this))
	{
		InProxy->OnPivotChanged.AddSP(this, &FEditorTransformGizmoDataBinder::OnProxyPivotChanged);
	}
	
	if (!InProxy->OnEndPivotEdit.IsBoundToObject(this))
	{
		InProxy->OnEndPivotEdit.AddSP(this, &FEditorTransformGizmoDataBinder::OnProxyEndPivotEdit);
	}
	
	BoundGizmos.Add(InGizmo);
}

void FEditorTransformGizmoDataBinder::UnbindFromGizmo(UTransformGizmo* InGizmo, UTransformProxy* InProxy)
{
	if (!InGizmo)
	{
		return;
	}

	FEditorModeTools* ModeTools = WeakContext.IsValid() ? WeakContext->GetModeTools() : nullptr;
	if (ensure(ModeTools))
	{
		ModeTools->OnWidgetModeChanged().RemoveAll(InGizmo);
	}

	// unbind InProxy OnBeginTransformEdit/OnTransformChanged/OnEndTransformEdit etc here if needed
	InProxy->OnBeginTransformEdit.RemoveAll(this);
	InProxy->OnEndTransformEdit.RemoveAll(this);
	
	InProxy->OnBeginPivotEdit.RemoveAll(this);
	InProxy->OnPivotChanged.RemoveAll(this);
	InProxy->OnEndPivotEdit.RemoveAll(this);
	
	BoundGizmos.Remove(InGizmo);
}

FEditorViewportClient* FEditorTransformGizmoDataBinder::GetViewportClient() const
{
	if (WeakContext.IsValid() && WeakContext->GetModeTools())
	{
		return WeakContext->GetModeTools()->GetFocusedViewportClient();	
	}
	return GLevelEditorModeTools().GetFocusedViewportClient();
}

void FEditorTransformGizmoDataBinder::OnProxyBeginTransformEdit(UTransformProxy* InTransformProxy)
{
	bHasTransformChanged = false;
}

void FEditorTransformGizmoDataBinder::OnProxyTransformChanged(UTransformProxy* InTransformProxy, FTransform InTransform)
{
	// TODO track if we actually did drag something to fallback to the viewport if we didn't
	bHasTransformChanged = true;
}

void FEditorTransformGizmoDataBinder::OnProxyEndTransformEdit(UTransformProxy* InTransformProxy)
{
	bHasTransformChanged = false;
}

void FEditorTransformGizmoDataBinder::OnProxyBeginPivotEdit(UTransformProxy* InTransformProxy)
{
	if (UEditorTransformGizmoContextObject* Context = WeakContext.Get())
	{
		Context->bIsManipulatingPivot = true;
	}
}

void FEditorTransformGizmoDataBinder::OnProxyPivotChanged(UTransformProxy* InTransformProxy, FTransform InTransform)
{
	
}

void FEditorTransformGizmoDataBinder::OnProxyEndPivotEdit(UTransformProxy* InTransformProxy)
{
	if (UEditorTransformGizmoContextObject* Context = WeakContext.Get())
	{
		Context->bIsManipulatingPivot = false;
	}
}
