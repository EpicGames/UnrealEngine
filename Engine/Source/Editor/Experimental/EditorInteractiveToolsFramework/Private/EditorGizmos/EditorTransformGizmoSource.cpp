// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/EditorTransformGizmoSource.h"

#include "Editor.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "TransformGizmoEditorSettings.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorTransformGizmoSource)

EGizmoTransformMode FEditorTransformGizmoUtil::GetGizmoMode(UE::Widget::EWidgetMode InWidgetMode)
{
	switch (InWidgetMode)
	{
		case UE::Widget::EWidgetMode::WM_Translate: return EGizmoTransformMode::Translate;
		case UE::Widget::EWidgetMode::WM_Rotate: return EGizmoTransformMode::Rotate;
		case UE::Widget::EWidgetMode::WM_Scale: return EGizmoTransformMode::Scale;
		default: return EGizmoTransformMode::None;
	}
}

UE::Widget::EWidgetMode FEditorTransformGizmoUtil::GetWidgetMode(EGizmoTransformMode InGizmoMode)
{
	switch (InGizmoMode)
	{
		case EGizmoTransformMode::Translate: return UE::Widget::EWidgetMode::WM_Translate;
		case EGizmoTransformMode::Rotate: return UE::Widget::EWidgetMode::WM_Rotate;
		case EGizmoTransformMode::Scale: return UE::Widget::EWidgetMode::WM_Scale;
		default: return UE::Widget::EWidgetMode::WM_None;
	}
}

EGizmoTransformMode UEditorTransformGizmoSource::GetGizmoMode() const
{
	if (const FEditorViewportClient* ViewportClient = GetViewportClient())
	{
		const UE::Widget::EWidgetMode WidgetMode = ViewportClient->GetWidgetMode();
		return FEditorTransformGizmoUtil::GetGizmoMode(WidgetMode);
	}
	return EGizmoTransformMode::None;
}

EAxisList::Type UEditorTransformGizmoSource::GetGizmoAxisToDraw(EGizmoTransformMode InGizmoMode) const
{ 
	if (const FEditorViewportClient* ViewportClient = GetViewportClient())
	{
		const UE::Widget::EWidgetMode WidgetMode = ViewportClient->GetWidgetMode();
		return GetModeTools().GetWidgetAxisToDraw(WidgetMode);
	}
	return EAxisList::None;
}

EToolContextCoordinateSystem UEditorTransformGizmoSource::GetGizmoCoordSystemSpace() const
{
	const FEditorViewportClient* ViewportClient = GetViewportClient();
	const ECoordSystem Space = ViewportClient ? ViewportClient->GetWidgetCoordSystemSpace() : COORD_World;
	return Space == COORD_World ? EToolContextCoordinateSystem::World : EToolContextCoordinateSystem::Local;
}

float UEditorTransformGizmoSource::GetGizmoScale() const
{
	return GetModeTools().GetWidgetScale();
}

bool UEditorTransformGizmoSource::GetVisible(const EViewportContext InViewportContext) const
{
	if (const FEditorViewportClient* ViewportClient = GetViewportClient(InViewportContext)) 
	{
		if (!ViewportClient->GetShowWidget())
		{
			return false;
		}

		const bool bEngineShowFlagsModeWidget = ViewportClient->EngineShowFlags.ModeWidgets;
		if (!bEngineShowFlagsModeWidget)
		{
			return false;
		}

		return CanInteract(InViewportContext);
	}
	return false;
}

bool UEditorTransformGizmoSource::CanInteract(const EViewportContext InViewportContext) const
{
	if (const FEditorViewportClient* ViewportClient = GetViewportClient(InViewportContext))
	{
		const FEditorModeTools& ModeTools = GetModeTools();
		if (ModeTools.GetShowWidget() && ModeTools.UsesTransformWidget())
		{
			const UE::Widget::EWidgetMode WidgetMode = ViewportClient->GetWidgetMode();
			bool bUseLegacyWidget = (WidgetMode == UE::Widget::WM_TranslateRotateZ || WidgetMode == UE::Widget::WM_2D);
			if (!bUseLegacyWidget)
			{
				bUseLegacyWidget = !UEditorInteractiveGizmoManager::AreEditorGizmosAllowed(&ModeTools);
			}
			return !bUseLegacyWidget;
		}
	}
	return false;
}

EGizmoTransformScaleType UEditorTransformGizmoSource::GetScaleType() const
{
	EGizmoTransformScaleType ResolvedScaleType = FGizmoElementScaleInteraction::GetDefaultScaleType();
	if (const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>())
	{
		ResolvedScaleType = Settings->GizmosParameters.ScaleType;
	}
	
	if (ResolvedScaleType != EGizmoTransformScaleType::Legacy)
	{
		return ResolvedScaleType;
	}

	if (GEditor->UsePercentageBasedScaling())
	{
		return EGizmoTransformScaleType::PercentageBased;
	}

	return EGizmoTransformScaleType::Default;
}

const FRotationContext& UEditorTransformGizmoSource::GetRotationContext() const
{
	static const FRotationContext DefaultContext;
	return WeakContext.IsValid() ? WeakContext->RotationContext : DefaultContext;
}

UEditorTransformGizmoSource* UEditorTransformGizmoSource::CreateNew(UObject* Outer, const UEditorTransformGizmoContextObject* InContext)
{
	UEditorTransformGizmoSource* NewSource = NewObject<UEditorTransformGizmoSource>(Outer);
	NewSource->WeakContext = InContext;
	return NewSource;
}

const FEditorModeTools& UEditorTransformGizmoSource::GetModeTools() const
{
	if (WeakContext.IsValid())
	{
		return *WeakContext->GetModeTools();
	}
	return GLevelEditorModeTools();
}

const FEditorViewportClient* UEditorTransformGizmoSource::GetViewportClient(const EViewportContext InViewportContext) const
{
	if (InViewportContext == EViewportContext::Hovered)
	{
		if (FEditorViewportClient* Hovered = GetModeTools().GetHoveredViewportClient())
		{
			return Hovered;
		}
	}
	return GetModeTools().GetFocusedViewportClient();
}
