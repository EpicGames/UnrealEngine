// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/EditorTransformGizmoUtil.h"

#include "ContextObjectStore.h"
#include "EditorInteractiveGizmoSubsystem.h"
#include "EditorInteractiveGizmoManager.h"
#include "InteractiveToolsContext.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "EditorGizmos/TransformGizmo.h"
#include "EditorGizmos/EditorTransformGizmoBuilder.h"
#include "EditorGizmos/EditorTransformGizmoDataBinder.h"
#include "EditorViewportClient.h"
#include "EditorModes.h"
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "Modules/ModuleManager.h"
#include "Tools/DefaultEdMode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorTransformGizmoUtil)

bool UE::EditorTransformGizmoUtil::RegisterTransformGizmoContextObject(FEditorModeTools* InModeTools)
{
	const UModeManagerInteractiveToolsContext* ToolsContext = InModeTools->GetInteractiveToolsContext();
	if (ensure(ToolsContext))
	{
		const UEditorTransformGizmoContextObject* Found = ToolsContext->ContextObjectStore->FindContext<UEditorTransformGizmoContextObject>();
		if (Found)
		{
			return true;
		}

		UEditorTransformGizmoContextObject* GizmoContextObject = NewObject<UEditorTransformGizmoContextObject>(ToolsContext->ToolManager);
		if (ensure(GizmoContextObject))
		{
			GizmoContextObject->RegisterGizmosWithManager(ToolsContext->ToolManager);
			GizmoContextObject->Initialize(InModeTools);
			return true;
		}
	}
	return false;
}

bool UE::EditorTransformGizmoUtil::UnregisterTransformGizmoContextObject(FEditorModeTools* InModeTools)
{
	const UModeManagerInteractiveToolsContext* ToolsContext = InModeTools->GetInteractiveToolsContext();
	if (ensure(ToolsContext))
	{
		UEditorTransformGizmoContextObject* Found = ToolsContext->ContextObjectStore->FindContext<UEditorTransformGizmoContextObject>();
		if (Found != nullptr)
		{
			Found->Shutdown();
			Found->UnregisterGizmosWithManager(ToolsContext->ToolManager);
		}
		return true;
	}
	return false;
}

UTransformGizmo* UE::EditorTransformGizmoUtil::CreateTransformGizmo(
	UInteractiveToolManager* InToolManager, const FString& InInstanceIdentifier, void* InOwner)
{
	if (!InToolManager)
	{
		return nullptr;
	}
	
	UEditorInteractiveGizmoManager* GizmoManager = Cast<UEditorInteractiveGizmoManager>(InToolManager->GetPairedGizmoManager());
	if (!GizmoManager)
	{
		return nullptr;
	}

	const UEditorTransformGizmoContextObject* Found = GizmoManager->GetContextObjectStore()->FindContext<UEditorTransformGizmoContextObject>();
	if (!Found)
	{
		return nullptr;
	}

	return Found->CreateTransformGizmo(GizmoManager, InInstanceIdentifier, InOwner);
}

UTransformGizmo* UE::EditorTransformGizmoUtil::GetDefaultTransformGizmo(UInteractiveToolManager* InToolManager)
{
	return CreateTransformGizmo(InToolManager, UEditorInteractiveGizmoManager::TransformInstanceIdentifier(), nullptr);
}

UTransformGizmo* UE::EditorTransformGizmoUtil::FindDefaultTransformGizmo(UInteractiveToolManager* InToolManager)
{
	if (!InToolManager)
	{
		return nullptr;
	}
	
	const UEditorInteractiveGizmoManager* GizmoManager = Cast<UEditorInteractiveGizmoManager>(InToolManager->GetPairedGizmoManager());
	if (!GizmoManager)
	{
		return nullptr;
	}

	UInteractiveGizmo* Found = GizmoManager->FindGizmoByInstanceIdentifier(UEditorInteractiveGizmoManager::TransformInstanceIdentifier());
	return Cast<UTransformGizmo>(Found);
}

void UE::EditorTransformGizmoUtil::RemoveDefaultTransformGizmo(UInteractiveToolManager* InToolManager)
{
	if (!InToolManager)
	{
		return;
	}
	
	UEditorInteractiveGizmoManager* GizmoManager = Cast<UEditorInteractiveGizmoManager>(InToolManager->GetPairedGizmoManager());
	if (!GizmoManager)
	{
		return;
	}

	UInteractiveGizmo* Found = GizmoManager->FindGizmoByInstanceIdentifier(UEditorInteractiveGizmoManager::TransformInstanceIdentifier());
	if (Found)
	{
		GizmoManager->DestroyGizmo(Found);
	}
}

void UEditorTransformGizmoContextObject::RegisterGizmosWithManager(UInteractiveToolManager* InToolManager)
{
	if (ensure(!bGizmosRegistered) == false)
	{
		return;
	}

	InToolManager->GetContextObjectStore()->AddContextObject(this);
	
	UEditorInteractiveGizmoManager* GizmoManager = Cast<UEditorInteractiveGizmoManager>(InToolManager->GetPairedGizmoManager());
	if (ensure(GizmoManager))
	{
		if (UEditorInteractiveGizmoSubsystem* GizmoSubsystem = GEditor->GetEditorSubsystem<UEditorInteractiveGizmoSubsystem>())
		{
			GizmoManager->RegisterGizmoType(UEditorInteractiveGizmoManager::TransformBuilderIdentifier(), GizmoSubsystem->GetTransformGizmoBuilder());
		}
		else
		{
			UEditorTransformGizmoBuilder* GizmoBuilder = NewObject<UEditorTransformGizmoBuilder>(this);
			GizmoManager->RegisterGizmoType(UEditorInteractiveGizmoManager::TransformBuilderIdentifier(), GizmoBuilder);	
		}
	}
	
	bGizmosRegistered = true;
}

void UEditorTransformGizmoContextObject::UnregisterGizmosWithManager(UInteractiveToolManager* InToolManager)
{
	ensure(bGizmosRegistered);
	
	InToolManager->GetContextObjectStore()->RemoveContextObject(this);

	UInteractiveGizmoManager* GizmoManager = InToolManager->GetPairedGizmoManager();
	GizmoManager->DeregisterGizmoType(UEditorInteractiveGizmoManager::TransformBuilderIdentifier());
	
	bGizmosRegistered = false;
}

void UEditorTransformGizmoContextObject::Initialize(FEditorModeTools* InModeTools)
{
	if (ensure(InModeTools && !ModeTools))
	{
		ModeTools = InModeTools;
		InitializeGizmoManagerBinding();

		DataBinder = MakeShared<FEditorTransformGizmoDataBinder>();
		DataBinder->BindToGizmoContextObject(this);
	}
}

void UEditorTransformGizmoContextObject::Shutdown()
{
	if (ensure(DataBinder))
	{
		DataBinder.Reset();
	}
	
	if (ensure(ModeTools))
	{
		UpdateGizmo({});
		RemoveViewportsBinding();
		RemoveGizmoManagerBinding();
		
		ModeTools = nullptr;
	}
}

UTransformGizmo* UEditorTransformGizmoContextObject::CreateTransformGizmo(
	UEditorInteractiveGizmoManager* InGizmoManager, const FString& InInstanceIdentifier, void* InOwner) const
{
	if (!InGizmoManager)
	{
		return nullptr;
	}
	
	UTransformGizmo* NewGizmo = Cast<UTransformGizmo>( InGizmoManager->CreateGizmo(
		UEditorInteractiveGizmoManager::TransformBuilderIdentifier(), InInstanceIdentifier, InOwner) );

	if (ensure(NewGizmo))
	{
		OnGizmoCreated.Broadcast(NewGizmo);
		InGizmoManager->OnGizmosParametersChangedDelegate().AddUObject(NewGizmo, &UTransformGizmo::OnParametersChanged);
		if (InGizmoManager->GetDefaultGizmosParameters())
		{
			NewGizmo->OnParametersChanged(*InGizmoManager->GetDefaultGizmosParameters());
		}
	}
	
	return NewGizmo;
}

FEditorModeTools* UEditorTransformGizmoContextObject::GetModeTools() const
{
	return ModeTools;
}

UEditorTransformGizmoContextObject::FOnGizmoCreated& UEditorTransformGizmoContextObject::OnGizmoCreatedDelegate()
{
	return OnGizmoCreated;
}

void UEditorTransformGizmoContextObject::UpdateGizmo(const TArray<FEditorViewportClient*>& InViewportClients) const
{
	if (!ModeTools)
	{
		return;
	}

	// get viewports pointing at ModeTools
	const TArray<FEditorViewportClient*> ViewportClients = InViewportClients.FilterByPredicate([this](const FEditorViewportClient* ViewportClient)
	{
		return ViewportClient && ViewportClient->GetModeTools() == ModeTools;
	});

	// remove useless gizmo
	if (ViewportClients.IsEmpty())
	{
		const UModeManagerInteractiveToolsContext* ToolsContext = ModeTools->GetInteractiveToolsContext();
		const TObjectPtr<UInteractiveToolManager> ToolManager = ToolsContext->ToolManager;
		UE::EditorTransformGizmoUtil::RemoveDefaultTransformGizmo(ToolManager);
		return;
	}

	// if the focus viewport has been deleted, switch to the first visible one if any
	// FIXME, use the hover one instead ?
	const bool bIsFocusStillAvailable = ViewportClients.Contains(ModeTools->GetFocusedViewportClient());
	if (!bIsFocusStillAvailable)
	{
		const int32 Index = ViewportClients.IndexOfByPredicate([](const FEditorViewportClient* ViewportClient)
		{
			return ViewportClient->IsVisible();
		});
		if (Index != INDEX_NONE)
		{
			ModeTools->ReceivedFocus(ViewportClients[Index], ViewportClients[Index]->Viewport);
		}
	}
	
	const UModeManagerInteractiveToolsContext* ToolsContext = ModeTools->GetInteractiveToolsContext();
	const TObjectPtr<UInteractiveToolManager> ToolManager = ToolsContext->ToolManager;
	
	if (UE::EditorTransformGizmoUtil::FindDefaultTransformGizmo(ToolManager))
	{
		return;
	}

	// create a new one
	if (UTransformGizmo* TransformGizmo = UE::EditorTransformGizmoUtil::GetDefaultTransformGizmo(ToolManager))
	{
		TransformGizmo->SetVisibility(!ModeTools->IsModeActive(FBuiltinEditorModes::EM_Default) ? ModeTools->UsesViewportITF() : false);
	}
}

void UEditorTransformGizmoContextObject::SetGizmosEnabled(bool bInEnabled)
{
	if (!bInEnabled)
	{
		// reset the coordinate system if the legacy gizmo doesn't support it
		if (ModeTools->GetCoordSystem() > COORD_Local)
		{
			ModeTools->SetCoordSystem(COORD_Local);
		}
			
		// remove viewports' binding + gizmos as they are useless
		const UModeManagerInteractiveToolsContext* ToolsContext = ModeTools->GetInteractiveToolsContext();
		const TObjectPtr<UInteractiveToolManager> ToolManager = ToolsContext->ToolManager;
		UE::EditorTransformGizmoUtil::RemoveDefaultTransformGizmo(ToolManager);
			
		RemoveViewportsBinding();
			
		return;
	}

	InitializeViewportsBinding();
}

void UEditorTransformGizmoContextObject::InitializeGizmoManagerBinding()
{
	auto OnGizmoVariableChanged = [this](const bool bUseNewTRSGizmos)
	{
		bool bEnableGizmos = bUseNewTRSGizmos;
	
		if (UModeManagerInteractiveToolsContext* ToolsContext = ModeTools->GetInteractiveToolsContext())
		{
			if (UEditorInteractiveGizmoManager* GizmoManager = Cast<UEditorInteractiveGizmoManager>(ToolsContext->GizmoManager))
			{
				// When the full context of the manager is available, rely on its state.
				// The setting to allow `NewTRSGizmo` may be on while a particular mode manager has disabled them.
				bEnableGizmos = GizmoManager->AreEditorGizmosAllowed();
			}
		}
		
		SetGizmosEnabled(bEnableGizmos);
	};

	// bind UseNewGizmosChangedHandled if needed
	if (!UseNewGizmosChangedHandled.IsValid())
	{
		UseNewGizmosChangedHandled = UEditorInteractiveGizmoManager::OnUsesNewTRSGizmosChangedDelegate().AddLambda(OnGizmoVariableChanged);
	}

	// initialize default viewport list change binding
	OnGizmoVariableChanged(UEditorInteractiveGizmoManager::UsesNewTRSGizmos());

	// bind GizmosParametersChangedHandle if needed
	auto OnGizmosParametersChanged = [this](const FGizmosParameters& InParameters)
	{
		if (!InParameters.bEnableExplicit)
		{
			// reset the coordinate system if explicit mode is not supported
			if (ModeTools->GetCoordSystem() == COORD_Explicit)
			{
				ModeTools->SetCoordSystem(COORD_Local);
			}
		}
	};
	
	if (!GizmosParametersChangedHandle.IsValid())
	{
		GizmosParametersChangedHandle = UEditorInteractiveGizmoManager::OnGizmosParametersChangedDelegate().AddLambda(OnGizmosParametersChanged);
	}
	
	if (UEditorInteractiveGizmoManager::GetDefaultGizmosParameters())
	{
		OnGizmosParametersChanged(*UEditorInteractiveGizmoManager::GetDefaultGizmosParameters());
	}
}

void UEditorTransformGizmoContextObject::InitializeViewportsBinding()
{
	if (GEditor)
	{
		// check added/removed viewports to ensure that ModeTools' FocusedViewportClient is still safe to use
		auto OnViewportClientsChanged = [this]()
		{
			const TArray<FEditorViewportClient*> ViewportClients =
				GEditor->GetAllViewportClients().FilterByPredicate([this](const FEditorViewportClient* ViewportClient)
				{
					return ViewportClient && ViewportClient->GetModeTools() == ModeTools;
				});

			UpdateGizmo(ViewportClients);
		};

		// bind ViewportClientsChangedHandle if needed
        if (!ViewportClientsChangedHandle.IsValid())
        {
        	ViewportClientsChangedHandle = GEditor->OnViewportClientListChanged().AddLambda(OnViewportClientsChanged);
        }
    
		// initialize
        OnViewportClientsChanged();

		auto OnLevelViewportClientsChanged = [this]()
		{
			if (!ModeTools)
			{
				return;
			}

			const bool bIsLevelViewportClient =
			GEditor->GetLevelViewportClients().ContainsByPredicate([this](const FLevelEditorViewportClient* ViewportClient)
			{
				return ViewportClient && ViewportClient->GetModeTools() == ModeTools;
			});

			if (bIsLevelViewportClient)
			{
				// ensure level default mode
				ModeTools->RemoveDefaultMode(FAssetEdModes::EM_AssetDefault);
				ModeTools->AddDefaultMode(FBuiltinEditorModes::EM_Default);

				if (ModeTools->IsModeActive(FAssetEdModes::EM_AssetDefault))
				{
					ModeTools->DeactivateMode(FAssetEdModes::EM_AssetDefault);
					ModeTools->ActivateMode(FBuiltinEditorModes::EM_Default);
				}
			}
		};

		if (!LevelViewportClientsChangedHandle.IsValid())
		{
			LevelViewportClientsChangedHandle = GEditor->OnLevelViewportClientListChanged().AddLambda(OnLevelViewportClientsChanged);
		}
		OnLevelViewportClientsChanged();
	}
}

void UEditorTransformGizmoContextObject::RemoveGizmoManagerBinding()
{
	if (UseNewGizmosChangedHandled.IsValid())
	{
		UEditorInteractiveGizmoManager::OnUsesNewTRSGizmosChangedDelegate().Remove(UseNewGizmosChangedHandled);
		UseNewGizmosChangedHandled.Reset();
	}
	
	if (GizmosParametersChangedHandle.IsValid())
	{
		UEditorInteractiveGizmoManager::OnGizmosParametersChangedDelegate().Remove(GizmosParametersChangedHandle);
		GizmosParametersChangedHandle.Reset();
	}
}

void UEditorTransformGizmoContextObject::RemoveViewportsBinding()
{
	if (ViewportClientsChangedHandle.IsValid())
	{
		if (ensure(GEditor))
		{
			GEditor->OnViewportClientListChanged().Remove(ViewportClientsChangedHandle);
		}
		ViewportClientsChangedHandle.Reset();
	}

	if (LevelViewportClientsChangedHandle.IsValid())
	{
		if (ensure(GEditor))
		{
			GEditor->OnLevelViewportClientListChanged().Remove(LevelViewportClientsChangedHandle);
		}
		LevelViewportClientsChangedHandle.Reset();
	}
}
