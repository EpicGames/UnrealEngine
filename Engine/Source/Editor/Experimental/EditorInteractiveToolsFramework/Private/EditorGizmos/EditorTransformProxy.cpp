// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/EditorTransformProxy.h"

#include "Editor.h"
#include "EditorGizmos/EditorTransformGizmoSource.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementViewportInteraction.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"
#include "Math/Vector.h"
#include "Subsystems/EditorElementSubsystem.h"
#include "Tools/AssetEditorContextInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorTransformProxy)


#define LOCTEXT_NAMESPACE "UEditorTransformProxy"

UEditorTransformProxy::UEditorTransformProxy()
{
	ViewportInteraction = NewObject<UTypedElementViewportInteraction>();
}

FTransform UEditorTransformProxy::GetTransform() const
{
	if (const FEditorViewportClient* ViewportClient = GetViewportClient())
	{
		const FQuat Rotation(ViewportClient->GetWidgetCoordSystem());
		const FVector Location = ViewportClient->GetWidgetLocation();
		const FVector Scale(
			bUseLegacyWidgetScale
			? FVector(WeakContext->GetModeTools()->GetWidgetScale())
			: SharedTransform.GetScale3D());
		return FTransform(Rotation, Location, Scale);
	}
	
	return FTransform::Identity;
}

void UEditorTransformProxy::BeginTransformEditSequence()
{
	Super::BeginTransformEditSequence();

	if (!bUseLegacyWidgetScale)
	{
		UpdateSharedTransform();
	}

	InitialSharedTransform = SharedTransform;
}

void UEditorTransformProxy::InputTranslateDelta(const FVector& InDeltaTranslate, EAxisList::Type InAxisList)
{
	if (InDeltaTranslate.IsNearlyZero(UE_DOUBLE_SMALL_NUMBER))
	{
		return;
	}

	if (FEditorViewportClient* ViewportClient = GetViewportClient())
	{
		FVector Translate = InDeltaTranslate;
		FRotator Rot = FRotator::ZeroRotator;
		FVector Scale = FVector::ZeroVector;

		// Set legacy widget axis temporarily because InputWidgetDelta branches/overrides may expect it
		ViewportClient->SetCurrentWidgetAxis(InAxisList);
		ViewportClient->InputWidgetDelta(ViewportClient->Viewport, InAxisList, Translate, Rot, Scale);
		OnTransformChanged.Broadcast(this, FTransform(Rot, Translate, Scale));
		ViewportClient->SetCurrentWidgetAxis(EAxisList::None);
	}
}

void UEditorTransformProxy::InputScaleDelta(const FVector& InDeltaScale, EAxisList::Type InAxisList)
{
	if (InDeltaScale.IsNearlyZero(UE_DOUBLE_SMALL_NUMBER))
	{
		return;
	}

	if (FEditorViewportClient* ViewportClient = GetViewportClient())
	{
		FVector Translate = FVector::ZeroVector;
		FRotator Rot = FRotator::ZeroRotator;
		FVector Scale = InDeltaScale;

		// Set legacy widget axis temporarily because InputWidgetDelta validates the axis in some crashes and crashes if it is not set
		ViewportClient->SetCurrentWidgetAxis(InAxisList);
		ViewportClient->InputWidgetDelta(ViewportClient->Viewport, InAxisList, Translate, Rot, Scale);
		OnTransformChanged.Broadcast(this, FTransform(Rot, Translate, Scale));
		ViewportClient->SetCurrentWidgetAxis(EAxisList::None);

		if (!bUseLegacyWidgetScale)
		{
			// @todo: This is a workaround for the fact that the scale is not applied to the SharedTransform in InputWidgetDelta.
			UpdateSharedTransform();
		}
	}
}

void UEditorTransformProxy::InputRotateDelta(const FRotator& InDeltaRotate, EAxisList::Type InAxisList)
{
	if (InDeltaRotate.IsNearlyZero(UE_DOUBLE_SMALL_NUMBER))
	{
		return;
	}

	if (FEditorViewportClient* ViewportClient = GetViewportClient())
	{
		FVector Translate = FVector::ZeroVector;
		FRotator Rot = InDeltaRotate;
		FVector Scale = FVector::ZeroVector;

		// Set legacy widget axis temporarily because InputWidgetDelta branches/overrides may expect it
		ViewportClient->SetCurrentWidgetAxis(InAxisList);
		ViewportClient->InputWidgetDelta(ViewportClient->Viewport, InAxisList, Translate, Rot, Scale);
		OnTransformChanged.Broadcast(this, FTransform(Rot, Translate, Scale));
		ViewportClient->SetCurrentWidgetAxis(EAxisList::None);
	}
}

UEditorTransformProxy* UEditorTransformProxy::CreateNew(const UEditorTransformGizmoContextObject* InContext, IAssetEditorContextInterface* InAssetEditorContext)
{
	UEditorTransformProxy* NewProxy = NewObject<UEditorTransformProxy>();
	NewProxy->WeakContext = InContext;
	NewProxy->WeakAssetEditorContext = InAssetEditorContext;

	if (!NewProxy->bUseLegacyWidgetScale)
	{
		NewProxy->UpdateSharedTransform();
		NewProxy->InitialSharedTransform = NewProxy->SharedTransform;
	}

	return NewProxy;
}

UEditorTransformProxy* UEditorTransformProxy::CreateNew(const UEditorTransformGizmoContextObject* InContext)
{
	return CreateNew(InContext, nullptr);
}

void UEditorTransformProxy::UpdateSharedTransform()
{
	const FTypedElementListConstRef ElementsToManipulate = GetElementsToManipulate();

	FVector ElementScale = FVector::OneVector;
	ViewportInteraction->GetScaleForGizmo(ElementsToManipulate, ElementScale);

	SharedTransform.SetScale3D(ElementScale);
}

FTypedElementListConstRef UEditorTransformProxy::GetElementsToManipulate() const
{
	if (IAssetEditorContextInterface* StrongAssetEditorContext = WeakAssetEditorContext.Get())
	{
		// @see: FLevelEditorViewportClient::CacheElementsToManipulate
		FTypedElementListRef ElementsToManipulate = UEditorElementSubsystem::GetEditorNormalizedSelectionSet(*StrongAssetEditorContext->GetSelectionSet());
		if (const FEditorViewportClient* ViewportClient = GetViewportClient())
		{
			const UE::Widget::EWidgetMode WidgetMode = ViewportClient->GetWidgetMode();
			const bool bIsSimulateInEditorViewport = ViewportClient->IsSimulateInEditorViewport();

			UEditorElementSubsystem::GetEditorManipulableElements(ElementsToManipulate, WidgetMode, bIsSimulateInEditorViewport ? GEditor->PlayWorld : GEditor->EditorWorld);
		}

		return ElementsToManipulate;
	}

	FTypedElementListConstRef EmptyElements = UTypedElementRegistry::GetInstance()->CreateElementList();
	return EmptyElements;
}

FEditorViewportClient* UEditorTransformProxy::GetViewportClient() const
{
	if (WeakContext.IsValid() && WeakContext->GetModeTools())
	{
		return WeakContext->GetModeTools()->GetFocusedViewportClient();	
	}
	return GLevelEditorModeTools().GetFocusedViewportClient();
}

void UEditorTransformProxy::SetCurrentAxis(const EAxisList::Type InAxisList) const
{
	if (FEditorViewportClient* ViewportClient = GetViewportClient())
	{
		ViewportClient->SetCurrentWidgetAxis(InAxisList);
	}
}

void UEditorTransformProxy::SetUseLegacyWidgetScale(const bool bInUseLegacyWidgetScale)
{
	if (bUseLegacyWidgetScale != bInUseLegacyWidgetScale)
	{
		bUseLegacyWidgetScale = bInUseLegacyWidgetScale;

		if (!bUseLegacyWidgetScale)
		{
			UpdateSharedTransform();	
		}
	}
}

#undef LOCTEXT_NAMESPACE
