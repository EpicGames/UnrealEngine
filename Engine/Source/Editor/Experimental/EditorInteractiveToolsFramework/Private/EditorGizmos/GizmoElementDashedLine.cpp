// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementDashedLine.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoUtil.h"
#include "Math/Ray.h"
#include "Math/UnrealMathUtility.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementDashedLine)

void UGizmoElementDashedLine::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (!ensure(RenderAPI))
	{
		return;
	}

	FRenderTraversalState CurrentRenderState(RenderState);
	const bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Base, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		if (const UMaterialInterface* UseMaterial = CurrentRenderState.GetCurrentMaterial())
		{
			if (FMath::IsNearlyZero(CurrentRenderState.PixelToWorldScale))
			{
				return;
			}

			const double ScaledLength = Length / CurrentRenderState.PixelToWorldScale;

			// Line is too small to draw
			if (FMath::IsNearlyZero(ScaledLength))
			{
				return;
			}

			const FQuat Rotation = FRotationMatrix::MakeFromZ(Direction).ToQuat();
			const double HalfLength = ScaledLength * 0.5;
			const FVector OriginOffset = Direction * HalfLength;

			const FTransform RenderLocalToWorldTransform = FTransform(Rotation, OriginOffset)
														 * CurrentRenderState.LocalToWorldTransform;

			// Applies flatten scale in local space (so it doesn't affect translation)
			FMatrix RenderLocalToWorldMatrix = RenderLocalToWorldTransform.ToMatrixWithScale();
			RenderLocalToWorldMatrix = RenderLocalToWorldMatrix * CurrentRenderState.SafeFlattenMatrix();
			RenderLocalToWorldMatrix.SetOrigin(RenderLocalToWorldTransform.GetTranslation()); // Restore origin

			const FVector WorldOrigin = CurrentRenderState.LocalToWorldTransform.GetLocation();
			const FVector WorldDirection = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Direction);
			const double WorldLength = ScaledLength * CurrentRenderState.LocalToWorldTransform.GetScale3D().X;

			// Get screen-space endpoints (used for visibility and for reparameterization, both world and screen-space)
			FVector ScreenStartPoint, ScreenEndPoint;
			double LineFractionInView = 1.0;
			double WorldStartFraction = 0.0;

			GizmoMath::GetScreenSpaceLineStartEnd(
				RenderAPI->GetSceneView(), WorldOrigin, WorldDirection, WorldLength, ScreenStartPoint, ScreenEndPoint, LineFractionInView, WorldStartFraction
			);

			GizmoMath::FDashLineSettings DashLineSettings;
			DashLineSettings.MaterialInterface = UseMaterial;
			DashLineSettings.Radius = GetLineThickness();
			DashLineSettings.Color = CurrentRenderState.GetCurrentVertexColor().ToFColor(true);
			DashLineSettings.RenderLocalToWorldMatrix = RenderLocalToWorldMatrix;
			DashLineSettings.DashGapLength = DashGapLength;
			DashLineSettings.DashLength = DashLength;
			DashLineSettings.ScreenStartPoint = ScreenStartPoint;
			DashLineSettings.ScreenEndPoint = ScreenEndPoint;
			DashLineSettings.ScaledLength = ScaledLength;
			DashLineSettings.LineFractionInView = LineFractionInView;
			DashLineSettings.WorldStartFraction = WorldStartFraction;
			DashLineSettings.NumSides = 3;

			if (bScreenSpaceLine)
			{
				// Screen-space dashed: need to reparameterize screen fractions -> world fractions and scale radii per segment, only used in screen-space path
				DashLineSettings.TotalLength = FVector2D::Distance(FVector2D(ScreenStartPoint), FVector2D(ScreenEndPoint));
				DashLineSettings.ParamToWorldFunction =
					[StartDepth = ScreenStartPoint.Z, EndDepth = ScreenEndPoint.Z](const double& InParam)
				{
					// Converts a screen-space fraction (0-1) to a world-space fraction (0-1), so that, for example,
					// the mid-point will appear to correctly to the viewer even if it's not the actual world-space mid-point.

					// Clamp screen fraction to normalized range
					const double ClampedParam = FMath::Clamp(InParam, 0.0, 1.0);

					return (ClampedParam * StartDepth) / (((1.0 - ClampedParam) * EndDepth) + (ClampedParam * StartDepth));
				};

				DashLineSettings.ParamToRadiusScaleFunction =
					[StartDepth = ScreenStartPoint.Z, EndDepth = ScreenEndPoint.Z](const double& InParam)
				{
					return GizmoMath::GetPixelToWorldScaleAtScreenFraction(StartDepth, EndDepth, InParam);
				};

				GizmoMath::BuildDashedLine(RenderAPI->GetPrimitiveDrawInterface(), DashLineSettings);
			}
			else
			{
				DashLineSettings.TotalLength = ScaledLength * LineFractionInView;
				DashLineSettings.ParamToWorldFunction = [](const double& InParam) { return InParam; }; // Linear mapping in world-space
				DashLineSettings.ParamToRadiusScaleFunction = [](const double&) { return 1.0; }; // Constant radius in world-space

				GizmoMath::BuildDashedLine(RenderAPI->GetPrimitiveDrawInterface(), DashLineSettings);
			}
		}
	}
}

FInputRayHit UGizmoElementDashedLine::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	const bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Base, CurrentLineTraceState);

	if (bHittableViewDependent)
	{
		const double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;

		const FVector WorldStart = CurrentLineTraceState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector);
		const FVector LocalEnd = Direction * Length;
		const FVector WorldEnd = CurrentLineTraceState.LocalToWorldTransform.TransformPosition(LocalEnd);

		const FRay Ray(RayOrigin, RayDirection);
		const FVector RayB = Ray.Origin + Ray.Direction * 100000.0f; // arbitrary large distance, to avoid numerical issues

		FVector HitPointA = Ray.Origin;
		FVector HitPointB = WorldStart;
		FMath::SegmentDistToSegmentSafe(Ray.Origin, RayB, WorldStart, WorldEnd, HitPointA, HitPointB);
		if (HitPointA.Equals(Ray.Origin))
		{
			// Not hit
			return FInputRayHit();
		}

		const double HitDepth = FVector::Distance(HitPointA, HitPointB);
		if (HitDepth < PixelHitThresholdAdjust)
		{
			return MakeRayHit(HitDepth, OutLineTraceOutput);
		}
	}

	return FInputRayHit();
}

void UGizmoElementDashedLine::SetBase(FVector InBase)
{
	Base = InBase;
}

FVector UGizmoElementDashedLine::GetBase() const
{
	return Base;
}

void UGizmoElementDashedLine::SetLength(const float& InLength)
{
	Length = InLength;
}

float UGizmoElementDashedLine::GetLength() const
{
	return Length;
}

void UGizmoElementDashedLine::SetDirection(const FVector& InDirection)
{
	Direction = InDirection.GetSafeNormal();
}

FVector UGizmoElementDashedLine::GetDirection() const
{
	return Direction;
}

void UGizmoElementDashedLine::SetDashParameters(const float InDashLength, const TOptional<float>& InGapLength)
{
	DashLength = InDashLength;
	DashGapLength = InGapLength.Get(InDashLength);
}

void UGizmoElementDashedLine::GetDashParameters(float& OutDashLength, float& OutGapLength) const
{
	OutDashLength = DashLength;
	OutGapLength = DashGapLength;
}
