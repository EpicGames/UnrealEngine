// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "BaseGizmos/GizmoElementLineBase.h"

#include "GizmoElementDashedLine.generated.h"

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a dashed line based on parameters.
 */
UCLASS()
class UGizmoElementDashedLine
	: public UGizmoElementLineBase
{
	GENERATED_BODY()

public:
	//~ Begin UGizmoElementBase Interface.
	virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	/** Set the world-space base (start) location of the line. */
	virtual void SetBase(FVector InBase);

	/** Get the world-space base (start) location of the line. */
	virtual FVector GetBase() const;

	/** Set the length of the line in world units. */
    virtual void SetLength(const float& InLength);

	/** Get the length of the line in world units. */
    virtual float GetLength() const;

	/** Set the direction of the line (unit vector). */
	virtual void SetDirection(const FVector& InDirection);

	/** Get the direction of the line. */
	virtual FVector GetDirection() const;

	/**
	 * Set the dash pattern parameters.
	 * @param InDashLength Length of each dash segment in world units.
	 * @param InGapLength Length of each gap between dashes. Defaults to half of InDashLength if not specified.
	 */
	virtual void SetDashParameters(const float InDashLength = 10.0f, const TOptional<float>& InGapLength = TOptional<float>());

	/** Get the current dash and gap lengths. */
	virtual void GetDashParameters(float& OutDashLength, float& OutGapLength) const;

private:
	/** World-space base (start) location of the line. */
	UPROPERTY(Getter, Setter)
	FVector Base = FVector::ZeroVector;

	/** Length of the line in world units. */
	UPROPERTY(Getter, Setter)
	float Length = 100.0f;

	/** Direction of the line (unit vector). */
	UPROPERTY(Getter, Setter)
	FVector Direction = FVector(0.0f, 0.0f, 1.0f);

	/** Length of each dash segment in world units. */
	UPROPERTY()
	float DashLength = 10.0f;

	/** Length of each gap between dashes in world units. */
	UPROPERTY()
	float DashGapLength = 5.0f;
};
