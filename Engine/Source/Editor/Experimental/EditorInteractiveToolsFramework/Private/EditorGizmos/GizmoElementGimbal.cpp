// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/GizmoElementGimbal.h"

#include "EditorGizmos/GizmoRotationUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementGimbal)

void UGizmoElementGimbal::Render(IToolsContextRenderAPI* InRenderAPI, const FRenderTraversalState& InRenderState)
{
	using namespace UE::GizmoRotationUtil;
	
	if (Elements.Num() != 3)
	{
		return;
	}
	
	FRenderTraversalState CurrentRenderState(InRenderState);
	const bool bVisible = UpdateRenderState(InRenderAPI, FVector::ZeroVector, CurrentRenderState);
	if (!bVisible)
	{
		return;
	}

	// decompose rotations
	FRotationDecomposition Decomposition;
	DecomposeRotations(CurrentRenderState.LocalToWorldTransform, RotationContext, Decomposition);
	
	int32 Index = 0;
	for (UGizmoElementBase* Element : Elements)
	{
		CurrentRenderState.LocalToWorldTransform.SetRotation(Decomposition.R[Index++]);

		if (Element)
		{
			Element->Render(InRenderAPI, CurrentRenderState);
		}
	}
}

FInputRayHit UGizmoElementGimbal::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	using namespace UE::GizmoRotationUtil;
	
	FInputRayHit Hit;
	if (Elements.Num() != 3)
	{
		return Hit;
	}

	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	const bool bHittable = UpdateLineTraceState(ViewContext, FVector::ZeroVector, CurrentLineTraceState);

	if (!bHittable)
	{
		return Hit;
	}
	
	// decompose rotations
    FRotationDecomposition Decomposition;
    DecomposeRotations(CurrentLineTraceState.LocalToWorldTransform, RotationContext, Decomposition);

	int32 Index = 0;
	for (UGizmoElementBase* Element : Elements)
	{
		CurrentLineTraceState.LocalToWorldTransform.SetRotation(Decomposition.R[Index++]);
		if (Element)
		{
			FLineTraceOutput NewLineTraceOutput;
			FInputRayHit NewHit = Element->LineTrace(ViewContext, CurrentLineTraceState, RayOrigin, RayDirection, NewLineTraceOutput);
			if (!Hit.bHit || NewHit.HitDepth < Hit.HitDepth)
			{
				OutLineTraceOutput = NewLineTraceOutput;
				Hit = NewHit;
			}
		}
	}
	
	return Hit;
}

void UGizmoElementGimbal::Add(UGizmoElementBase* InElement)
{
	if (Elements.Num() < 3)
	{
		UGizmoElementGroup::Add(InElement);
	}
}
