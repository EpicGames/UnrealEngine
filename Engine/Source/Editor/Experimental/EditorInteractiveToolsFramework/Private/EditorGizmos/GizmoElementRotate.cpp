// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/GizmoElementRotate.h"

#include "TransformGizmoEditorSettings.h"

FGizmoElementRotateAxisStyle::FGizmoElementRotateAxisStyle()
{
	PixelHitDistanceThreshold = 10.0f;
}

FGizmoElementRotateArcballStyle::FGizmoElementRotateArcballStyle()
{
	Colors.Default = FLinearColor(0.9f, 0.9f, 0.9f, 0.1f);
	LineColors.Default = (Colors.Default.GetValue() * 0.25f).CopyWithNewOpacity(1.0f);
	PixelHitDistanceThreshold = 15.0f;
}

EAxisRotateMode::Type FGizmoElementRotateInteraction::GetRotateMode() const
{
	if (RotateMode.IsSet())
	{
		return RotateMode.GetValue();
	}

	if (const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>())
	{
		return Settings->GizmosParameters.RotateMode;
	}

	return GetDefaultRotateMode();
}
