// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/GizmoElementRotateAxis.h"

#include "BaseGizmos/GizmoElementArc.h"
#include "BaseGizmos/GizmoElementCircle.h"
#include "BaseGizmos/GizmoElementCylinder.h"
#include "BaseGizmos/GizmoElementLineStrip.h"
#include "BaseGizmos/GizmoElementTorus.h"
#include "CircleTypes.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.inl"
#include "EditorGizmos/EditorGizmoMath.h"
#include "EditorGizmos/GizmoElementWidget.h"
#include "EditorGizmos/GizmoRotationUtil.h"
#include "GizmoElementDashedLine.h"
#include "GizmoElementValueWidget.h"
#include "Math/UnitConversion.h"
#include "ToolDataVisualizer.h"

DEFINE_LOG_CATEGORY_STATIC(LogGizmoElementRotateAxis, Log, All);

namespace UE::Editor::InteractiveToolsFramework::Private
{
	namespace GizmoElementRotateAxisLocals
	{
		constexpr bool bLog = true;

		FVector2d GetTextOffsetFromCircle(const Geometry::FCircle2d& InCircle, const double InAngleOnCircle, const FBox2d& InTextBox)
		{
			FVector2d::FReal SinTheta = 0.0;
			FVector2d::FReal CosTheta = 0.0;
			FMath::SinCos(&SinTheta, &CosTheta, InAngleOnCircle);

			const FVector2d TextBoxSizeHalf = InTextBox.GetExtent();

			// @note: we switch TextBox axis below - we need this relative to up, but the math below is relative to right.
			const FVector2d TextOffset = FVector2d(
					InCircle.Center.X + CosTheta * InCircle.Radius + FMath::Sign(CosTheta) * TextBoxSizeHalf.Y,
					InCircle.Center.Y + (InCircle.Radius + TextBoxSizeHalf.X) * SinTheta);

			return TextOffset;
		}
	}
}

UGizmoElementRotateAxis::UGizmoElementRotateAxis()
	: Axis()
	, NormalAxis()
	, SideAxis()
	, UpAxis()
	, DebugData()
{
	TextNumberFormattingOptions.AlwaysSign = true; // Always add +/-
	TextNumberFormattingOptions.UseGrouping = false;
	TextNumberFormattingOptions.RoundingMode = ERoundingMode::HalfFromZero;
	TextNumberFormattingOptions.MinimumFractionalDigits = 1;
	TextNumberFormattingOptions.MaximumFractionalDigits = 1;
}

void UGizmoElementRotateAxis::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (!ensure(RenderAPI))
	{
		return;
	}

	// Non-Delta Elements
	FRenderTraversalState CurrentRenderState(RenderState);
	if (UpdateRenderState(RenderAPI, FVector::ZeroVector, CurrentRenderState))
	{
		// Update Debug
		{
			DebugData.LastRenderTransform =	CurrentRenderState.LocalToWorldTransform;
		}

		ApplyUniformConstantScaleToTransform(CurrentRenderState.PixelToWorldScale, CurrentRenderState.LocalToWorldTransform);

		if (AxisRingElement)
		{
			AxisRingElement->Render(RenderAPI, RenderState);
		}
	}

	if (bIsShowingDelta)
	{
		// Delta Elements are based on the Start Transform, not current
		FRenderTraversalState DeltaRenderState(RenderState);
		if (UpdateDeltaRenderState(RenderAPI, FVector::ZeroVector, DeltaRenderState))
		{
			// Update Debug
			{
				DebugData.LastDeltaTransform = DeltaRenderState.LocalToWorldTransform;
			}

			ApplyUniformConstantScaleToTransform(DeltaRenderState.PixelToWorldScale, DeltaRenderState.LocalToWorldTransform);

			auto RenderDeltaElementWithState = [&](UGizmoElementBase* InElement, const FRenderTraversalState& InRenderState)
			{
				if (InElement)
				{
					InElement->Render(RenderAPI, InRenderState);
				}
			};

			auto RenderDeltaElement = [&](UGizmoElementBase* InElement)
			{
				RenderDeltaElementWithState(InElement, DeltaRenderState);
			};

			RenderDeltaElement(DeltaArcElement);

			if (CurrentState.bShowCursor)
			{
				// Match PixelToWorldScale to the transform scale so the dashed line's
				// Length (world space) converts to local space and back without distortion.
				FRenderTraversalState CursorLineRenderState = DeltaRenderState;
				CursorLineRenderState.LocalToWorldTransform.SetRotation(FQuat::Identity);
				CursorLineRenderState.PixelToWorldScale = CursorLineRenderState.LocalToWorldTransform.GetScale3D().X;
				RenderDeltaElementWithState(OriginToCursorLineElement, CursorLineRenderState);
			}

			RenderDeltaElement(OriginElement);
		}
	}
}

void UGizmoElementRotateAxis::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (bIsShowingDelta)
	{
		FRenderTraversalState DeltaRenderState(RenderState);
		UpdateDeltaRenderState(RenderAPI, FVector::ZeroVector, DeltaRenderState);

		if (DeltaWidgetElement)
		{
			DeltaWidgetElement->DrawHUD(Canvas, RenderAPI, DeltaRenderState);
		}
	}
}

void UGizmoElementRotateAxis::SetViewAlignType(EGizmoElementViewAlignType InViewAlignType)
{
	// We intentionally don't call Super here

	const bool bIsAxial = (InViewAlignType == EGizmoElementViewAlignType::Axial);
	if (!bIsAxial)
	{
		Super::SetViewAlignType(InViewAlignType);
	}

	if (AxisRingElement)
	{
		AxisRingElement->SetViewAlignType(InViewAlignType);

		// Axial Elements have a Partial End Angle of 180 degrees
		AxisRingElement->SetPartialEndAngle(bIsAxial ? UE_PI : UE_PI * 2.0f);
	}

	// Axial alignment is not supported for Delta Elements
	if (!bIsAxial)
	{
		for (const TObjectPtr<UGizmoElementBase>& Element
			: std::initializer_list<TObjectPtr<UGizmoElementBase>>{ DeltaArcElement })
		{
			if (Element)
			{
				Element->SetViewAlignType(InViewAlignType);
			}
		}
	}
}

void UGizmoElementRotateAxis::SetViewAlignNormal(FVector InAxis)
{
	// We intentionally don't call Super here
	// Super::SetViewAlignNormal(InAxis);

	for (const TObjectPtr<UGizmoElementBase>& Element
		: std::initializer_list<TObjectPtr<UGizmoElementBase>>{ AxisRingElement, DeltaArcElement })
	{
		if (Element)
		{
			Element->SetViewAlignNormal(InAxis);
		}
	}
}

void UGizmoElementRotateAxis::SetViewAlignAxis(FVector InAxis)
{
	// We intentionally don't call Super here
	// Super::SetViewAlignAxis(InAxis);

	for (const TObjectPtr<UGizmoElementBase>& Element
		: std::initializer_list<TObjectPtr<UGizmoElementBase>>{ AxisRingElement, DeltaArcElement })
	{
		if (Element)
		{
			Element->SetViewAlignAxis(InAxis);
		}
	}
}

const FGizmoElementRotateAxisStyle& UGizmoElementRotateAxis::GetStyle() const
{
	return Style;
}

void UGizmoElementRotateAxis::SetStyle(const FGizmoElementRotateAxisStyle& InStyle)
{
	Style = InStyle;
	ApplyStyle();
}

UE::Geometry::FFrame3d UGizmoElementRotateAxis::MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const EAxisList::Type InAxisList) const
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;

	if (Axis == EAxisList::Screen)
	{
		return MakeScreenAlignedPlane(InTransform.GetLocation(), InViewContext);
	}

	return MakeTransformedPlane(
		InTransform,
		InCoordinateSystem,
		NormalAxis,
		UpAxis,
		SideAxis);
}

UE::Geometry::FFrame3d UGizmoElementRotateAxis::MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const FRotationContext& InRotationContext, const EAxisList::Type InAxisList) const
{
	UE::Geometry::FFrame3d Plane;

	if (InRotationContext.bUseExplicitRotator)
	{
		using namespace UE::Editor::InteractiveToolsFramework::Internal;

		// Decompose rotations
		UE::GizmoRotationUtil::FRotationDecomposition Decomposition;
		DecomposeRotations(InTransform, InRotationContext, Decomposition);

		FTransform LocationOnlyTransform = FTransform::Identity;
		LocationOnlyTransform.SetLocation(InTransform.GetLocation());

		Plane = MakePlane(LocationOnlyTransform, InViewContext, InCoordinateSystem, Axis);

		Plane.Rotate(UE::Geometry::TQuaternion<double>(Decomposition.R[AxisIndex]));
	}
	else
	{
		Plane = MakePlane(InTransform, InViewContext, InCoordinateSystem, Axis);
	}

	return Plane;
}

void UGizmoElementRotateAxis::DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings)
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;

	if (bIsShowingDelta || GetElementInteractionState() == EGizmoElementInteractionState::Selected)
	{
		FToolDataVisualizer Draw;
		Draw.BeginFrame(RenderAPI);

		auto DrawAxis = [&](const FVector& InDirection, const FLinearColor& InColor)
	    {
    		Draw.DrawDirectionalArrow(
    			DebugData.TransformStart.GetLocation(),
    			DebugData.TransformStart.GetLocation() + (InDirection * 50),
    			InDirection,
    			InColor,
    			5.0f,
    			0.0f,
    			false);
	    };

		auto DrawTransform = [&](const FTransform& InTransform, const float InAxisLength)
		{
			const FVector TransformX = InTransform.GetUnitAxis(EAxis::X);
			const FVector TransformY = InTransform.GetUnitAxis(EAxis::Y);
			const FVector TransformZ = InTransform.GetUnitAxis(EAxis::Z);

			Draw.DrawDirectionalArrow(
				InTransform.GetLocation(),
				InTransform.GetLocation() + (TransformX * InAxisLength),
				TransformX,
				FLinearColor::Red,
				5.0f,
				0.0f,
				false);

			Draw.DrawDirectionalArrow(
				InTransform.GetLocation(),
				InTransform.GetLocation() + (TransformY * InAxisLength),
				TransformY,
				FLinearColor::Green,
				5.0f,
				0.0f,
				false);

			Draw.DrawDirectionalArrow(
				InTransform.GetLocation(),
				InTransform.GetLocation() + (TransformZ * InAxisLength),
				TransformZ,
				FLinearColor::Blue,
				5.0f,
				0.0f,
				false);
		};

		if (InSettings.bDrawCursorRay)
		{
			Draw.DrawPoint(DebugData.HitPointStart, FLinearColor::Red, 15.0f, false);
			Draw.DrawPoint(DebugData.HitPointCurrent, FLinearColor::Blue, 17.0f, false);
		}

		constexpr bool bDrawCursorLocal = false;
		if (InSettings.bDrawCursor && bDrawCursorLocal)
		{
			Draw.DrawLine(DebugData.HitPointCurrent, DebugData.HitPointCurrent + (DebugData.CursorDirection * 50));
		}

		// Element Transform
		if (InSettings.bDrawElementTransform)
		{
			FVector Origin = StartState.RotatedTransform.GetLocation();

			FVector Axis0 = StartState.RotatedTransform.TransformVectorNoScale(DeltaArcElement->GetAxisBitangent());
			FVector Axis1 = StartState.RotatedTransform.TransformVectorNoScale(DeltaArcElement->GetAxisTangent());
			FVector Normal = StartState.RotatedTransform.TransformVectorNoScale(NormalAxis);

			// @todo: This isn't right - coordinate systems should be fetched from somewhere
			UE::Geometry::FFrame3d P1 = MakePlane(DebugData.LastRenderTransform, GizmoViewContext, EToolContextCoordinateSystem::Local);
			UE::Geometry::FFrame3d P2 = MakePlane(DebugData.LastDeltaTransform, GizmoViewContext, EToolContextCoordinateSystem::Local);

			using namespace UE::InteractiveToolsFramework;
			UE::Geometry::FFrame3d ElementPlane = MakePlaneFrame(
				Origin,
				Normal,
				UpAxis,
				SideAxis);

			DrawTransform(ElementPlane.ToFTransform(), 30.0f);

			// Clock
			{
				GetAxisBasis(
					Axis,
					Normal,
					Axis0,
					Axis1);

				const FQuat ElementQuat = FQuat(ElementPlane.Rotation);

				const FVector AxisX = Axis0;
				const FVector AxisY = Axis1;

				// const double Sign = TransformGizmo->ViewToAlignedSign.X;
				Normal = StartState.RotatedTransform.TransformVectorNoScale(Normal);
				Axis0 = StartState.RotatedTransform.TransformVectorNoScale(AxisX);
				Axis1 = StartState.RotatedTransform.TransformVectorNoScale(AxisY);

				Draw.DrawLine(Origin, Origin + (Normal * 100.0f), FLinearColor::Blue, 0.0f, false);
				Draw.DrawLine(Origin, Origin - (Normal * 100.0f), FLinearColor::White, 0.0f, false);

				const double RadialSpacingRad = FMath::DegreesToRadians(30.0f);
				const double Radius = FVector::Distance(StartState.RotatedTransform.GetLocation(), DebugData.HitPointCurrent) * 1.1f;

				const int32 NumSpokePoints = FMath::FloorToInt32(UE_DOUBLE_TWO_PI / RadialSpacingRad) + 1;

				for (int32 SpokeIndex = 0; SpokeIndex < NumSpokePoints; ++SpokeIndex)
				{
					const FVector LocalSpokePoint(
						0.0f,
						FMath::Cos(RadialSpacingRad * SpokeIndex),
						FMath::Sin(RadialSpacingRad * SpokeIndex));

					const FVector SpokePoint = Origin + ElementQuat.RotateVector(LocalSpokePoint) * Radius;
					// const FVector SpokePoint = Origin + Axis0.RotateAngleAxisRad(RadialSpacingRad * SpokeIndex, Normal) * Radius;

					Draw.DrawLine(
						Origin,
						SpokePoint,
						FLinearColor::White,
						0.0f,
						false);
				}

				// Rotating Clock Hand
				{
					const double RotationSpeed = 1.0;
					const double TimeOffset = FPlatformTime::Seconds() * RotationSpeed;

					const FVector LocalSpokePoint(
						0.0f,
						FMath::Cos(RadialSpacingRad + TimeOffset),
						FMath::Sin(RadialSpacingRad + TimeOffset));

					const FVector SpokePoint = Origin + ElementQuat.RotateVector(LocalSpokePoint) * Radius;

					Draw.DrawLine(
						Origin,
						SpokePoint,
						FLinearColor::Red,
						4.5f,
						false);
				}

				Draw.DrawCircle(
					Origin,
					Normal,
					static_cast<float>(Radius),
					128,
					FLinearColor::White,
					0.0f,
					false);
			}
		}

		Draw.EndFrame();
	}
}

void UGizmoElementRotateAxis::SetGizmoViewContext(UGizmoViewContext* InGizmoViewContext)
{
	GizmoViewContext = InGizmoViewContext;
}

void UGizmoElementRotateAxis::SetWidgetHost(IToolkitHost* const InWidgetHost)
{
	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetWidgetHost(InWidgetHost);
	}
}

void UGizmoElementRotateAxis::Setup(
	const uint32 InPartId,
	const EAxisList::Type InAxis,
	const FGizmoElementRotateAxisStyle& InStyle)
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;
	using namespace UE::Editor::InteractiveToolsFramework::Private;
	using namespace GizmoElementRotateAxisLocals;

	if (AxisRingElement && DeltaArcElement && DeltaWidgetElement && OriginElement && OriginToCursorLineElement)
	{
		return;
	}

	Axis = InAxis;
	AxisIndex = GetAxisIndex(InAxis);

	GetAxisBasis<FVector::FReal>(InAxis, NormalAxis, UpAxis, SideAxis);

	// Primary Ring Element
	if (!AxisRingElement)
	{
		AxisRingElement = NewObject<UGizmoElementTorus>();
		AxisRingElement->SetPartIdentifier(InPartId);
		AxisRingElement->SetCenter(FVector::ZeroVector);
		AxisRingElement->SetAxisBitangent(UpAxis);
		AxisRingElement->SetAxisTangent(SideAxis);
		AxisRingElement->SetPartialType(EGizmoElementPartialType::PartialViewDependent);
		AxisRingElement->SetPartialStartAngle(0.0f);
		AxisRingElement->SetPartialEndAngle(UE_PI);
		AxisRingElement->SetViewDependentAxis(NormalAxis);
		AxisRingElement->SetViewAlignType(EGizmoElementViewAlignType::Axial);
		AxisRingElement->SetViewAlignNormal(UpAxis);
		AxisRingElement->SetViewAlignAxialAngleTol(static_cast<float>(UE_DOUBLE_SMALL_NUMBER));
		AxisRingElement->SetViewAlignAxis(NormalAxis);

		Add(AxisRingElement);
	}

	// Apply common delta element settings
	auto ApplyDeltaElementSettings = [&, InPartId](UGizmoElementBase* InElement)
	{
		InElement->SetPartIdentifier(InPartId);

		InElement->SetHittableState(false);
		InElement->SetEnabledForDefaultState(false);
		InElement->SetEnabledForHoveringState(false);
		InElement->SetEnabledForInteractingState(true);
		InElement->SetEnabledForSelectedState(false);
		InElement->SetEnabledForSubduedState(false);
	};

	// Delta Arc Element
	if (!DeltaArcElement)
	{
		DeltaArcElement = NewObject<UGizmoElementArc>();
		DeltaArcElement->SetCenter(FVector::ZeroVector);
		DeltaArcElement->SetAxisBitangent(UpAxis);
		DeltaArcElement->SetAxisTangent(SideAxis);
		DeltaArcElement->SetPartialType(EGizmoElementPartialType::Partial);
		DeltaArcElement->SetPartialStartAngle(0.0f);
		DeltaArcElement->SetPartialEndAngle(0.0f);

		ApplyDeltaElementSettings(DeltaArcElement);

		DeltaArcElement->SetVisibleState(false);

		Add(DeltaArcElement);
	}

	// Delta Widget Element
	if (!DeltaWidgetElement)
	{
		DeltaWidgetElement = NewObject<UGizmoElementValueWidget>();

		DeltaWidgetElement->SetPartIdentifier(InPartId);
		DeltaWidgetElement->SetViewAlignType(EGizmoElementViewAlignType::PointScreen);
		DeltaWidgetElement->SetViewAlignAxis(FVector::UpVector);
		DeltaWidgetElement->SetViewAlignNormal(-FVector::ForwardVector);
		DeltaWidgetElement->SetClampToScreen(true);

		DeltaWidgetElement->SetLocation(FVector::ZeroVector);

		ApplyDeltaElementSettings(DeltaWidgetElement);

		DeltaWidgetElement->ClearSubdueVertexColor();
		DeltaWidgetElement->ClearSelectVertexColor();

		DeltaWidgetElement->SetVisibleState(false);

		Add(DeltaWidgetElement);
	}

	// Origin Circle Element
	if (!OriginElement)
	{
		OriginElement = NewObject<UGizmoElementCircle>();
		OriginElement->SetCenter(FVector::ZeroVector);
		OriginElement->SetAxisBitangent(FVector::UpVector);
		OriginElement->SetAxisTangent(FVector::RightVector);
		OriginElement->SetViewAlignType(EGizmoElementViewAlignType::PointOnly);
		OriginElement->SetViewAlignNormal(-FVector::ForwardVector);

		ApplyDeltaElementSettings(OriginElement);

		OriginElement->SetDrawMesh(true);
		OriginElement->SetHitMesh(false);
		OriginElement->SetVertexColor(FLinearColor::Transparent);

		OriginElement->SetDrawLine(true);
		OriginElement->SetHitLine(false);

		constexpr float OtherStateLineThickness = 1.0f;
		OriginElement->SetHoverLineThicknessMultiplier(OtherStateLineThickness);
		OriginElement->SetInteractLineThicknessMultiplier(OtherStateLineThickness);

		OriginElement->SetLineColor(FLinearColor::Black);
		OriginElement->SetHoverLineColor(FLinearColor::Black);
		OriginElement->SetInteractLineColor(FLinearColor::Black);

		Add(OriginElement);
	}

	// Origin to Cursor Line Element
	if (!OriginToCursorLineElement)
	{
		OriginToCursorLineElement = NewObject<UGizmoElementDashedLine>();
		OriginToCursorLineElement->SetBase(FVector::ZeroVector);
		OriginToCursorLineElement->SetDirection(FVector::UpVector);
		OriginToCursorLineElement->SetScreenSpaceLine(true);

		OriginToCursorLineElement->SetViewAlignType(EGizmoElementViewAlignType::None);

		OriginToCursorLineElement->SetIsVisibleFunction([](
			const FSceneView*, EViewInteractionState InCurrentViewInteractionState, EGizmoElementInteractionState,
			const FTransform&, const FVector&)
		{
			const bool bIsFocusedView = !!(InCurrentViewInteractionState & EViewInteractionState::Focused);

			// We only draw in the single focused/active/primary viewport
			return bIsFocusedView;
		});

		ApplyDeltaElementSettings(OriginToCursorLineElement);

		OriginToCursorLineElement->SetVisibleState(false);

		Add(OriginToCursorLineElement);
	}

	SetViewAlignType(EGizmoElementViewAlignType::Axial);
	SetViewAlignNormal(SideAxis);

	SetPartIdentifier(InPartId, true);

	bIsValid = true;

	SetStyle(InStyle);

	UpdateElements();
}

void UGizmoElementRotateAxis::UpdateElements()
{
	if (!bIsValid)
	{
		return;
	}

	// Delta Arc Element
	if (DeltaArcElement)
	{
		DeltaArcElement->SetViewDepthOffset(10.0f); // Ensure the delta arc is always behind the ring, dashed line, etc.
		DeltaArcElement->SetInnerRadius(0.0f); // We want a pizza slice here, so the inner radius is 0
	}

	// Origin Circle Element
	if (OriginElement)
	{
		OriginElement->SetViewDepthOffset(-50.0f); // Ensure the origin is always on top of the arc, dashed line, etc.
	}

	// Origin to Cursor Line Element
	if (OriginToCursorLineElement)
	{
		OriginToCursorLineElement->SetViewDepthOffset(-100.0f); // Ensure the dashed line is always on top of the arc, but behind the text.
	}
}

void UGizmoElementRotateAxis::BeginDelta(const FDeltaParameters& InParameters)
{
	UE_CLOGF(
		UE::Editor::InteractiveToolsFramework::Private::GizmoElementRotateAxisLocals::bLog,
		LogGizmoElementRotateAxis, Verbose, "BeginDelta: Angle: %.1f", FMath::RadiansToDegrees(InParameters.Angle));

	bIsShowingDelta = true;
	StartState.Initialize(InParameters, AxisIndex);
	CurrentState = StartState;

	if (OriginToCursorLineElement && StartState.bShowCursor)
	{
		OriginToCursorLineElement->SetVisibleState(!InParameters.bIsIndirectManipulation);
	}
	
	if (DeltaArcElement)
	{
		DeltaArcElement->SetVisibleState(EnumHasAnyFlags(Style.ShowFlags, EGizmoElementRotateShowFlags::DeltaArc));
	}

	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetVisibleState(EnumHasAnyFlags(Style.ShowFlags, EGizmoElementRotateShowFlags::DeltaLabel));
		DeltaWidgetElement->SetClampToScreen(InParameters.bIsCursorInViewport); // Only set this on Begin, if the interaction starts within the bounds of the viewport
	}

	// Update Debug
	{
		DebugData.TransformStart = StartState.RotatedTransform;
		DebugData.HitPointCurrent = DebugData.HitPointStart = InParameters.CursorLocation;
	}

	// Update State
	UpdateDelta(InParameters);
}

void UGizmoElementRotateAxis::UpdateDelta(const FDeltaParameters& InParameters)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementRotateAxisLocals;

	if (!DeltaArcElement || !DeltaWidgetElement)
	{
		return;
	}

	double ScaledRadius = 0.0;

	CurrentState.Update(InParameters);

	const double StartAngle = StartState.Angle;
	const double CurrentAngle = CurrentState.Angle; // CurrentAngle is affected by modification, so it may be snapped

	FVector DeltaNextNormal = FVector::UpVector;

	// Delta Arc Element
	if (DeltaArcElement)
	{
		ScaledRadius = DeltaArcElement->GetRadius();

		DeltaNextNormal = UpAxis.RotateAngleAxisRad(CurrentAngle, NormalAxis);

		// Arc angles are CW, incoming angle is CCW, so flip
		double ArcStartAngle = StartAngle * -1;
		double ArcEndAngle = CurrentAngle * -1;

		FMath::GetMinMax(ArcStartAngle, ArcEndAngle);

		// Clamp End such that it's no more than 360 degrees relative to Start
		ArcEndAngle = FMath::Min(ArcEndAngle, ArcStartAngle + UE_TWO_PI);

		UE_CLOGF(bLog, LogGizmoElementRotateAxis, Verbose, "[%lli] Start Angle: %.1f | Current Angle: %.1f",
			GFrameCounter,
			FMath::RadiansToDegrees(StartAngle),
			FMath::RadiansToDegrees(CurrentAngle));

		DeltaArcElement->SetPartialStartAngle(ArcStartAngle);
		DeltaArcElement->SetPartialEndAngle(ArcEndAngle);
	}

	FVector DirectionToCursor = InParameters.CursorLocation - StartState.Transform.GetLocation();
	const double DistanceToCursor = DirectionToCursor.Length();

	//const FVector CursorDirection = FVector::CrossProduct(DirectionToCursor, InParameters.Transform.GetUnitAxis(EAxis::X));
	const FVector CursorDirection = DirectionToCursor ^ StartState.PlaneNormal;
	DebugData.CursorDirection = CursorDirection;

	// Delta Text Elements
	if (DeltaWidgetElement)
	{
		{
			constexpr int32 TextBoxWidthGlyphCount = 5; // Approximate number of gyphs needed for the angle number, with decimals and the degree symbol, adjusted for the font width vs. height ratio
			const float TextBoxHeight = FMath::Min(1.0f, 28.0f);
			const FVector2d TextBoxSize(TextBoxHeight * TextBoxWidthGlyphCount, TextBoxHeight);

			const double TextOffsetFromCircle = 20.0f + (TextBoxSize * 0.5f).Length();

			const FVector TextCenter = DeltaNextNormal * (ScaledRadius + TextOffsetFromCircle);
			DeltaWidgetElement->SetLocation(TextCenter);
		}

		const double DeltaAngle = CurrentState.Angle - StartState.Angle;

		DeltaWidgetElement->SetUnitText(StartState.DisplaySign * DeltaAngle, EUnitType::Angle, TextNumberFormattingOptions);
	}

	// Origin to Cursor Line Element
	{
		OriginToCursorLineElement->SetDirection(DirectionToCursor);
		OriginToCursorLineElement->SetLength(static_cast<float>(DistanceToCursor));
		OriginToCursorLineElement->SetViewDepthOffset(0.0f);
	}

	// Update Debug
	{
		DebugData.HitPointCurrent = InParameters.CursorLocation;
	}
}

void UGizmoElementRotateAxis::EndDelta()
{
	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetVisibleState(false);
	}

	bIsShowingDelta = false;
	StartState = CurrentState;
}

void UGizmoElementRotateAxis::ApplyStyle()
{
	if (!bIsValid)
	{
		return;
	}

	SetVertexColor(Style.Colors.Default.Get(GetVertexColor()));

	const float SizeCoefficient = Style.SizeCoefficient.Get(1.0f);

	const float MinLineRadius = Style.MinLineThickness * 0.5f;
	const float LineRadius = Style.LineThickness * 0.5f;

	// Primary Ring Element
	if (AxisRingElement)
	{
		AxisRingElement->SetNumSegments(Style.NumSegments);
		AxisRingElement->SetNumInnerSlices(Style.NumInnerSlices);
		AxisRingElement->SetRadius((Style.Radius * Style.RadiusMultiplier) * SizeCoefficient);
		AxisRingElement->SetInnerRadius(FMath::Max(MinLineRadius, (LineRadius * Style.LineThicknessMultiplier)) * SizeCoefficient);
		AxisRingElement->SetHoverInnerRadiusMultiplier(Style.HoverLineThicknessMultiplier);

		UE::Editor::InteractiveToolsFramework::ApplyMaterialsToElement(AxisRingElement, Style.Materials);
		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(AxisRingElement, Style.Colors);

		AxisRingElement->SetPixelHitDistanceThreshold(Style.PixelHitDistanceThreshold);
	}

	// Apply common delta element settings
	auto ApplyDeltaElementSettings = [&](UGizmoElementBase* InElement)
	{
		UE::Editor::InteractiveToolsFramework::ApplyMaterialsToElement(InElement, Style.Materials);
		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(InElement, Style.Colors);

		InElement->SetHoverMaterial(Style.DeltaMaterial);
		InElement->SetInteractMaterial(Style.DeltaMaterial);
	};

	// Delta Arc Element
	if (DeltaArcElement)
	{
		DeltaArcElement->SetNumSegments(Style.NumSegments);
		DeltaArcElement->SetRadius((Style.Radius * Style.RadiusMultiplier) * SizeCoefficient);
		DeltaArcElement->SetInnerRadius(0.0f); // We want a pizza slice here, so the inner radius is 0

		ApplyDeltaElementSettings(DeltaArcElement);

		DeltaArcElement->SetMaterial(Style.DeltaMaterial);
		DeltaArcElement->SetSelectMaterial(Style.DeltaMaterial);
		DeltaArcElement->SetSubdueMaterial(Style.DeltaMaterial);

		auto AdjustHSV = [](FLinearColor& InOutColor, const FLinearColor& InHSV)
		{
			InOutColor = InOutColor.LinearRGBToHSV();

			InOutColor.R += InHSV.R;
			InOutColor.G *= InHSV.G;
			InOutColor.B *= InHSV.B;

			InOutColor.R = FMath::Fmod(InOutColor.R, 360.0f); // Ensure Hue is in [0, 360] range
			InOutColor.G = FMath::Clamp(InOutColor.G, 0.0f, 1.0f);
			InOutColor.B = FMath::Clamp(InOutColor.B, 0.0f, 1.0f);

			InOutColor = InOutColor.HSVToLinearRGB();
		};

		FLinearColor LineColor = Style.Colors.Default.Get(FLinearColor::White);
		AdjustHSV(LineColor, Style.DeltaFillHSVModifier);
		LineColor = LineColor.CopyWithNewOpacity(Style.DeltaStrokeOpacity);

		DeltaArcElement->SetHoverLineColor(LineColor);
		DeltaArcElement->SetInteractLineColor(LineColor);

		const float DeltaArcLineThickness = Style.DeltaStrokeThickness * Style.LineThicknessMultiplier * SizeCoefficient;
		DeltaArcElement->SetLineThickness(FMath::Max(Style.MinLineThickness, DeltaArcLineThickness));
	}

	// Delta Widget Element
	if (DeltaWidgetElement)
	{
		const FLinearColor BackgroundColor = Style.DeltaTextBackgroundColor;

		DeltaWidgetElement->SetMaterial(Style.VertexColorMaterial);
		DeltaWidgetElement->SetHoverMaterial(Style.VertexColorMaterial);
		DeltaWidgetElement->SetInteractMaterial(Style.VertexColorMaterial);

		DeltaWidgetElement->SetVertexColor(BackgroundColor);
		DeltaWidgetElement->SetHoverVertexColor(BackgroundColor);
		DeltaWidgetElement->SetInteractVertexColor(BackgroundColor);

		DeltaWidgetElement->ClearSubdueVertexColor();
		DeltaWidgetElement->ClearSelectVertexColor();
	}

	// Origin Circle Element
	if (OriginElement)
	{
		ApplyDeltaElementSettings(OriginElement);

		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(OriginElement, Style.Colors);

		OriginElement->SetRadius(Style.OriginRadius * SizeCoefficient);

		OriginElement->SetMaterial(Style.VertexColorMaterial);
		OriginElement->SetHoverMaterial(Style.VertexColorMaterial);
		OriginElement->SetInteractMaterial(Style.VertexColorMaterial);

		OriginElement->SetLineThickness(FMath::Max(Style.MinLineThickness, Style.OriginLineThickness * Style.LineThicknessMultiplier));
	}

	// Origin to Cursor Line Element
	if (OriginToCursorLineElement)
	{
		const float Radius = Style.OriginToCursorLineThickness * 0.5f;
		OriginToCursorLineElement->SetLineThickness(FMath::Max(MinLineRadius, (Radius * Style.LineThicknessMultiplier)) * SizeCoefficient);

		const float DashLength = FMath::Max(Style.MinLineThickness, Style.OriginToCursorLineDashSpacing * Style.LineThicknessMultiplier) * SizeCoefficient;
		const float GapLength = FMath::Max(Style.MinLineThickness, Style.OriginToCursorLineDashGapSpacing * Style.LineThicknessMultiplier) * SizeCoefficient;
		OriginToCursorLineElement->SetDashParameters(DashLength, GapLength);

		OriginToCursorLineElement->SetHoverVertexColor(Style.CursorColor);
		OriginToCursorLineElement->SetInteractVertexColor(Style.CursorColor);

		OriginToCursorLineElement->SetInteractMaterial(Style.VertexColorMaterial);

		OriginToCursorLineElement->SetLineColor(FLinearColor::Black);
		OriginToCursorLineElement->SetHoverLineColor(FLinearColor::Black);
		OriginToCursorLineElement->SetInteractLineColor(FLinearColor::Black);
	}
}

bool UGizmoElementRotateAxis::UpdateDeltaRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalOrigin, FRenderTraversalState& InOutRenderState)
{
	InOutRenderState.LocalToWorldTransform.SetRotation(StartState.RotatedTransform.GetRotation());

	return UpdateRenderState(RenderAPI, InLocalOrigin, InOutRenderState);
}

void UGizmoElementRotateAxis::FState::Initialize(const FDeltaParameters& InParameters, const int32 InAxisIndex)
{
	Transform = InParameters.Transform;
	RotatedTransform = InParameters.Transform;
	RotationContext = InParameters.RotationContext;
	Angle = InParameters.Angle;
	PlaneNormal = InParameters.PlaneNormal;
	bShowCursor = !InParameters.bIsIndirectManipulation && InParameters.RotateMode != EAxisRotateMode::Pull;
	DisplaySign = InParameters.DisplaySign;

	// Store rotated transform if needed
	if (RotationContext.bUseExplicitRotator)
	{
		// Decompose rotations
		UE::GizmoRotationUtil::FRotationDecomposition Decomposition;
		DecomposeRotations(Transform, RotationContext, Decomposition);

		RotatedTransform.SetRotation(-Decomposition.R[InAxisIndex]);
	}
}

void UGizmoElementRotateAxis::FState::Update(const FDeltaParameters& InParameters)
{
	Transform = InParameters.Transform;
	Angle = InParameters.Angle;
	bShowCursor = !InParameters.bIsIndirectManipulation && InParameters.RotateMode != EAxisRotateMode::Pull;
}
