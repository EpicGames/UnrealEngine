// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoElementGroup.h"
#include "EditorGizmoElementInterfaces.h"
#include "EditorGizmos/EditorGizmoElementShared.h"
#include "EditorGizmos/GizmoElementRotate.h"
#include "FrameTypes.h"

#include "GizmoElementRotateAxis.generated.h"

class IToolkitHost;
class SGizmoCursor;
class UGizmoElementRoundedRectangle;
class UGizmoElementDashedLine;
class UGizmoElementRectangle;
class UGizmoElementBox;
class UGizmoElementCylinder;
class UGizmoElementLineStrip;
class UGizmoElementCircle;
class UGizmoElementSphere;
class UGizmoElementArc;
class UGizmoElementValueWidget;
class UGizmoElementTorus;
class UGizmoViewContext;

/**
 * Combines the visual elements of a single rotation axis: a torus ring handle,
 * a delta arc showing the rotation angle, an origin indicator, a cursor-to-origin line,
 * and a value label shown during interaction.
 * Also implements IPlaneProvider to define the interaction plane for rotation projection.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementRotateAxis
	: public UGizmoElementGroupBase
	, public UE::Editor::InteractiveToolsFramework::IPlaneProvider
{
	GENERATED_BODY()

public:
	/** Parameters shared between BeginDelta and UpdateDelta calls, describing the current rotation interaction state. */
	struct FDeltaParameters
	{
		/** The current world transform of the gizmo. */
		FTransform Transform = FTransform::Identity;

		/** The active coordinate system (World, Local, etc.). */
		EToolContextCoordinateSystem CoordinateSystem = EToolContextCoordinateSystem::World;

		/** Context for the rotation (e.g. gimbal state, accumulated rotation). */
		FRotationContext RotationContext = FRotationContext();

		/** Normal of the interaction plane. */
		FVector PlaneNormal = FVector::UpVector;

		/** Accumulated rotation angle in radians from the start of the interaction. */
		double Angle = 0.0;

		/** Display sign for the angle. Ensures clockwise visual matches clockwise input, even when the applied angle is negated (e.g. for X, Y axes). */
		int8 DisplaySign = 1;

		/** Whether this is an indirect manipulation (e.g. input field rather than drag). */
		bool bIsIndirectManipulation = false;

		/** Whether the cursor is currently within the viewport (affects delta label visibility). */
		bool bIsCursorInViewport = true;

		/** World-space cursor location, used for origin-to-cursor line. */
		FVector CursorLocation = FVector::ZeroVector;

		/** Screen-space cursor location. */
		FVector2D CursorLocation2D = FVector2D::ZeroVector;

		/** The rotation interaction mode (Pull or Dial). */
		EAxisRotateMode::Type RotateMode = EAxisRotateMode::Type::Pull;
	};

public:
	UGizmoElementRotateAxis();

	virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;

	virtual void SetViewAlignType(EGizmoElementViewAlignType InViewAlignType) override;
	virtual void SetViewAlignNormal(FVector InAxis) override;
	virtual void SetViewAlignAxis(FVector InAxis) override;

	/** Returns the current visual style for this rotation axis element. */
	const FGizmoElementRotateAxisStyle& GetStyle() const;

	/** Sets and applies the given Style. */
	void SetStyle(const FGizmoElementRotateAxisStyle& InStyle);

	//~ Begin IPlaneProvider
	/** Creates the interaction plane for this rotation axis, transformed into the provided coordinate space. */
	virtual UE::Geometry::FFrame3d MakePlane(
		const FTransform& InTransform,
		const UGizmoViewContext* InViewContext,
		const EToolContextCoordinateSystem InCoordinateSystem,
		const EAxisList::Type InAxisList = EAxisList::None) const override;

	/** Creates the interaction plane using the rotation context for gimbal-aware coordinate space transformation. */
	virtual UE::Geometry::FFrame3d MakePlane(
		const FTransform& InTransform,
		const UGizmoViewContext* InViewContext,
		const EToolContextCoordinateSystem InCoordinateSystem,
		const FRotationContext& InRotationContext,
		const EAxisList::Type InAxisList = EAxisList::None) const override;
	//~ End IPlaneProvider

	/** Renders various debug visualizations, if enabled. */
	void DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings);

	/** Sets the Gizmo view context, needed for screen space interactions */
	void SetGizmoViewContext(UGizmoViewContext* InGizmoViewContext);

	/** Sets the host to add/remove a widget to.
	* Currently, this is a Toolkit host, but it could be abstracted to an ISlateWidgetHost (see IEditorViewportClientProxy). */
	void SetWidgetHost(IToolkitHost* const InWidgetHost);

	/**
	 * Initialize the rotation axis element with a part identifier, axis, and visual style.
	 * Must be called before the element can render or receive interactions.
	 * @param InPartId Unique identifier for hit-testing this axis.
	 * @param InAxis The axis this element represents (e.g. EAxisList::X).
	 * @param InStyle Visual style parameters for the axis.
	 */
	void Setup(
		const uint32 InPartId,
		const EAxisList::Type InAxis,
		const FGizmoElementRotateAxisStyle& InStyle);

	/** Rebuilds child elements to reflect the current style. Call after modifying style properties. */
	void UpdateElements();

	/** Begin a delta interaction, recording the start state. */
	void BeginDelta(const FDeltaParameters& InParameters);

	/** Update the delta arc and value label with the current angle and cursor location. */
	void UpdateDelta(const FDeltaParameters& InParameters);

	/** End the current delta interaction and hide delta visuals. */
	void EndDelta();

private:
	/** Style contains various uniform and varying properties that affect the appearance of this gizmo element. */
	UPROPERTY(Getter, Setter)
	FGizmoElementRotateAxisStyle Style;

	/** Rotate Axis. */
	UPROPERTY()
	TObjectPtr<UGizmoElementTorus> AxisRingElement;

	/** Arc element that visualizes the rotation delta angle during interaction. */
	UPROPERTY()
	TObjectPtr<UGizmoElementArc> DeltaArcElement;

	/** Widget element that displays the numeric rotation angle value during interaction. */
	UPROPERTY()
	TObjectPtr<UGizmoElementValueWidget> DeltaWidgetElement;

	/** Circle element drawn at the gizmo origin during rotation interaction. */
	UPROPERTY()
	TObjectPtr<UGizmoElementCircle> OriginElement;

	/** Dashed line element drawn from the origin to the cursor during rotation interaction. */
	UPROPERTY()
	TObjectPtr<UGizmoElementDashedLine> OriginToCursorLineElement;

private:
	/** Applies the current Style properties to all child visual elements. */
	void ApplyStyle();

	/**
	 * Updates render state for delta elements during render traversal, using the start transform rather than the current.
	 * @return View-dependent visibility; true if this element is visible in the current view.
	 */
	bool UpdateDeltaRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalOrigin, FRenderTraversalState& InOutRenderState);

private:
	/** Flag indicating element validity - generally true after Setup is called. */
	std::atomic_bool bIsValid = false;
	
	/** Gizmo view context, needed for screen space interactions */
	UPROPERTY(Setter)
	TObjectPtr<UGizmoViewContext> GizmoViewContext;

	/** The represented axis (e.g. EAxisList::X). */
	EAxisList::Type Axis;
	/** Index of the axis (0=X, 1=Y, 2=Z). */
	int32 AxisIndex = 0;

	/** Normal direction of the rotation plane (axis of rotation). */
	FVector NormalAxis;

	/** Side direction in the rotation plane, perpendicular to NormalAxis. */
	FVector SideAxis;

	/** Up direction in the rotation plane, perpendicular to both NormalAxis and SideAxis. */
	FVector UpAxis;

	/** Formatting options for the delta angle label text. */
	FNumberFormattingOptions TextNumberFormattingOptions;

	/** Snapshot of rotation interaction state, captured at the start and updated each frame. */
	struct FState
	{
		/** World transform of the gizmo at this point in the interaction. */
		FTransform Transform;
		/** World transform with the accumulated rotation applied. */
		FTransform RotatedTransform;
		/** Rotation context (e.g. gimbal state) at this point in the interaction. */
		FRotationContext RotationContext;
		/** Accumulated rotation angle in radians. */
		double Angle = 0.0;
		/** Normal of the rotation plane. */
		FVector PlaneNormal = FVector::UpVector;
		/** Whether to show the cursor-to-origin line and origin indicator. */
		bool bShowCursor = false;
		/** Display sign multiplier for the angle (accounts for axis direction). */
		int8 DisplaySign = 1;

		/** Initializes this state from the given delta parameters and axis index. */
		void Initialize(const FDeltaParameters& InParameters, const int32 InAxisIndex);
		/** Updates this state with new delta parameters. */
		void Update(const FDeltaParameters& InParameters);
	};

	/** State captured at the beginning of a rotation interaction. */
	FState StartState;
	/** State updated each frame during a rotation interaction. */
	FState CurrentState;

	/** Whether a delta interaction is currently active and delta visuals should be shown. */
	bool bIsShowingDelta = false;

	/** Debug data collected during interactions for optional debug visualization. */
	struct FGizmoDebugData
	{
		/** Gizmo transform at the start of the interaction. */
		FTransform TransformStart;

		/** Most recent transform used during rendering. */
		FTransform LastRenderTransform = FTransform::Identity;
		/** Most recent transform used for delta element rendering. */
		FTransform LastDeltaTransform = FTransform::Identity;

		/** World-space hit point at the start of the interaction. */
		FVector HitPointStart = FVector::ZeroVector;
		/** World-space hit point at the current frame. */
		FVector HitPointCurrent = FVector::ZeroVector;

		/** Direction from the origin toward the cursor in world space. */
		FVector CursorDirection;
	} DebugData;
};
