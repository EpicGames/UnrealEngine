// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementRotateAxisSet.h"

#include "BaseGizmos/GizmoUtil.h"

namespace UE::Editor::InteractiveToolsFramework::Private
{
	namespace GizmoElementRotateAxisSetLocals
	{
		template <typename Func>
		bool EnsureRequiredFunc(const Func& InFunc, const FString& InFuncName)
		{
			return ensureMsgf(InFunc.IsSet(), TEXT("%s is required to be set."), *InFuncName);
		}
	}
}

void FGizmoElementRotateAxisStyleOverride::ApplyTo(FGizmoElementRotateAxisStyle& InOutStyle) const
{
	using namespace UE::Editor::InteractiveToolsFramework;

	if (Colors.IsSet())
	{
		ApplyColorOverrides(InOutStyle.Colors, Colors.GetValue());
	}

	if (Materials.IsSet())
	{
		ApplyMaterialOverrides(InOutStyle.Materials, Materials.GetValue());
	}

	if (DeltaMaterial.IsSet())
	{
		InOutStyle.DeltaMaterial = DeltaMaterial.GetValue();
	}

	if (VertexColorMaterial.IsSet())
	{
		InOutStyle.VertexColorMaterial = VertexColorMaterial.GetValue();
	}
}

void UGizmoElementRotateAxisSet::DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings, const uint32 InPartId)
{
	if (InPartId == 0)
	{
		ForEachAxisElement([&](UGizmoElementRotateAxis* InAxisElement)
		{
			if (InAxisElement)
			{
				InAxisElement->DrawDebug(RenderAPI, InSettings);
			}
		});
	}
	else
	{
		ForEachSubElementRecursive([&](UGizmoElementBase* InElement)
		{
			if (UGizmoElementRotateAxis* RotateAxisElement = Cast<UGizmoElementRotateAxis>(InElement))
			{
				RotateAxisElement->DrawDebug(RenderAPI, InSettings);
			}
		},
		InPartId);
	}
}

void UGizmoElementRotateAxisSet::SetWidgetHost(IToolkitHost* const InWidgetHost)
{
	if (!bIsValid)
	{
		return;
	}

	ForEachAxisElement([&](UGizmoElementRotateAxis* InAxisElement)
	{
		InAxisElement->SetWidgetHost(InWidgetHost);
	});
}

void UGizmoElementRotateAxisSet::SetAxisEnabled(const EAxisList::Type InAxisListToEnable)
{
	auto SetAxisEnabled = [&](UGizmoElementRotateAxis* InAxisElement, const EAxisList::Type InSingleAxis) -> bool
	{
		const bool bEnableAxis = static_cast<uint8>(InAxisListToEnable) & static_cast<uint8>(InSingleAxis);
		InAxisElement->SetEnabled(bEnableAxis);

		return bEnableAxis;
	};

	SetAxisEnabled(AxisElementX, EAxisList::X);
	SetAxisEnabled(AxisElementY, EAxisList::Y);
	SetAxisEnabled(AxisElementZ, EAxisList::Z);
}

const FGizmoElementRotateAxisStyle& UGizmoElementRotateAxisSet::GetStyle() const
{
	return Style;
}

void UGizmoElementRotateAxisSet::SetStyle(
	const FGizmoElementRotateAxisStyle& InStyle,
	const FGizmoElementRotateAxisStyleOverride& InAxisStyleX,
	const FGizmoElementRotateAxisStyleOverride& InAxisStyleY,
	const FGizmoElementRotateAxisStyleOverride& InAxisStyleZ)
{
	Style = InStyle;
	StyleX = InAxisStyleX;
	StyleY = InAxisStyleY;
	StyleZ = InAxisStyleZ;

	ApplyStyle();
}

void UGizmoElementRotateAxisSet::ApplyStyle()
{
	if (!bIsValid)
	{
		return;
	}

	if (Style.Colors.Default.IsSet())
	{
		SetVertexColor(Style.Colors.Default.GetValue());
	}

	auto UpdateAxisElement = [&](UGizmoElementRotateAxis* InAxisElement, const FGizmoElementRotateAxisStyleOverride& InAxisStyleOverride)
	{
		if (!ensure(InAxisElement))
		{
			return;
		}

		FGizmoElementRotateAxisStyle AxisStyle = Style;
		InAxisStyleOverride.ApplyTo(AxisStyle);

		InAxisElement->SetStyle(AxisStyle);
	};

	UpdateAxisElement(AxisElementX, StyleX);
	UpdateAxisElement(AxisElementY, StyleY);
	UpdateAxisElement(AxisElementZ, StyleZ);
}

void UGizmoElementRotateAxisSet::Setup(
	const EAxisList::Type InAxisList,
	const FGizmoElementRotateAxisStyle& InStyle,
	const FAxisParameters& InAxisX,
	const FAxisParameters& InAxisY,
	const FAxisParameters& InAxisZ)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementRotateAxisSetLocals;

	if (AxisElementX && AxisElementY && AxisElementZ)
	{
		return;
	}

	auto MakeAxisElement = [&](const FAxisParameters& InParameters)
	{
		FGizmoElementRotateAxisStyle AxisStyle = InStyle;
		InParameters.StyleOverride.ApplyTo(AxisStyle);

		UGizmoElementRotateAxis* AxisElement = NewObject<UGizmoElementRotateAxis>();

		AxisElement->Setup(
			InParameters.PartId,
			InParameters.Axis,
			AxisStyle);

		return AxisElement;
	};

	AxisElementX = MakeAxisElement(InAxisX);
	AxisElementY = MakeAxisElement(InAxisY);
	AxisElementZ = MakeAxisElement(InAxisZ);

	Add(AxisElementX);
	Add(AxisElementY);
	Add(AxisElementZ);

	bIsValid = true;

	SetStyle(InStyle, StyleX, StyleY, StyleZ);

	UpdateElements();
}

void UGizmoElementRotateAxisSet::UpdateElements()
{
	if (!bIsValid)
	{
		return;
	}

	auto UpdateAxisElement = [&](UGizmoElementRotateAxis* InAxisElement)
	{
		if (!ensure(InAxisElement))
		{
			return;
		}

		InAxisElement->UpdateElements();
	};

	UpdateAxisElement(AxisElementX);
	UpdateAxisElement(AxisElementY);
	UpdateAxisElement(AxisElementZ);
}

UGizmoElementRotateAxis* UGizmoElementRotateAxisSet::GetAxisElement(const EAxis::Type InAxis) const
{
	switch (InAxis)
	{
	case EAxis::X:
	if (ensure(AxisElementX))
	{
		return AxisElementX;
	}
	break;
		

	case EAxis::Y:
	if (ensure(AxisElementY))
	{
		return AxisElementY;
	}
	break;

	case EAxis::Z:
	if (ensure(AxisElementZ))
	{
		return AxisElementZ;
	}
	break;

	default:
		return nullptr;
	}

	return nullptr;
}

void UGizmoElementRotateAxisSet::ForEachAxisElement(const TFunctionRef<void(UGizmoElementRotateAxis* InElement)>& InFunc)
{
	if (AxisElementX)
	{
		InFunc(AxisElementX);
	}

	if (AxisElementY)
	{
		InFunc(AxisElementY);
	}

	if (AxisElementZ)
	{
		InFunc(AxisElementZ);
	}
}
