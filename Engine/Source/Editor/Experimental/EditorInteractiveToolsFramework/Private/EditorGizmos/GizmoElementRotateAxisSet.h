// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "BaseGizmos/GizmoElementGroup.h"
#include "GizmoElementRotateAxis.h"

#include "GizmoElementRotateAxisSet.generated.h"

class IToolkitHost;

/** This style contains optional per-axis overrides. FGizmoElementRotateAxisStyle is used for the shared style. */
USTRUCT(MinimalAPI)
struct FGizmoElementRotateAxisStyleOverride
{
	GENERATED_BODY()

	/** The Axis color, usually Red for X, etc. */
	UPROPERTY()
	TOptional<FGizmoPerStateValueLinearColor> Colors;

	/** Optional per-state material overrides for the axis. */
	UPROPERTY()
	TOptional<FGizmoPerStateValueMaterialVariant> Materials;

	/** Optional override for the material used to render the rotation delta arc. */
	UPROPERTY()
	TOptional<TObjectPtr<UMaterialInterface>> DeltaMaterial;

	/** Optional override for the vertex-color material used for the axis. */
	UPROPERTY()
	TOptional<TObjectPtr<UMaterialInterface>> VertexColorMaterial;

	/** Applies the overrides, if set, to the given Style. */
	void ApplyTo(FGizmoElementRotateAxisStyle& InOutStyle) const;
};

/** Represents a set of rotate-axis elements used in gizmo rendering. Provides functionality for managing and rendering multiple axis components as a single entity. */
UCLASS(Transient, MinimalAPI)
class UGizmoElementRotateAxisSet : public UGizmoElementGroupBase
{
	GENERATED_BODY()

public:
	/** Per-Axis parameters. */
	struct FAxisParameters
	{
		/** Hit-test part identifier for this axis element. */
		uint32 PartId;

		/** Which axis (X, Y, or Z) this element represents. */
		EAxisList::Type Axis;

		/** Per-axis style overrides applied on top of the shared style. */
		FGizmoElementRotateAxisStyleOverride StyleOverride;
	};

public:
	/** Renders various debug visualizations, if enabled. */
	void DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings, const uint32 InPartId = 0);

	/** Sets the host to add/remove a widget to.
	* Currently, this is a Toolkit host, but it could be abstracted to an ISlateWidgetHost (see IEditorViewportClientProxy). */
	void SetWidgetHost(IToolkitHost* const InWidgetHost);

	/** Enables the elements specified in the provided AxisList, and disables those that aren't. */
	void SetAxisEnabled(const EAxisList::Type InAxisListToEnable);

	/** Returns the current shared style used by this rotation axis set. */
	const FGizmoElementRotateAxisStyle& GetStyle() const;

	/** Sets and applies the given Style + Overrides. */
	void SetStyle(
		const FGizmoElementRotateAxisStyle& InStyle,
		const FGizmoElementRotateAxisStyleOverride& InAxisStyleX,
		const FGizmoElementRotateAxisStyleOverride& InAxisStyleY,
		const FGizmoElementRotateAxisStyleOverride& InAxisStyleZ);

	/**
	 * Initialize the rotation axis set with all three axis elements.
	 * @param InAxisList Which axes to initially enable.
	 * @param InStyle Shared visual style for all rotation axis elements.
	 * @param InAxisX Parameters for the X axis element.
	 * @param InAxisY Parameters for the Y axis element.
	 * @param InAxisZ Parameters for the Z axis element.
	 */
	void Setup(
		const EAxisList::Type InAxisList,
		const FGizmoElementRotateAxisStyle& InStyle,
		const FAxisParameters& InAxisX,
		const FAxisParameters& InAxisY,
		const FAxisParameters& InAxisZ);

	/** Rebuilds child elements to reflect the current style. Call after modifying style properties. */
	void UpdateElements();

	/** Get the rotation axis element for the given single axis (X, Y, or Z). Returns nullptr if not set up. */
	UGizmoElementRotateAxis* GetAxisElement(const EAxis::Type InAxis) const;

private:
	/** Invokes the given function on each axis element (X, Y, Z). */
	void ForEachAxisElement(const TFunctionRef<void(UGizmoElementRotateAxis* InElement)>& InFunc);

	/** Applies the current style and per-axis overrides to all axis elements. */
	void ApplyStyle();

private:
	/** Flag indicating element validity - generally true after Setup is called. */
	std::atomic_bool bIsValid = false;

	/** Style contains various uniform and varying properties that affect the appearance of this gizmo element. */
	UPROPERTY()
	FGizmoElementRotateAxisStyle Style;

	/** Optional overrides for the X axis. */
	UPROPERTY()
	FGizmoElementRotateAxisStyleOverride StyleX;

	/** Optional overrides for the Y axis. */
	UPROPERTY()
	FGizmoElementRotateAxisStyleOverride StyleY;

	/** Optional overrides for the Z axis. */
	UPROPERTY()
	FGizmoElementRotateAxisStyleOverride StyleZ;

	/** X axis rotation element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementRotateAxis> AxisElementX;

	/** Y axis rotation element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementRotateAxis> AxisElementY;

	/** Z axis rotation element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementRotateAxis> AxisElementZ;
};
