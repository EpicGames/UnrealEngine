// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/GizmoElementScale.h"

#include "Editor.h"
#include "TransformGizmoEditorSettings.h"

FGizmoElementScaleAxisStyle::FGizmoElementScaleAxisStyle()
{
	PixelHitDistanceThreshold = 8.0f;
}

const FText& FGizmoElementScaleAxisStyle::GetDeltaTextPrefixForScaleType(const EGizmoTransformScaleType InScaleType) const
{
	// The default prefix for both scale modes is empty
	static FText DefaultPrefix;

	if (InScaleType == EGizmoTransformScaleType::Default)
	{
		return DeltaOffsetTextPrefix.Get(DefaultPrefix);
	}
	else if (InScaleType == EGizmoTransformScaleType::PercentageBased)
	{
		return DeltaPercentageTextPrefix.Get(DefaultPrefix);
	}

	// (We don't reach this point, but some compilers demand it)
	return DefaultPrefix;
}

const FText& FGizmoElementScaleAxisStyle::GetDeltaTextSuffixForScaleType(const EGizmoTransformScaleType InScaleType) const
{
	if (InScaleType == EGizmoTransformScaleType::Default)
	{
		// The default suffix is empty
		static FText DefaultOffsetSuffix;
		return DeltaOffsetTextSuffix.Get(DefaultOffsetSuffix);
	}
	else if (InScaleType == EGizmoTransformScaleType::PercentageBased)
	{
		static FText DefaultPercentageSuffix = FText::FromString(TEXT("x"));
		return DeltaPercentageTextSuffix.Get(DefaultPercentageSuffix);
	}

	// (We don't reach this point, but some compilers demand it)
	return FText::GetEmpty();
}

FGizmoElementScalePlanarStyle::FGizmoElementScalePlanarStyle()
{
	PixelHitDistanceThreshold = 5.0f;
}

FGizmoElementScaleUniformStyle::FGizmoElementScaleUniformStyle()
{
	Colors.Default = FLinearColor(0.418984f, 0.745564f, 0.835069f);
	PixelHitDistanceThreshold = 2.0f;
}
