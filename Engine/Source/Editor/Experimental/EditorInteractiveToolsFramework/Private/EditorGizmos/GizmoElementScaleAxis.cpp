// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/GizmoElementScaleAxis.h"

#include "Algo/AnyOf.h"
#include "BaseGizmos/GizmoElementArrow.h"
#include "BaseGizmos/GizmoElementArrowHead.h"
#include "BaseGizmos/GizmoElementCylinder.h"
#include "BaseGizmos/GizmoElementLineStrip.h"
#include "BaseGizmos/GizmoMath.h"
#include "CircleTypes.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.inl"
#include "EditorGizmos/EditorGizmoMath.h"
#include "GizmoElementValueWidget.h"
#include "Math/UnitConversion.h"
#include "Misc/AxisDisplayInfo.h"
#include "Selection.h"
#include "ToolDataVisualizer.h"

DEFINE_LOG_CATEGORY_STATIC(LogGizmoElementScaleAxis, Log, All);

namespace UE::Editor::InteractiveToolsFramework::Private
{
	namespace GizmoElementScaleAxisLocals
	{
		// Compile-time toggle - some of the debug stuff is expensive to compute, so we can disable it
		constexpr bool bDrawDebug = true;
	}
}

UGizmoElementScaleAxis::UGizmoElementScaleAxis()
	: Axis()
	, NormalDirection()
	, UpDirection()
	, SideDirection()
	, DebugData()
{
	TextNumberFormattingOptions.AlwaysSign = true; // Always add +/-
	TextNumberFormattingOptions.UseGrouping = false;
	TextNumberFormattingOptions.RoundingMode = ERoundingMode::HalfToEven;
	TextNumberFormattingOptions.MinimumFractionalDigits = 2;
	TextNumberFormattingOptions.MaximumFractionalDigits = 2;
}

void UGizmoElementScaleAxis::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (!ensure(RenderAPI))
	{
		return;
	}

	CurrentState.DPIScale = static_cast<float>(FMath::Max(UE_DOUBLE_KINDA_SMALL_NUMBER, RenderAPI->GetCameraState().DPIScale));

	// To subtract from the regular line element - calculated here to account for PixelToWorldScale
	float DeltaLineLength = 0.0f;

	if (bIsShowingDelta)
	{
		// Delta Elements are based on the Start Transform, not current
		FRenderTraversalState DeltaRenderState(RenderState);
		if (UpdateDeltaRenderState(RenderAPI, FVector::ZeroVector, DeltaRenderState))
		{
			ApplyUniformConstantScaleToTransform(DeltaRenderState.PixelToWorldScale, DeltaRenderState.LocalToWorldTransform, true);

			auto RenderDeltaElement = [&](UGizmoElementBase* InElement)
			{
				if (InElement)
				{
					InElement->Render(RenderAPI, DeltaRenderState);
				}
			};

			DeltaLineLength = GetDeltaAxisLength(DeltaRenderState.LocalToWorldTransform);

			// Adjust so that the delta line only appears for positive scale - for negative scale, it only appears when the handle is on the opposing side of the axis.
			const float DeltaLineLengthChop = CurrentState.AxisSign < 0 ? GetTotalAxisLength() * CurrentState.DPIScale  : 0.0f;
			const float DeltaLineLengthMultiplier = (CurrentState.DeltaSign == 1 || (CurrentState.DeltaSign == -1 && CurrentState.AxisSign == -1)) ? 1.0f : 0.0f;
			const float AppliedDeltaLineLength = FMath::Abs((DeltaLineLength - DeltaLineLengthChop) * DeltaLineLengthMultiplier);

			if (DeltaLineElement)
			{
				DeltaLineElement->SetHeight(AppliedDeltaLineLength);
			}

			RenderDeltaElement(DeltaLineElement);
		}
	}

	// Non-Delta Elements
	FRenderTraversalState CurrentRenderState(RenderState);
	if (UpdateRenderState(RenderAPI, FVector::ZeroVector, CurrentRenderState))
	{
		if (ArrowBodyElement && CurrentState.AxisSign > 0)
		{
			if (bIsShowingDelta)
			{
				// We only care about negative lengths, then invert to subtract from the body length
				const float DeltaLineLengthToSubtract =
					FMath::Abs(
						FMath::Min(0.0f, (DeltaLineLength / CurrentState.DPIScale) * CurrentState.DeltaSign));

				ArrowBodyElement->SetHeight(FMath::Max(0.0f, GetAxisLength() - DeltaLineLengthToSubtract));	
			}

			if (CurrentState.AxisSign > 0)
			{
				ArrowBodyElement->Render(RenderAPI, CurrentRenderState);
			}
		}

		if (ArrowHeadElement)
		{
			ArrowHeadElement->SetCenter(GetHeadCenter(CurrentState.DPIScale));

			ArrowHeadElement->Render(RenderAPI, CurrentRenderState);
		}
	}
}

void UGizmoElementScaleAxis::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (bIsShowingDelta)
	{
		// FRenderTraversalState DeltaRenderState(RenderState);
		// UpdateDeltaRenderState(RenderAPI, FVector::ZeroVector, DeltaRenderState);

		FRenderTraversalState CurrentRenderState(RenderState);
		if (UpdateRenderState(RenderAPI, FVector::ZeroVector, CurrentRenderState))
		{
			if (DeltaWidgetElement)
			{
				const double TextOffsetFromCircle = 0.0;

				const FVector TextCenter = GetHeadCenter()
										+ (SideDirection * TextOffsetFromCircle);
				DeltaWidgetElement->SetLocation(TextCenter);

				DeltaWidgetElement->DrawHUD(Canvas, RenderAPI, CurrentRenderState);
			}
		}
	}
}

void UGizmoElementScaleAxis::DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings)
{
	if (bIsShowingDelta || GetElementInteractionState() == EGizmoElementInteractionState::Selected)
	{
		FToolDataVisualizer Draw;
		Draw.BeginFrame(RenderAPI);

		Draw.EndFrame();
	}
}

void UGizmoElementScaleAxis::SetWidgetHost(IToolkitHost* const InWidgetHost)
{
	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetWidgetHost(InWidgetHost);
	}
}

const FGizmoElementScaleAxisStyle& UGizmoElementScaleAxis::GetStyle() const
{
	return Style;
}

void UGizmoElementScaleAxis::SetStyle(const FGizmoElementScaleAxisStyle& InStyle)
{
	Style = InStyle;
	ApplyStyle();
}

const FGizmoElementScaleInteraction& UGizmoElementScaleAxis::GetInteraction() const
{
	return Interaction;
}

void UGizmoElementScaleAxis::SetInteraction(const FGizmoElementScaleInteraction& InInteraction)
{
	Interaction = InInteraction;
	ApplyInteraction();
}

UE::Geometry::FFrame3d UGizmoElementScaleAxis::MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const EAxisList::Type InAxisList) const
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;

	return MakeTransformedPlane(
		InTransform,
		InCoordinateSystem,
		NormalDirection,
		UpDirection,
		SideDirection);
}

void UGizmoElementScaleAxis::ApplyStyle()
{
	if (!bIsValid)
	{
		return;
	}

	SetVertexColor(Style.Colors.Default.Get(GetVertexColor()));

	const float SizeCoefficient = Style.SizeCoefficient.Get(1.0f);
	
	const float HeadSize = Style.HeadSize * Style.HandleSizeMultiplier;

	const float MinLineRadius = Style.MinLineThickness * 0.5f;
	const float MaxLineRadius = HeadSize * 0.5f; // Prevents the line radius becoming larger than and overshooting the axis "head"
	const float LineRadius = Style.LineThickness * 0.5f;

	const FVector BaseLocation = NormalDirection * Style.AxisOffsetFromCenter * SizeCoefficient;

	// Primary Arrow Element Body
	if (ArrowBodyElement)
	{
		ArrowBodyElement->SetNumSides(Style.NumSegments);
		ArrowBodyElement->SetBase(BaseLocation);
		ArrowBodyElement->SetHeight(GetAxisLength());
		ArrowBodyElement->SetRadius(FMath::Clamp(LineRadius * Style.LineThicknessMultiplier, MinLineRadius, MaxLineRadius) * SizeCoefficient);

		UE::Editor::InteractiveToolsFramework::ApplyMaterialsToElement(ArrowBodyElement, Style.Materials);
		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(ArrowBodyElement, Style.Colors);

		ArrowBodyElement->SetPixelHitDistanceThreshold(Style.PixelHitDistanceThreshold);
	}

	// Primary Arrow Element Head
	if (ArrowHeadElement)
	{
		ArrowHeadElement->SetCenter(GetHeadCenter());

		ArrowHeadElement->SetNumSides(Style.NumSegments);
		ArrowHeadElement->SetSideDirection(SideDirection);
		ArrowHeadElement->SetLength(HeadSize * SizeCoefficient);

		UE::Editor::InteractiveToolsFramework::ApplyMaterialsToElement(ArrowHeadElement, Style.Materials);
		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(ArrowHeadElement, Style.Colors);

		ArrowHeadElement->SetPixelHitDistanceThreshold(Style.PixelHitDistanceThreshold);
	}

	// Apply common delta element settings
	auto ApplyDeltaElementSettings = [&](UGizmoElementBase* InElement)
	{
		UE::Editor::InteractiveToolsFramework::ApplyMaterialsToElement(InElement, Style.Materials);
		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(InElement, Style.Colors);
	};

	// Delta Line Element
	if (DeltaLineElement)
	{
		DeltaLineElement->SetNumSides(Style.NumSegments);
		DeltaLineElement->SetBase(BaseLocation + NormalDirection * GetAxisLength());
		DeltaLineElement->SetHeight(0.0f); // Height is non-zero during delta operations only
		DeltaLineElement->SetRadius(FMath::Max(Style.MinLineThickness * 0.5f, Style.DeltaLineThickness.Get(Style.LineThickness) * 0.5f * Style.LineThicknessMultiplier) * SizeCoefficient);

		DeltaLineElement->SetIsDashed(true);
		DeltaLineElement->SetDashParameters(Style.DeltaLineDashSpacing, Style.DeltaLineDashGapSpacing);

		DeltaLineElement->SetScreenSpace(true);

		ApplyDeltaElementSettings(DeltaLineElement);
	}

	// Delta Widget Element
    if (DeltaWidgetElement)
    {
    	const FLinearColor BackgroundColor = Style.DeltaTextBackgroundColor;

    	DeltaWidgetElement->SetMaterial(Style.VertexColorMaterial);
    	DeltaWidgetElement->SetHoverMaterial(Style.VertexColorMaterial);
    	DeltaWidgetElement->SetInteractMaterial(Style.VertexColorMaterial);

    	DeltaWidgetElement->SetVertexColor(BackgroundColor);
    	DeltaWidgetElement->SetHoverVertexColor(BackgroundColor);
    	DeltaWidgetElement->SetInteractVertexColor(BackgroundColor);

    	DeltaWidgetElement->ClearSubdueVertexColor();
    	DeltaWidgetElement->ClearSelectVertexColor();
    }
}

void UGizmoElementScaleAxis::ApplyInteraction()
{
}

void UGizmoElementScaleAxis::Setup(
	const uint32 InPartId,
	const EAxisList::Type InAxis,
	const FGizmoElementScaleAxisStyle& InStyle)
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;
	using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementScaleAxisLocals;

	if (ArrowBodyElement)
	{
		return;
	}

	Axis = InAxis;

	GetAxisBasis<double>(InAxis, NormalDirection, UpDirection, SideDirection);

	// Primary Arrow Element Body
	if (!ArrowBodyElement)
	{
		ArrowBodyElement = NewObject<UGizmoElementCylinder>();
		ArrowBodyElement->SetPartIdentifier(InPartId, true, true);
		ArrowBodyElement->SetViewDependentType(EGizmoElementViewDependentType::Axis);
		ArrowBodyElement->SetViewDependentAxis(NormalDirection);

		Add(ArrowBodyElement);
	}

	// Primary Arrow Element Head
	if (!ArrowHeadElement)
	{
		ArrowHeadElement = NewObject<UGizmoElementArrowHead>();
		ArrowHeadElement->SetPartIdentifier(InPartId, true, true);
		ArrowHeadElement->SetType(EGizmoElementArrowHeadType::Cube);
		ArrowHeadElement->SetViewDependentType(EGizmoElementViewDependentType::Axis);
		ArrowHeadElement->SetViewDependentAxis(NormalDirection);

		Add(ArrowHeadElement);
	}

	// Apply common delta element settings
	auto ApplyDeltaElementSettings = [&, InPartId](UGizmoElementBase* InElement)
	{
		InElement->SetPartIdentifier(InPartId);

		InElement->SetHittableState(false);
		InElement->SetEnabledForDefaultState(false);
		InElement->SetEnabledForHoveringState(false);
		InElement->SetEnabledForInteractingState(true);
		InElement->SetEnabledForSelectedState(false);
		InElement->SetEnabledForSubduedState(false);
	};

	// Delta Line Element
	if (!DeltaLineElement)
	{
		DeltaLineElement = NewObject<UGizmoElementCylinder>();
		DeltaLineElement->SetPartIdentifier(InPartId);
		DeltaLineElement->SetViewDependentType(EGizmoElementViewDependentType::Axis);
		DeltaLineElement->SetViewDependentAxis(NormalDirection);

		ApplyDeltaElementSettings(DeltaLineElement);

		DeltaLineElement->SetVisibleState(false);

		Add(DeltaLineElement);
	}

	// Delta Widget Element
	if (!DeltaWidgetElement)
	{
		DeltaWidgetElement = NewObject<UGizmoElementValueWidget>();

		DeltaWidgetElement->SetPartIdentifier(InPartId);	
		DeltaWidgetElement->SetViewAlignType(EGizmoElementViewAlignType::PointScreen);
		DeltaWidgetElement->SetViewAlignAxis(FVector::UpVector);
		DeltaWidgetElement->SetViewAlignNormal(-FVector::ForwardVector);

		DeltaWidgetElement->SetLocation(FVector::ZeroVector);
		DeltaWidgetElement->SetClampToScreen(true);
		DeltaWidgetElement->SetClampToScreenFunction(
			[&](const FVector2D& InWidgetLocation, const FVector2D& InWidgetSize, const FIntRect& InViewRect) -> FVector2D
			{
				const FVector2d HalfViewSize = FVector2d(InViewRect.Size()) / 2.0;
				const FVector2d HalfWidgetSize = InWidgetSize / 2.0;

				// Assumes center alignment in canvas @see UGizmoElementWidget::UGizmoElementWidget()
				const FVector2d ExtraPadding(8.0f);
	
				const FVector2d PaddedViewMin = -HalfViewSize + HalfWidgetSize + ExtraPadding;
				const FVector2d PaddedViewMax = HalfViewSize - HalfWidgetSize - ExtraPadding;

				const bool bNeedsClamping = 
					(InWidgetLocation.X < PaddedViewMin.X) || (InWidgetLocation.X > PaddedViewMax.X)
					|| (InWidgetLocation.Y < PaddedViewMin.Y) || (InWidgetLocation.Y > PaddedViewMax.Y);

				const FVector2d Origin = StartState.TransformLocation2D - HalfViewSize;

				if (!bNeedsClamping)
				{
					return InWidgetLocation;
				}

				FBox2d PaddedViewBox(PaddedViewMin, PaddedViewMax);
				FVector2d::FReal ClippedStart, ClippedEnd;
				if (UE::Editor::GizmoMath::ClipLineToBox(
					Origin,
					InWidgetLocation,
					PaddedViewBox,
					ClippedStart,
					ClippedEnd))
				{
					return FVector2D(Origin + (InWidgetLocation - Origin) * ClippedEnd);
				}

				return InWidgetLocation;
			});

		ApplyDeltaElementSettings(DeltaWidgetElement);

		DeltaWidgetElement->ClearSubdueVertexColor();
		DeltaWidgetElement->ClearSelectVertexColor();

		DeltaWidgetElement->SetVisibleState(false);

		Add(DeltaWidgetElement);
	}

	SetPartIdentifier(InPartId, true);

	bIsValid = true;

	SetStyle(InStyle);

	UpdateElements();
}

void UGizmoElementScaleAxis::UpdateElements()
{
	if (!bIsValid)
	{
		return;
	}

	// Primary Arrow Element Body
	if (ArrowBodyElement)
	{
		ArrowBodyElement->SetDirection(NormalDirection);
	}

	// Primary Arrow Element Head
	if (ArrowHeadElement)
	{
		ArrowHeadElement->SetDirection(NormalDirection);
	}

	// Delta Line Element
	if (DeltaLineElement)
	{
		DeltaLineElement->SetDirection(NormalDirection);
	}

	// Delta Widget Element
	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetOffset2D(FVector2D(0.0f, 28.0f));
	}
}

void UGizmoElementScaleAxis::BeginDelta(const FDeltaParameters& InParameters)
{
	bIsShowingDelta = true;
	StartState.Initialize(InParameters);
	StartState.AxisSign = InParameters.Scale > -GetTotalAxisLength() ? 1 : -1;
	CurrentState = StartState;
	
	if (DeltaLineElement)
	{
		DeltaLineElement->SetVisibleState(EnumHasAnyFlags(Style.ShowFlags, EGizmoElementScaleShowFlags::DeltaLine));
	}

	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetVisibleState(InParameters.bIsTrustworthy && StartState.bIsSingleAxis && EnumHasAnyFlags(Style.ShowFlags, EGizmoElementScaleShowFlags::DeltaLabel));
		DeltaWidgetElement->SetClampToScreen(InParameters.bIsCursorInViewport); // Only set this on Begin, if the interaction starts within the bounds of the viewport
	}

	// Update Debug
	{
		DebugData.TransformStart = StartState.Transform;
	}

	// Update State
	UpdateDelta(InParameters);
}

void UGizmoElementScaleAxis::UpdateDelta(const FDeltaParameters& InParameters)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementScaleAxisLocals;

	if (!DeltaWidgetElement)
	{
		return;
	}

	CurrentState.Update(InParameters);
	CurrentState.AxisSign = InParameters.Scale > -GetTotalAxisLength() ? 1 : -1;

	if (ArrowHeadElement)
	{
		ArrowHeadElement->SetDirection(GetSignedDeltaDirection());
	}

	// Delta Line Element
	if (DeltaLineElement)
	{
		// If either sign is negative, we start at the origin rather than the head
		if (CurrentState.DeltaSign < 0 || CurrentState.AxisSign < 0)
		{
			DeltaLineElement->SetBase(FVector::ZeroVector);
		}
		else
		{
			DeltaLineElement->SetBase(NormalDirection * GetTotalAxisLength() * CurrentState.DPIScale );
		}

		DeltaLineElement->SetDirection(GetSignedDeltaDirection());
	}

	// Delta Text Elements
	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetVisibleState(InParameters.bIsTrustworthy && CurrentState.bIsSingleAxis && EnumHasAnyFlags(Style.ShowFlags, EGizmoElementScaleShowFlags::DeltaLabel));

		// Start at 1.0 for Percentage (multiplier), and 0.0 for Default (offset)
		const double ScaleBase = InParameters.ScaleType == EGizmoTransformScaleType::PercentageBased ? 1.0 : 0.0;

		// GetTotalAxisLength() returns the *visual* axis length where SizeCoefficient also scales the offset. 
		// Using it here would break the round-trip when SizeCoefficient != 1.0.
		const double LabelDivisor = Style.AxisOffsetFromCenter
			+ Style.AxisLength * Style.AxisLengthMultiplier * Style.SizeCoefficient.Get(1.0f);

		double ScaleFraction = ScaleBase + ((CurrentState.DeltaRadius * CurrentState.DeltaSign) / LabelDivisor);
		ScaleFraction = FMath::IsNearlyZero(ScaleFraction) ? UE_DOUBLE_SMALL_NUMBER : ScaleFraction; // Clamp minimum near 0 to ensure the prepended sign string is +

		const FText Prefix = Style.GetDeltaTextPrefixForScaleType(InParameters.ScaleType);
		const FText Suffix = Style.GetDeltaTextSuffixForScaleType(InParameters.ScaleType);

		const FText ScaleText = FText::Format(
			FText::FromString(TEXT("{0}{1}{2}")),
			Prefix,
			FText::AsNumber(ScaleFraction, &TextNumberFormattingOptions),
			Suffix);

		DeltaWidgetElement->SetText(ScaleText);
	}
}

void UGizmoElementScaleAxis::EndDelta()
{
	bIsShowingDelta = false;

	CurrentState.Reset();
	StartState = CurrentState;

	if (DeltaLineElement)
	{
		DeltaLineElement->SetVisibleState(false);
	}

	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetVisibleState(false);
	}

	if (ArrowBodyElement)
	{
		ArrowBodyElement->SetHeight(GetAxisLength());
	}

	if (ArrowHeadElement)
	{
		ArrowHeadElement->SetCenter(GetHeadCenter());
		ArrowHeadElement->SetDirection(GetSignedDeltaDirection());
	}
}

void UGizmoElementScaleAxis::FState::Initialize(const FDeltaParameters& InParameters)
{
	Transform = InParameters.Transform;
	TransformLocation2D = InParameters.TransformLocation2D;
	DeltaRadius = FMath::Abs(InParameters.Scale);
	DeltaSign = InParameters.Scale >= 0.0 ? 1 : -1;
	AxisSign = InParameters.Scale > -1.0 ? 1 : -1;
	bIsSingleAxis = UE::Editor::InteractiveToolsFramework::Internal::IsAxisSingular(InParameters.AxisList);
	PlaneIntersectionPoint = InParameters.PlaneIntersectionPoint;
}

void UGizmoElementScaleAxis::FState::Update(const FDeltaParameters& InParameters)
{
	Transform = InParameters.Transform;
	TransformLocation2D = InParameters.TransformLocation2D;
	DeltaRadius = FMath::Abs(InParameters.Scale);
	DeltaSign = InParameters.Scale > 0 ? 1 : -1;
	AxisSign = InParameters.Scale > -1 ? 1 : -1;
	PlaneIntersectionPoint = InParameters.PlaneIntersectionPoint;
}

void UGizmoElementScaleAxis::FState::Reset()
{
	Transform = FTransform::Identity;
	TransformLocation2D = FVector2D::ZeroVector;
	DeltaRadius = 0.0;
	DeltaSign = 1;
	AxisSign = 1;
	PlaneIntersectionPoint = FVector::ZeroVector;
}

FVector UGizmoElementScaleAxis::GetHeadCenter(const float InPixelToWorldScale) const
{
	const FVector BaseLocation = NormalDirection * ((Style.AxisOffsetFromCenter + Style.AxisLength * Style.AxisLengthMultiplier) * Style.SizeCoefficient.Get(1.0f));
	const FVector DeltaLocationOffset = NormalDirection * (CurrentState.DeltaSign * (GetDeltaAxisLength(StartState.Transform) / InPixelToWorldScale));
	return BaseLocation + DeltaLocationOffset;
}

float UGizmoElementScaleAxis::GetAxisLength() const
{
	return Style.AxisLength * Style.AxisLengthMultiplier * Style.SizeCoefficient.Get(1.0f);
}

float UGizmoElementScaleAxis::GetTotalAxisLength() const
{
	return (Style.AxisOffsetFromCenter + Style.AxisLength * Style.AxisLengthMultiplier) * Style.SizeCoefficient.Get(1.0f);
}

float UGizmoElementScaleAxis::GetDeltaAxisLength(const FTransform& InTransform) const
{
	return static_cast<float>(CurrentState.DeltaRadius);
}

FVector UGizmoElementScaleAxis::GetSignedDeltaDirection() const
{
	return NormalDirection * CurrentState.DeltaSign;
}

bool UGizmoElementScaleAxis::UpdateDeltaRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalOrigin, FRenderTraversalState& InOutRenderState)
{
	const FSceneView* SceneView = RenderAPI->GetSceneView();
	check(SceneView);
	const UE::GizmoRenderingUtil::FSceneViewWrapper SceneViewInterface(*SceneView);

	InOutRenderState.PixelToWorldScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale<double>(&SceneViewInterface, StartState.Transform.GetLocation());
	const FVector4 ScreenSpaceStart = SceneView->WorldToScreen(StartState.Transform.GetLocation());
	InOutRenderState.DepthAtPixelToWorldReferencePoint = ScreenSpaceStart.W;
	InOutRenderState.LocalToWorldTransform.SetLocation(StartState.Transform.GetLocation());
	InOutRenderState.LocalToWorldTransform.SetScale3D(FVector::OneVector);

	return UpdateRenderState(RenderAPI, InLocalOrigin, InOutRenderState);
}
