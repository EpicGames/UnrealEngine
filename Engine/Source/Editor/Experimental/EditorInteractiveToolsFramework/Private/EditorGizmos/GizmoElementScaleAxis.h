// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseGizmos/GizmoElementGroup.h"
#include "EditorGizmoElementInterfaces.h"
#include "EditorGizmos/GizmoElementScale.h"
#include "FrameTypes.h"

#include "GizmoElementScaleAxis.generated.h"

class IToolkitHost;
class UGizmoElementValueWidget;
class UGizmoElementRoundedRectangle;
class UGizmoElementArc;
class UGizmoElementArrow;
class UGizmoElementArrowHead;
class UGizmoElementBox;
class UGizmoElementCircle;
class UGizmoElementCylinder;
class UGizmoElementDashedLine;
class UGizmoElementLineStrip;
class UGizmoElementRectangle;
class UGizmoElementSphere;
class UGizmoElementText;
class UGizmoElementTorus;
class UGizmoViewContext;

/**
 * Combines the visual elements of a single scale axis: a cylinder body, an arrow head handle,
 * a delta line, and a value label shown during interaction.
 * Also implements IPlaneProvider to define the interaction plane for drag projection.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementScaleAxis
	: public UGizmoElementGroupBase
	, public UE::Editor::InteractiveToolsFramework::IPlaneProvider
{
	GENERATED_BODY()

	/** Allow UGizmoElementScaleGroup to access internals for coordinating multi-axis scale operations. */
	friend class UGizmoElementScaleGroup;

public:
	/** Parameters shared between BeginDelta and UpdateDelta calls, describing the current scale interaction state. */
	struct FDeltaParameters
	{
		/** The current world transform of the gizmo. */
		FTransform Transform = FTransform::Identity;

		/** Screen-space location of the gizmo origin, used for label placement. */
		FVector2D TransformLocation2D = FVector2D::ZeroVector;

		/** The active coordinate system (World, Local, etc.). */
		EToolContextCoordinateSystem CoordinateSystem = EToolContextCoordinateSystem::World;
		
		/** The scale type being used for the interaction. */
		EGizmoTransformScaleType ScaleType = EGizmoTransformScaleType::PercentageBased;

		/** Direction of the scale axis in the current coordinate system. */
		FVector AxisDirection = FVector::ZeroVector;

		/** Normal of the interaction plane. */
		FVector PlaneNormal = FVector::UpVector;

		/** World-space point where the interaction ray intersects the plane. */
		FVector PlaneIntersectionPoint = FVector::ZeroVector;

		/** Current scale factor (1.0 = no change). */
		double Scale = 1.0;

		/** Whether the cursor is currently within the viewport (affects delta label visibility). */
		bool bIsCursorInViewport = true;

		/** Whether this is an indirect interaction (e.g. input field rather than drag). */
		bool bIsIndirectInteraction = false;

		/** Whether the scale value is reliable (false when the cursor is near the gizmo origin, causing instability). */
		bool bIsTrustworthy = true;

		/** The axis or axes being scaled. */
		EAxisList::Type AxisList = EAxisList::None;
	};

public:
	UGizmoElementScaleAxis();

	//~ Begin UGizmoElementBase
	virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	//~ End UGizmoElementBase

	/** Renders various debug visualizations, if enabled. */
	void DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings);

	/** Sets the host to add/remove a widget to.
	* Currently, this is a Toolkit host, but it could be abstracted to an ISlateWidgetHost (see IEditorViewportClientProxy). */
	void SetWidgetHost(IToolkitHost* const InWidgetHost);

	/** Returns the current visual style for this scale axis element. */
	const FGizmoElementScaleAxisStyle& GetStyle() const;

	/** Sets and applies the given Style. */
	void SetStyle(const FGizmoElementScaleAxisStyle& InStyle);

	/** Returns the current interaction options for this scale axis element. */
	const FGizmoElementScaleInteraction& GetInteraction() const;

	/** Sets and applies the given Interaction options. */
	void SetInteraction(const FGizmoElementScaleInteraction& InInteraction);

	//~ Begin IPlaneProvider
	/** Creates the interaction plane for this scale axis, transformed into the provided coordinate space. */
	virtual UE::Geometry::FFrame3d MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const EAxisList::Type InAxisList) const override;
	//~ End IPlaneProvider

	/**
	 * Initialize the scale axis element with a part identifier, axis, and visual style.
	 * Must be called before the element can render or receive interactions.
	 * @param InPartId Unique identifier for hit-testing this axis.
	 * @param InAxis The axis this element represents (e.g. EAxisList::X).
	 * @param InStyle Visual style parameters for the axis.
	 */
	void Setup(
		const uint32 InPartId,
		const EAxisList::Type InAxis,
		const FGizmoElementScaleAxisStyle& InStyle);

	/** Rebuilds child elements to reflect the current style. Call after modifying style properties. */
	void UpdateElements();

	/** Begin a delta interaction, recording the start state. */
    void BeginDelta(const FDeltaParameters& InParameters);

    /** Update the delta visualization with the current scale factor. */
    void UpdateDelta(const FDeltaParameters& InParameters);

	/** End the current delta interaction and hide delta visuals. */
    void EndDelta();

private:
	/** Style contains various uniform and varying properties that affect the appearance of this gizmo element. */
	UPROPERTY()
	FGizmoElementScaleAxisStyle Style;

	/** Interaction options controlling how this scale axis responds to user input. */
	UPROPERTY()
	FGizmoElementScaleInteraction Interaction;

	/** Scale Axis Body. */
	UPROPERTY()
	TObjectPtr<UGizmoElementCylinder> ArrowBodyElement;

	/** Scale Axis Head. */
	UPROPERTY()
	TObjectPtr<UGizmoElementArrowHead> ArrowHeadElement;

	/** Dashed Line. */
	UPROPERTY()
	TObjectPtr<UGizmoElementCylinder> DeltaLineElement;

	/** Widget that displays the numeric scale value label during interaction. */
	UPROPERTY()
	TObjectPtr<UGizmoElementValueWidget> DeltaWidgetElement;

private:
	/** Applies the current Style to all child visual elements. */
	void ApplyStyle();
	/** Applies the current Interaction options to all child elements. */
	void ApplyInteraction();

	/** Returns the world-space center of the arrow head, optionally scaled by a pixel-to-world factor. */
	FVector GetHeadCenter(const float InPixelToWorldScale = 1.0f) const;

	/** Returns the visual length of the axis body (excluding offset from center). */
	float GetAxisLength() const;

	/** Get the axis length with offset-from-center. */
	float GetTotalAxisLength() const;

	/** Returns the length of the delta line for the given transform, accounting for pixel-to-world scaling. */
	float GetDeltaAxisLength(const FTransform& InTransform) const;

	/** Returns the axis direction vector, signed according to the current delta direction. */
	FVector GetSignedDeltaDirection() const;

	/**
	 * Updates the render state for delta elements during render traversal, using the start transform rather than the current.
	 * @return View-dependent visibility; true if this element is visible in the current view.
	 */
	bool UpdateDeltaRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalOrigin, FRenderTraversalState& InOutRenderState);

private:
	/** Flag indicating element validity - generally true after Setup is called. */
	std::atomic_bool bIsValid = false;

	/** The axis this element represents (e.g. EAxisList::X). */
	EAxisList::Type Axis = EAxisList::None;

	/** Normal (forward) direction along the axis in local space. */
	FVector NormalDirection = FVector::ForwardVector;

	/** Up direction perpendicular to the axis, used for interaction plane construction. */
	FVector UpDirection = FVector::UpVector;

	/** Side direction perpendicular to the axis, used for interaction plane construction. */
	FVector SideDirection = FVector::RightVector;

	/** Formatting options for the delta value label text. */
	FNumberFormattingOptions TextNumberFormattingOptions;

	/** Snapshot of the interaction state at a point in time, used to track start and current states during a scale drag. */
	struct FState
	{
		/** The world transform of the gizmo at this point in the interaction. */
		FTransform Transform = FTransform::Identity;

		/** The gizmo origin projected into 2D screen-space. */
		FVector2D TransformLocation2D = FVector2D::ZeroVector;

		/** Unsigned distance of the delta from the axis origin (always positive). */
		double DeltaRadius = 0.0;

		/** Sign of the delta relative to the start of the operation/arrow head direction. */
		int8 DeltaSign = 1;

		/** Sign of the axis relative to the actual object origin. */
		int8 AxisSign = 1;

		/** Whether the interaction is constrained to a single axis (as opposed to uniform/planar scale). */
		bool bIsSingleAxis = false;

		/** The current DPI Scale */
		float DPIScale = 1.0f;

		/** World-space point where the interaction ray intersects the drag plane. */
		FVector PlaneIntersectionPoint = FVector::ZeroVector;

		/** Initializes the state from the given delta parameters at the start of an interaction. */
		void Initialize(const FDeltaParameters& InParameters);

		/** Updates the state from the given delta parameters during an ongoing interaction. */
		void Update(const FDeltaParameters& InParameters);

		/** Resets all members to their default values. */
		void Reset();
	};

	/** Captured state at the beginning of the current scale interaction. */
	FState StartState;

	/** State updated each frame during the current scale interaction. */
	FState CurrentState;

	/** Whether delta visuals (line and value label) are currently being displayed. */
	bool bIsShowingDelta = false;

	/** Debug visualization data collected during interactions. */
	struct FGizmoDebugData
	{
		/** Determines whether certain data is displayed, ie. drag operation deltas. */
		bool bIsEditing = false;

		/** The gizmo transform captured at the start of the current drag operation, used for debug rendering. */
		FTransform TransformStart;
	} DebugData;
};
