// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementScaleGroup.h"

#include "BaseGizmos/GizmoElementBox.h"
#include "BaseGizmos/GizmoElementSphere.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.inl"
#include "EditorGizmos/EditorGizmoMath.h"
#include "GizmoElementScaleAxis.h"
#include "ToolDataVisualizer.h"

DEFINE_LOG_CATEGORY_STATIC(LogGizmoElementScaleGroup, Log, All);

namespace UE::Editor::InteractiveToolsFramework::Private
{
	namespace GizmoElementScaleAxisSetLocals
	{
		UGizmoElementScaleAxis::FDeltaParameters MakeAxisDeltaParameters(const UGizmoElementScaleGroup::FDeltaParameters& InGroupParameters, const EAxis::Type InAxis)
		{
			UGizmoElementScaleAxis::FDeltaParameters AxisDeltaParameters;
			AxisDeltaParameters.Transform = InGroupParameters.Transform;
			AxisDeltaParameters.TransformLocation2D = InGroupParameters.TransformLocation2D;
			AxisDeltaParameters.Scale = InGroupParameters.DeltaScale.GetComponentForAxis(InAxis);
			AxisDeltaParameters.CoordinateSystem = InGroupParameters.CoordinateSystem;
			AxisDeltaParameters.ScaleType = InGroupParameters.ScaleType;
			AxisDeltaParameters.PlaneNormal = InGroupParameters.PlaneNormal;
			AxisDeltaParameters.PlaneIntersectionPoint = InGroupParameters.PlaneIntersectionPoint;
			AxisDeltaParameters.bIsIndirectInteraction = InGroupParameters.bIsIndirectInteraction;
			AxisDeltaParameters.AxisList = InGroupParameters.AxisList;
			AxisDeltaParameters.bIsTrustworthy = InGroupParameters.bIsTrustworthy;

			AxisDeltaParameters.AxisDirection =
				InGroupParameters.CoordinateSystem == EToolContextCoordinateSystem::Local
				? InGroupParameters.Transform.GetUnitAxis(InAxis)
				: Internal::GetAxisVector(InAxis);

			return AxisDeltaParameters;
		}

		static bool ShouldShowElement(const bool bInIsAlternateElement,	const TFunctionRef<FToolContextSnappingConfiguration()>& InGetCurrentSnappingSettingsFunction)
		{
			auto GetCVarValue = []() -> int32
			{
				constexpr const TCHAR* CVarName = TEXT("Editor.Gizmo.AlternateSnappingElements");
				if (const IConsoleVariable* DebugDrawCVar = IConsoleManager::Get().FindConsoleVariable(CVarName, false))
				{
					return FMath::Clamp(DebugDrawCVar->GetInt(), 0, 2);
				}

				constexpr int32 DefaultValue = 0;
				return DefaultValue;
			};

			const int32 AlternateSnappingElementsOption = GetCVarValue();

			// Snapping Elements disabled, only show non-alternates
			if (AlternateSnappingElementsOption == 0)
			{
				return !bInIsAlternateElement;
			}

			const FToolContextSnappingConfiguration& CurrentSnappingSettings = InGetCurrentSnappingSettingsFunction();
			const bool bIsSnappingEnabled =	CurrentSnappingSettings.bEnableScaleGridSnapping;

			// Only show alternate when snapping on
			if (AlternateSnappingElementsOption == 1)
			{
				return bIsSnappingEnabled == bInIsAlternateElement;
			}

			// Inverse - only show alternate when snapping off, regular when on
			if (AlternateSnappingElementsOption == 2)
			{
				return bIsSnappingEnabled != bInIsAlternateElement;
			}

			// We shouldn't even be here
			return false;
		}
	}
}

void FGizmoElementScaleAxisStyleOverride::ApplyTo(FGizmoElementScaleAxisStyle& InOutStyle) const
{
	using namespace UE::Editor::InteractiveToolsFramework;

	if (Colors.IsSet())
	{
		ApplyColorOverrides(InOutStyle.Colors, Colors.GetValue());
	}

	if (Materials.IsSet())
	{
		ApplyMaterialOverrides(InOutStyle.Materials, Materials.GetValue());
	}

	if (VertexColorMaterial.IsSet())
	{
		InOutStyle.VertexColorMaterial = VertexColorMaterial.GetValue();
	}
}

void FGizmoElementScalePlanarStyleOverride::ApplyTo(FGizmoElementScalePlanarStyle& InOutStyle) const
{
	using namespace UE::Editor::InteractiveToolsFramework;

	if (Colors.IsSet())
	{
		ApplyColorOverrides(InOutStyle.Colors, Colors.GetValue());
	}

	if (Materials.IsSet())
	{
		ApplyMaterialOverrides(InOutStyle.Materials, Materials.GetValue());
	}

	if (VertexColorMaterial.IsSet())
	{
		InOutStyle.VertexColorMaterial = VertexColorMaterial.GetValue();
	}
}

void UGizmoElementScaleGroup::DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings, const uint32 InPartId)
{
	if (InPartId == 0)
	{
		ForEachAxisElement([&](UGizmoElementScaleAxis* InAxisElement)
    	{
    		if (InAxisElement && InAxisElement->GetElementInteractionState() != EGizmoElementInteractionState::None)
			{
				InAxisElement->DrawDebug(RenderAPI, InSettings);
			}
    	});
	}
	else
	{
		ForEachSubElementRecursive([&](UGizmoElementBase* InElement)
		{
			if (UGizmoElementScaleAxis* ScaleAxisElement = Cast<UGizmoElementScaleAxis>(InElement))
			{
				ScaleAxisElement->DrawDebug(RenderAPI, InSettings);
			}
		},
		InPartId);
	}
}

void UGizmoElementScaleGroup::SetWidgetHost(IToolkitHost* const InWidgetHost)
{
	if (!bIsValid)
	{
		return;
	}

	ForEachAxisElement([&](UGizmoElementScaleAxis* InAxisElement)
	{
		InAxisElement->SetWidgetHost(InWidgetHost);
	});
}

void UGizmoElementScaleGroup::SetGetCurrentSnappingSettingsFunction(const TFunction<FToolContextSnappingConfiguration()>& InFunction)
{
	GetCurrentSnappingSettingsFunction = InFunction;

	if (!bIsValid)
	{
		return;
	}

	ApplyGetCurrentSnappingSettingsFunction();
}

void UGizmoElementScaleGroup::SetAxisEnabled(const EAxisList::Type InAxisListToEnable)
{
	auto SetAxisEnabled = [&](UGizmoElementScaleAxis* InAxisElement, const EAxisList::Type InSingleAxis)
	{
		const bool bEnableAxis = static_cast<uint8>(InAxisListToEnable) & static_cast<uint8>(InSingleAxis);
		InAxisElement->SetEnabled(bEnableAxis);
	};

	SetAxisEnabled(AxisXElement, EAxisList::X);
	SetAxisEnabled(AxisYElement, EAxisList::Y);
	SetAxisEnabled(AxisZElement, EAxisList::Z);
}

void UGizmoElementScaleGroup::SetPlanarEnabled(const EAxisList::Type InAxisListToEnable)
{
	auto SetPlanarEnabled = [&](UGizmoElementBox* InPlanarElement, const EAxisList::Type InPlaneAxisPair)
	{
		if (!ensure(InPlanarElement))
		{
			return;
		}
		
		const bool bEnablePlanar = static_cast<uint8>(InAxisListToEnable) & static_cast<uint8>(InPlaneAxisPair);
		InPlanarElement->SetEnabled(bEnablePlanar);
	};

	SetPlanarEnabled(PlanarXYElement, EAxisList::XY);
	SetPlanarEnabled(PlanarYZElement, EAxisList::YZ);
	SetPlanarEnabled(PlanarXZElement, EAxisList::XZ);
}

const FGizmoElementScaleStyle& UGizmoElementScaleGroup::GetStyle() const
{
	return Style;
}

void UGizmoElementScaleGroup::SetStyle(
	const FGizmoElementScaleStyle& InStyle,
	const FGizmoElementScaleAxisStyleOverride& InAxisStyleX,
	const FGizmoElementScaleAxisStyleOverride& InAxisStyleY,
	const FGizmoElementScaleAxisStyleOverride& InAxisStyleZ,
	const FGizmoElementScalePlanarStyleOverride& InPlanarStyleXY,
	const FGizmoElementScalePlanarStyleOverride& InPlanarStyleYZ,
	const FGizmoElementScalePlanarStyleOverride& InPlanarStyleXZ)
{
	Style = InStyle;
	AxisXStyle = InAxisStyleX;
	AxisYStyle = InAxisStyleY;
	AxisZStyle = InAxisStyleZ;
	PlanarXYStyle = InPlanarStyleXY;
	PlanarYZStyle = InPlanarStyleYZ;
	PlanarXZStyle = InPlanarStyleXZ;

	ApplyStyle();
}

const FGizmoElementScaleInteraction& UGizmoElementScaleGroup::GetInteraction() const
{
	return Interaction;
}

void UGizmoElementScaleGroup::SetInteraction(const FGizmoElementScaleInteraction& InInteraction)
{
	Interaction = InInteraction;

	ApplyInteraction();
}

UE::Geometry::FFrame3d UGizmoElementScaleGroup::MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const EAxisList::Type InAxisList) const
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;
	
	return MakeTransformedPlaneForAxisList(
		InTransform,
		InViewContext,
		InCoordinateSystem,
		InAxisList,
		[this](const EAxisList::Type AxisList) -> const UGizmoElementBox*
		{
			return GetPlanarElement(AxisList);
		},
		[this](const EAxisList::Type AxisList) -> const IPlaneProvider*
		{
			return GetAxisElement(EAxis::FromAxisList(AxisList));
		});
}

void UGizmoElementScaleGroup::SetUniformEnabled(const bool bInEnable)
{
	if (UniformElement)
	{
		UniformElement->SetEnabled(bInEnable);
	}

	if (UniformElementAlternate)
	{
		UniformElementAlternate->SetEnabled(bInEnable);
	}
}

void UGizmoElementScaleGroup::Setup(
	const uint32 InPartId,
	const FGizmoElementScaleStyle& InStyle,
	const FAxisParameters& InAxisX,
	const FAxisParameters& InAxisY,
	const FAxisParameters& InAxisZ,
	const FPlanarParameters& InPlanarXY,
	const FPlanarParameters& InPlanarYZ,
	const FPlanarParameters& InPlanarXZ,
	const uint32 InUniformPartId)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementScaleAxisSetLocals;

	if (AxisXElement && AxisYElement && AxisZElement)
	{
		return;
	}

	// Axis Elements
	{
		auto MakeAxisElement = [&](const FAxisParameters& InParameters)
		{
			FGizmoElementScaleAxisStyle AxisStyle = InStyle.AxisStyle;
			InParameters.StyleOverride.ApplyTo(AxisStyle);

			UGizmoElementScaleAxis* AxisElement = NewObject<UGizmoElementScaleAxis>();
			AxisElement->Setup(
				InParameters.PartId,
				InParameters.Axis,
				AxisStyle);

			return AxisElement;
		};

		AxisXElement = MakeAxisElement(InAxisX);
		AxisYElement = MakeAxisElement(InAxisY);
		AxisZElement = MakeAxisElement(InAxisZ);

		Add(AxisXElement);
		Add(AxisYElement);
		Add(AxisZElement);
	}

	// Planar Elements
	{
		auto MakePlanarElement = [&](const FPlanarParameters& InParameters)
		{
			using namespace UE::Editor::InteractiveToolsFramework::Internal;

			FGizmoElementScalePlanarStyle PlanarStyle = InStyle.PlanarStyle;
			InParameters.StyleOverride.ApplyTo(PlanarStyle);

			const EAxisList::Type PlaneNormalAxis = GetPlaneNormalAxis(InParameters.Axis);

			FVector Normal;
			FVector Up;
			FVector Side;
			GetAxisBasis<FVector::FReal>(PlaneNormalAxis, Normal, Up, Side);

			UGizmoElementBox* BoxElement = NewObject<UGizmoElementBox>();
			BoxElement->SetPartIdentifier(InParameters.PartId, true);
			BoxElement->SetUpDirection(Up);
			BoxElement->SetSideDirection(Side);
			BoxElement->SetViewDependentType(EGizmoElementViewDependentType::Plane);
			BoxElement->SetViewDependentAxis(Normal);

			return BoxElement;
		};

		PlanarXYElement = MakePlanarElement(InPlanarXY);
		PlanarYZElement = MakePlanarElement(InPlanarYZ);
		PlanarXZElement = MakePlanarElement(InPlanarXZ);

		Add(PlanarXYElement);
		Add(PlanarYZElement);
		Add(PlanarXZElement);
	}

	constexpr int32 UniformHitPriority = 10;

	// Uniform Element
	{
		UniformElement = NewObject<UGizmoElementBox>();
		UniformElement->SetPartIdentifier(InUniformPartId, true);
		UniformElement->SetHitPriority(UniformHitPriority);
		UniformElement->SetCenter(FVector::ZeroVector);
		UniformElement->SetUpDirection(FVector::UpVector);
		UniformElement->SetSideDirection(FVector::RightVector);

		Add(UniformElement);
	}

	// Uniform Element (Alternate)
	{
		UniformElementAlternate = NewObject<UGizmoElementSphere>();
		UniformElementAlternate->SetPartIdentifier(InUniformPartId, true);
		UniformElementAlternate->SetHitPriority(UniformHitPriority);
		UniformElementAlternate->SetCenter(FVector::ZeroVector);

		Add(UniformElementAlternate);
	}

	SetPartIdentifier(InPartId, true);

	ApplyGetCurrentSnappingSettingsFunction();

	bIsValid = true;

	SetStyle(
		InStyle,
		InAxisX.StyleOverride,
		InAxisY.StyleOverride,
		InAxisZ.StyleOverride,
		InPlanarXY.StyleOverride,
		InPlanarYZ.StyleOverride,
		InPlanarXZ.StyleOverride);

	UpdateElements();
}

void UGizmoElementScaleGroup::UpdateElements()
{
	if (!bIsValid)
	{
		return;
	}
}

void UGizmoElementScaleGroup::BeginDelta(const FDeltaParameters& InParameters)
{
	bIsShowingDelta = true;
	StartState.Initialize(InParameters);
	CurrentState = StartState;

	ForEachAxisElementByList([&](UGizmoElementScaleAxis* InAxisElement, const EAxis::Type InAxis)
	{
		if (InAxisElement)
		{
			using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementScaleAxisSetLocals;
			const UGizmoElementScaleAxis::FDeltaParameters AxisDeltaParameters = MakeAxisDeltaParameters(InParameters, InAxis);

			InAxisElement->BeginDelta(AxisDeltaParameters);
		}
	},
	StartState.AxisList);

	// Update State
	UpdateDelta(InParameters);
}

void UGizmoElementScaleGroup::UpdateDelta(const FDeltaParameters& InParameters)
{
	CurrentState.Update(InParameters);

	ForEachAxisElementByList([&](UGizmoElementScaleAxis* InAxisElement, const EAxis::Type InAxis)
	{
		if (InAxisElement)
		{
			using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementScaleAxisSetLocals;
			const UGizmoElementScaleAxis::FDeltaParameters AxisDeltaParameters = MakeAxisDeltaParameters(InParameters, InAxis);

			InAxisElement->UpdateDelta(AxisDeltaParameters);
		}
	},
	StartState.AxisList);
}

void UGizmoElementScaleGroup::EndDelta()
{
	bIsShowingDelta = false;
	StartState = CurrentState;

	ForEachAxisElementByList([&](UGizmoElementScaleAxis* InAxisElement, const EAxis::Type InAxis)
	{
		if (InAxisElement)
		{
			InAxisElement->EndDelta();
		}
	},
	StartState.AxisList);
}

void UGizmoElementScaleGroup::FState::Initialize(const FDeltaParameters& InParameters)
{
	Transform = InParameters.Transform;
	TransformLocation2D = InParameters.TransformLocation2D;
	Scale = InParameters.DeltaScale;
	PlaneNormal = InParameters.PlaneNormal;
	PlaneIntersectionPoint = InParameters.PlaneIntersectionPoint;
	AxisList = InParameters.AxisList;
	bIsSingleAxis = UE::Editor::InteractiveToolsFramework::Internal::IsAxisSingular(AxisList);
}

void UGizmoElementScaleGroup::FState::Update(const FDeltaParameters& InParameters)
{
	Transform = InParameters.Transform;
	TransformLocation2D = InParameters.TransformLocation2D;
	Scale = InParameters.DeltaScale;
	PlaneIntersectionPoint = InParameters.PlaneIntersectionPoint;
}

UGizmoElementScaleAxis* UGizmoElementScaleGroup::GetAxisElement(const EAxis::Type InAxis) const
{
	switch (InAxis)
	{
	case EAxis::X:
	if (ensure(AxisXElement))
	{
		return AxisXElement;
	}
	break;

	case EAxis::Y:
	if (ensure(AxisYElement))
	{
		return AxisYElement;
	}
	break;

	case EAxis::Z:
	if (ensure(AxisZElement))
	{
		return AxisZElement;
	}
	break;

	default:
		return nullptr;
	}

	return nullptr;
}

UGizmoElementBox* UGizmoElementScaleGroup::GetPlanarElement(const EAxisList::Type InAxis) const
{
	switch (InAxis)
	{
	case EAxisList::XY:
	if (ensure(PlanarXYElement))
	{
		return PlanarXYElement;
	}
	break;

	case EAxisList::YZ:
	if (ensure(PlanarYZElement))
	{
		return PlanarYZElement;
	}
	break;

	case EAxisList::XZ:
	if (ensure(PlanarXZElement))
	{
		return PlanarXZElement;
	}
	break;

	default:
		return nullptr;
	}

	return nullptr;
}

void UGizmoElementScaleGroup::ForEachAxisElement(const TFunctionRef<void(UGizmoElementScaleAxis* InElement)>& InFunc)
{
	if (AxisXElement)
	{
		InFunc(AxisXElement);
	}

	if (AxisYElement)
	{
		InFunc(AxisYElement);
	}

	if (AxisZElement)
	{
		InFunc(AxisZElement);
	}
}

void UGizmoElementScaleGroup::ForEachAxisElementByList(const TFunctionRef<void(UGizmoElementScaleAxis* InElement, const EAxis::Type InAxis)>& InFunc, const EAxisList::Type InAxisList) const
{
	auto CallOnElement = [&](UGizmoElementScaleAxis* InElement, const EAxis::Type InAxis, const EAxisList::Type InAxisXYZ, const EAxisList::Type InAxisLUF)
	{
		if ((EnumHasAnyFlags(InAxisList, InAxisXYZ)
			|| EnumHasAnyFlags(InAxisList, InAxisLUF))
			&& InElement)
		{
			InFunc(InElement, InAxis);
		}
	};

	CallOnElement(AxisXElement, EAxis::X, EAxisList::X, EAxisList::Forward);
	CallOnElement(AxisYElement, EAxis::Y, EAxisList::Y, EAxisList::Left);
	CallOnElement(AxisZElement, EAxis::Z, EAxisList::Z, EAxisList::Up);
}

void UGizmoElementScaleGroup::ApplyStyle()
{
	if (!bIsValid)
	{
		return;
	}

	if (Style.Colors.Default.IsSet())
	{
		SetVertexColor(Style.Colors.Default.GetValue());
	}

	auto UpdateAxisElement = [&](UGizmoElementScaleAxis* InAxisElement, const FGizmoElementScaleAxisStyleOverride& InAxisStyleOverride)
	{
		if (!ensure(InAxisElement))
		{
			return;
		}

		FGizmoElementScaleAxisStyle AxisStyle = Style.AxisStyle;
		InAxisStyleOverride.ApplyTo(AxisStyle);

		InAxisElement->SetStyle(AxisStyle);
	};

	auto UpdatePlanarElement = [&, SizeCoefficient = Style.SizeCoefficient.Get(1.0f)](UGizmoElementBox* InPlanarElement, const FGizmoElementScalePlanarStyleOverride& InPlanarStyleOverride, const EAxisList::Type InPlaneAxis)
	{
		if (!ensure(InPlanarElement))
		{
			return;
		}

		using namespace UE::Editor::InteractiveToolsFramework::Internal;

		FGizmoElementScalePlanarStyle PlanarStyle = Style.PlanarStyle;
		InPlanarStyleOverride.ApplyTo(PlanarStyle);

		const EAxisList::Type PlaneNormalAxis = GetPlaneNormalAxis(InPlaneAxis);

		// The two vectors below are used for positive offsets, so we convert them from cartesian (might be negative) to UE handedness, depending on the current coordinate system
		FVector UnusedForwardAxis;
		FVector PositiveUpDirection;
		FVector PositiveSideDirection;
		GetSignedAxisBasis(PlaneNormalAxis, UnusedForwardAxis, PositiveUpDirection, PositiveSideDirection);

		const float PlanarSize = PlanarStyle.Size * PlanarStyle.SizeMultiplier;

		// Prevent the handles from being centered (or negative)
		const float MinOffsetFromOrigin = PlanarSize * 0.5f;
		const FVector PlanarHandleCenter = (PositiveUpDirection + PositiveSideDirection) * FMath::Max(MinOffsetFromOrigin, PlanarStyle.OffsetFromOrigin) * SizeCoefficient;

		InPlanarElement->SetCenter(PlanarHandleCenter);
		InPlanarElement->SetDimensions(FVector(PlanarStyle.Thickness * PlanarStyle.LineThicknessMultiplier, PlanarSize, PlanarSize) * SizeCoefficient);

		using namespace UE::Editor::InteractiveToolsFramework;

		ApplyMaterialsToElement(InPlanarElement, PlanarStyle.Materials);
		ApplyColorsToElement(InPlanarElement, PlanarStyle.Colors);

		InPlanarElement->SetPixelHitDistanceThreshold(PlanarStyle.PixelHitDistanceThreshold);
	};

	UpdateAxisElement(AxisXElement, AxisXStyle);
	UpdateAxisElement(AxisYElement, AxisYStyle);
	UpdateAxisElement(AxisZElement, AxisZStyle);

	UpdatePlanarElement(PlanarXYElement, PlanarXYStyle, EAxisList::XY);
	UpdatePlanarElement(PlanarYZElement, PlanarYZStyle, EAxisList::YZ);
	UpdatePlanarElement(PlanarXZElement, PlanarXZStyle, EAxisList::XZ);

	auto UpdateUniformElement = [&](UGizmoElementBase* InUniformElement)
	{
		if (!ensure(InUniformElement))
		{
			return;
		}

		using namespace UE::Editor::InteractiveToolsFramework;

		ApplyMaterialsToElement(InUniformElement, Style.UniformStyle.Materials);
		ApplyColorsToElement(InUniformElement, Style.UniformStyle.Colors);

		InUniformElement->SetPixelHitDistanceThreshold(Style.UniformStyle.PixelHitDistanceThreshold);
	};

	const float SizeCoefficient = Style.SizeCoefficient.Get(1.0f);

	// Uniform Element
	if (UniformElement)
	{
		UpdateUniformElement(UniformElement);
		UniformElement->SetDimensions(FVector(Style.UniformStyle.Size * Style.UniformStyle.SizeMultiplier) * SizeCoefficient);	
	}

	// Uniform Element (Alternate)
	if (UniformElementAlternate)
	{
		UpdateUniformElement(UniformElementAlternate);
		UniformElementAlternate->SetRadius(Style.UniformStyle.Size * 0.5f * SizeCoefficient);
	}
}

void UGizmoElementScaleGroup::ApplyInteraction()
{
	if (!bIsValid)
	{
		return;
	}

	auto UpdateAxisElement = [&](UGizmoElementScaleAxis* InAxisElement, const FGizmoElementScaleInteraction& InInteraction)
	{
		if (!ensure(InAxisElement))
		{
			return;
		}

		InAxisElement->SetInteraction(InInteraction);
	};

	UpdateAxisElement(AxisXElement, Interaction);
	UpdateAxisElement(AxisYElement, Interaction);
	UpdateAxisElement(AxisZElement, Interaction);
}

void UGizmoElementScaleGroup::ApplyGetCurrentSnappingSettingsFunction()
{
	// Only applies if set
	if (!GetCurrentSnappingSettingsFunction.IsSet())
	{
		return;
	}

	using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementScaleAxisSetLocals;

	if (UniformElement)
	{
		constexpr bool bIsAlternateElement = false;

		UniformElement->SetIsVisibleFunction([GetCurrentSnappingSettingsFunction = GetCurrentSnappingSettingsFunction](
			const FSceneView*, EViewInteractionState, EGizmoElementInteractionState, 
			const FTransform&, const FVector&) -> bool
		{
			return ShouldShowElement(bIsAlternateElement, GetCurrentSnappingSettingsFunction);
		});

		UniformElement->SetIsHittableFunction([GetCurrentSnappingSettingsFunction = GetCurrentSnappingSettingsFunction](const UGizmoViewContext*, const FTransform&, const FVector&) -> bool
		{
			return ShouldShowElement(bIsAlternateElement, GetCurrentSnappingSettingsFunction);
		});
	}

	if (UniformElementAlternate)
	{
		constexpr bool bIsAlternateElement = true;

		UniformElementAlternate->SetIsVisibleFunction([GetCurrentSnappingSettingsFunction = GetCurrentSnappingSettingsFunction](
			const FSceneView*, EViewInteractionState, EGizmoElementInteractionState, 
			const FTransform&, const FVector&) -> bool
		{
			return ShouldShowElement(bIsAlternateElement, GetCurrentSnappingSettingsFunction);
		});

		UniformElementAlternate->SetIsHittableFunction([GetCurrentSnappingSettingsFunction = GetCurrentSnappingSettingsFunction](const UGizmoViewContext*, const FTransform&, const FVector&) -> bool
		{
			return ShouldShowElement(bIsAlternateElement, GetCurrentSnappingSettingsFunction);
		});
	}
}
