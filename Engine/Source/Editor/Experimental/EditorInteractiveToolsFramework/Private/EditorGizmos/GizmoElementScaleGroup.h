// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "BaseGizmos/GizmoElementGroup.h"
#include "EditorGizmoElementInterfaces.h"
#include "EditorGizmos/GizmoElementScale.h"
#include "FrameTypes.h"

#include "GizmoElementScaleGroup.generated.h"

class UGizmoElementSphere;
class IToolkitHost;
class UGizmoElementScaleAxis;
class UGizmoElementBox;

/** This style contains optional per-axis overrides. FGizmoElementScaleAxisStyle is used for the shared style. */
USTRUCT(MinimalAPI)
struct FGizmoElementScaleAxisStyleOverride
{
	GENERATED_BODY()

	/** The Axis color, usually Red for X, etc. */
	UPROPERTY()
	TOptional<FGizmoPerStateValueLinearColor> Colors;

	/** Optional override for the per-state materials. */
	UPROPERTY()
	TOptional<FGizmoPerStateValueMaterialVariant> Materials;

	/** Optional override for the vertex color material. */
	UPROPERTY()
	TOptional<TObjectPtr<UMaterialInterface>> VertexColorMaterial;

	/** Applies the overrides, if set, to the given Style. */
	void ApplyTo(FGizmoElementScaleAxisStyle& InOutStyle) const;
};

/** This style contains optional per-planar overrides. FGizmoElementScalePlanarStyle is used for the shared style. */
USTRUCT(MinimalAPI)
struct FGizmoElementScalePlanarStyleOverride
{
	GENERATED_BODY()

	/** The Plane color, usually Red for YZ, etc. */
	UPROPERTY()
	TOptional<FGizmoPerStateValueLinearColor> Colors;

	/** Optional override for the per-state materials. */
	UPROPERTY()
	TOptional<FGizmoPerStateValueMaterialVariant> Materials;

	/** Optional override for the vertex color material. */
	UPROPERTY()
	TOptional<TObjectPtr<UMaterialInterface>> VertexColorMaterial;

	/** Applies the overrides, if set, to the given Style. */
	void ApplyTo(FGizmoElementScalePlanarStyle& InOutStyle) const;
};

/**
 * Container for all scale gizmo elements: per-axis handles (X, Y, Z),
 * planar handles (XY, YZ, XZ), and the uniform screen-space handle.
 * Manages style propagation, interaction options, axis enable/disable, and delta visualization across all children.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementScaleGroup
	: public UGizmoElementGroupBase
	, public UE::Editor::InteractiveToolsFramework::IPlaneProvider
{
	GENERATED_BODY()

public:
	/** Per-Axis parameters. */
	struct FAxisParameters
	{
		/** Part identifier for this axis element. */
		uint32 PartId = 0;

		/** Which axis this element represents. */
		EAxisList::Type Axis = EAxisList::None;

		/** Optional style overrides specific to this axis. */
		FGizmoElementScaleAxisStyleOverride StyleOverride;
	};

	/** Per-Plane parameters. */
	struct FPlanarParameters
	{
		/** Part identifier for this planar element. */
		uint32 PartId = 0;

		/** Which axis pair this planar element represents (e.g. XY, YZ, XZ). */
		EAxisList::Type Axis = EAxisList::None;

		/** Optional style overrides specific to this plane. */
		FGizmoElementScalePlanarStyleOverride StyleOverride;
	};

	/** Parameters shared between BeginDelta and UpdateDelta calls, describing the current scale interaction state. */
	struct FDeltaParameters
	{
		/** The current world transform of the gizmo. */
		FTransform Transform = FTransform::Identity;

		/** Screen-space location of the gizmo origin, used for label placement. */
		FVector2D TransformLocation2D = FVector2D::ZeroVector;

		/** Per-axis scale delta from the start of the interaction. */
		FVector DeltaScale = FVector::ZeroVector;

		/** The active coordinate system (World, Local, etc.). */
		EToolContextCoordinateSystem CoordinateSystem = EToolContextCoordinateSystem::World;
		
		/** The scale type being used for the interaction. */
		EGizmoTransformScaleType ScaleType = EGizmoTransformScaleType::PercentageBased;

		/** Normal of the interaction plane. */
		FVector PlaneNormal = FVector::UpVector;

		/** World-space point where the interaction ray intersects the plane. */
		FVector PlaneIntersectionPoint = FVector::ZeroVector;

		/** The axis or axes being scaled. */
		EAxisList::Type AxisList = EAxisList::None;

		/** Whether this is an indirect interaction (e.g. input field rather than drag). */
		bool bIsIndirectInteraction = false;

		/** Whether the scale value is reliable (false when the cursor is near the gizmo origin, causing instability). */
		bool bIsTrustworthy = true;
	};

public:
	/** Renders various debug visualizations, if enabled. */
	void DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings, const uint32 InPartId = 0);

	/** Sets the host to add/remove a widget to.
	* Currently, this is a Toolkit host, but it could be abstracted to an ISlateWidgetHost (see IEditorViewportClientProxy). */
	void SetWidgetHost(IToolkitHost* const InWidgetHost);

	/** Sets a function that provides the current snapping settings to use during interaction. */
	void SetGetCurrentSnappingSettingsFunction(const TFunction<FToolContextSnappingConfiguration()>& InFunction);

	/** Enables the elements specified in the provided AxisList and disables those that aren't. */
	void SetAxisEnabled(const EAxisList::Type InAxisListToEnable);

	/** Enables the elements specified in the provided AxisList and disables those that aren't. */
	void SetPlanarEnabled(const EAxisList::Type InAxisListToEnable);

	/** Enables or disables the uniform screen-space scale element. */
	void SetUniformEnabled(const bool bInEnable);

	/** Returns the current shared scale style. */
	const FGizmoElementScaleStyle& GetStyle() const;

	/** Sets and applies the given Style + Overrides. */
	void SetStyle(
		const FGizmoElementScaleStyle& InStyle,
		const FGizmoElementScaleAxisStyleOverride& InAxisStyleX,
		const FGizmoElementScaleAxisStyleOverride& InAxisStyleY,
		const FGizmoElementScaleAxisStyleOverride& InAxisStyleZ,
		const FGizmoElementScalePlanarStyleOverride& InPlanarStyleXY,
		const FGizmoElementScalePlanarStyleOverride& InPlanarStyleYZ,
		const FGizmoElementScalePlanarStyleOverride& InPlanarStyleXZ);

	/** Returns the current interaction options. */
	const FGizmoElementScaleInteraction& GetInteraction() const;

	/** Sets and applies the given Interaction options. */
	void SetInteraction(const FGizmoElementScaleInteraction& InInteraction);

	//~ Begin IPlaneProvider
	/** Creates the interaction plane for the active scale axis or plane, transformed into the provided coordinate space. */
	virtual UE::Geometry::FFrame3d MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const EAxisList::Type InAxisList) const override;
	//~ End IPlaneProvider

	/**
	 * Initialize the scale group with all axis, planar, and uniform elements.
	 * Must be called before the group can render or receive interactions.
	 * @param InPartId Base part identifier for the group.
	 * @param InStyle Shared visual style for all scale elements.
	 * @param InAxisX Parameters for the X axis element.
	 * @param InAxisY Parameters for the Y axis element.
	 * @param InAxisZ Parameters for the Z axis element.
	 * @param InPlanarXY Parameters for the XY planar element.
	 * @param InPlanarYZ Parameters for the YZ planar element.
	 * @param InPlanarXZ Parameters for the XZ planar element.
	 * @param InUniformPartId Part identifier for the uniform screen-space element.
	 */
	void Setup(
		const uint32 InPartId,
		const FGizmoElementScaleStyle& InStyle,
		const FAxisParameters& InAxisX,
		const FAxisParameters& InAxisY,
		const FAxisParameters& InAxisZ,
		const FPlanarParameters& InPlanarXY,
		const FPlanarParameters& InPlanarYZ,
		const FPlanarParameters& InPlanarXZ,
		const uint32 InUniformPartId);

	/** Rebuilds child elements to reflect the current style. Call after modifying style properties. */
	void UpdateElements();

	/** Get the scale axis element for the given single axis (X, Y, or Z). Returns nullptr if not set up. */
	UGizmoElementScaleAxis* GetAxisElement(const EAxis::Type InAxis) const;

	/** Get the planar element for the given axis pair (XY, YZ, or XZ). Returns nullptr if not set up. */
	UGizmoElementBox* GetPlanarElement(const EAxisList::Type InAxis) const;

	/** Begin a delta interaction, recording the start state. Propagates to the active axis element. */
	void BeginDelta(const FDeltaParameters& InParameters);

	/** Update the delta visualization with the current scale factors. */
	void UpdateDelta(const FDeltaParameters& InParameters);

	/** End the current delta interaction and hide delta visuals. */
	void EndDelta();

private:
	/** Invokes the given function for each axis element (X, Y, Z). */
	void ForEachAxisElement(const TFunctionRef<void(UGizmoElementScaleAxis* InElement)>& InFunc);

	/** Invokes the given function for each axis element whose axis is included in InAxisList. */
	void ForEachAxisElementByList(const TFunctionRef<void(UGizmoElementScaleAxis* InElement, const EAxis::Type InAxis)>& InFunc, const EAxisList::Type InAxisList) const;

	/** Applies the current style and per-axis/planar overrides to all child elements. */
	void ApplyStyle();
	/** Applies the current interaction options to all child elements. */
	void ApplyInteraction();

	/** Applies and propagates the current GetCurrentSnappingSettingsFunction, if set. */
	void ApplyGetCurrentSnappingSettingsFunction();

private:
	/** Whether this group has been initialized via Setup(). */
	std::atomic_bool bIsValid = false;
	
	/** Style contains various uniform and varying properties that affect the appearance of this gizmo element. */
	UPROPERTY()
	FGizmoElementScaleStyle Style;

	/** Interaction options controlling how scale handles behave during user input. */
	UPROPERTY()
	FGizmoElementScaleInteraction Interaction;

	/** Optional overrides for the X axis. */
	UPROPERTY()
	FGizmoElementScaleAxisStyleOverride AxisXStyle;

	/** Optional overrides for the Y axis. */
	UPROPERTY()
	FGizmoElementScaleAxisStyleOverride AxisYStyle;

	/** Optional overrides for the Z axis. */
	UPROPERTY()
	FGizmoElementScaleAxisStyleOverride AxisZStyle;

	/** Optional overrides for the XY plane. */
	UPROPERTY()
	FGizmoElementScalePlanarStyleOverride PlanarXYStyle;

	/** Optional overrides for the YZ plane. */
	UPROPERTY()
	FGizmoElementScalePlanarStyleOverride PlanarYZStyle;

	/** Optional overrides for the XZ plane. */
	UPROPERTY()
	FGizmoElementScalePlanarStyleOverride PlanarXZStyle;

	/** Axis X element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementScaleAxis> AxisXElement;

	/** Axis Y element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementScaleAxis> AxisYElement;
	
	/** Axis Z element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementScaleAxis> AxisZElement;

	/** Planar XY element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> PlanarXYElement;

	/** Planar YZ element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> PlanarYZElement;

	/** Planar XZ element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> PlanarXZElement;

	/** Uniform element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> UniformElement;
	
	/** Uniform element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementSphere> UniformElementAlternate;

	/** Callback that provides the current snapping configuration for scale interactions. */
	TFunction<FToolContextSnappingConfiguration()> GetCurrentSnappingSettingsFunction = nullptr;

private:
	/** Snapshot of interaction state used to track the start and current values during a delta operation. */
	struct FState
	{
		/** World transform of the gizmo at this point in the interaction. */
		FTransform Transform = FTransform::Identity;

		/** Screen-space location of the gizmo origin. */
		FVector2D TransformLocation2D = FVector2D::ZeroVector;

		/** Per-axis scale values at this point in the interaction. */
		FVector Scale = FVector::ZeroVector;

		/** Normal of the interaction plane. */
		FVector PlaneNormal = FVector::UpVector;

		/** World-space point where the interaction ray intersects the plane. */
		FVector PlaneIntersectionPoint = FVector::ZeroVector;

		/** The axis or axes being scaled. */
		EAxisList::Type AxisList = EAxisList::None;

		/** True when only a single axis is being scaled (not planar or uniform). */
		bool bIsSingleAxis = false;

		/** Distance from the interaction start point along the interaction plane. */
		double DistanceFromStart = 0.0;

		/** Initializes this state from the given delta parameters at the start of an interaction. */
		void Initialize(const FDeltaParameters& InParameters);

		/** Updates this state from the given delta parameters during an ongoing interaction. */
		void Update(const FDeltaParameters& InParameters);
	};

	/** State captured at the beginning of the current delta interaction. */
	FState StartState;

	/** State representing the most recent update of the current delta interaction. */
	FState CurrentState;

	/** Whether a delta visualization is currently being displayed. */
	bool bIsShowingDelta = false;
};
