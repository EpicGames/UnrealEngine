// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/GizmoElementTranslate.h"

FGizmoElementTranslateAxisStyle::FGizmoElementTranslateAxisStyle()
{
	PixelHitDistanceThreshold = 8.0f;
}

FGizmoElementTranslatePlanarStyle::FGizmoElementTranslatePlanarStyle()
{
	PixelHitDistanceThreshold = 5.0f;
}

FGizmoElementTranslateUniformStyle::FGizmoElementTranslateUniformStyle()
{
	Colors.Default = FLinearColor(0.418984f, 0.745564f, 0.835069f);
	PixelHitDistanceThreshold = 2.0f;
}
