// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/GizmoElementTranslateAxis.h"

#include "Algo/AnyOf.h"
#include "BaseGizmos/GizmoElementArc.h"
#include "BaseGizmos/GizmoElementArrow.h"
#include "BaseGizmos/GizmoElementLineStrip.h"
#include "BaseGizmos/GizmoMath.h"
#include "CircleTypes.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.inl"
#include "EditorGizmos/EditorGizmoMath.h"
#include "GizmoElementValueWidget.h"
#include "Math/UnitConversion.h"
#include "Misc/AxisDisplayInfo.h"

DEFINE_LOG_CATEGORY_STATIC(LogGizmoElementTranslateAxis, Log, All);

namespace UE::Editor::InteractiveToolsFramework::Private
{
	namespace GizmoElementTranslateAxisLocals
	{
		FVector2f GetTextOffsetFromCircle(const Geometry::FCircle2f& InCircle, const float InAngleOnCircle, const FBox2f& InTextBox)
		{
			FVector2f::FReal SinTheta = 0.0f;
			FVector2f::FReal CosTheta = 0.0f;
			FMath::SinCos(&SinTheta, &CosTheta, InAngleOnCircle);

			const FVector2f TextBoxSizeHalf = InTextBox.GetExtent();
			
			FVector2f TextOffset = FVector2f(
					InCircle.Center.X + CosTheta * InCircle.Radius + FMath::Sign(CosTheta) * TextBoxSizeHalf.X,
					InCircle.Center.Y + (InCircle.Radius + TextBoxSizeHalf.Y) * SinTheta
				);

			return TextOffset;
		}

		template <typename RealType>
		RealType GetClosestAngleFrom(const RealType InFrom, const RealType InTo)
		{
			return InFrom + FMath::FindDeltaAngleRadians(InFrom, InTo);
		}

		template <typename Func>
		bool EnsureRequiredFunc(const Func& InFunc, const FString& InFuncName)
		{
			return ensureMsgf(InFunc.IsSet(), TEXT("%s is required to be set."), *InFuncName);
		}
	}
}

UGizmoElementTranslateAxis::UGizmoElementTranslateAxis()
	: Axis()
	, ForwardDirection()
	, UpDirection()
	, SideDirection()
{
	TextNumberFormattingOptions.AlwaysSign = false;
	TextNumberFormattingOptions.UseGrouping = false;
	TextNumberFormattingOptions.RoundingMode = ERoundingMode::HalfFromZero;
	TextNumberFormattingOptions.MinimumFractionalDigits = 1;
	TextNumberFormattingOptions.MaximumFractionalDigits = 1;
}

void UGizmoElementTranslateAxis::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (!ensure(RenderAPI))
	{
		return;
	}

	FRenderTraversalState CurrentRenderState(RenderState);
	if (UpdateRenderState(RenderAPI, FVector::ZeroVector, CurrentRenderState))
	{
		ApplyUniformConstantScaleToTransform(CurrentRenderState.PixelToWorldScale, CurrentRenderState.LocalToWorldTransform);

		if (ArrowElement)
		{
			ArrowElement->Render(RenderAPI, RenderState);
		}
	}
}

void UGizmoElementTranslateAxis::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (bIsShowingDelta)
	{
		FRenderTraversalState CurrentRenderState(RenderState);
		if (UpdateRenderState(RenderAPI, FVector::ZeroVector, CurrentRenderState))
		{
			if (DeltaWidgetElement)
			{
				DeltaWidgetElement->DrawHUD(Canvas, RenderAPI, CurrentRenderState);
			}
		}
	}
}

const FGizmoElementTranslateAxisStyle& UGizmoElementTranslateAxis::GetStyle() const
{
	return Style;
}

void UGizmoElementTranslateAxis::SetStyle(const FGizmoElementTranslateAxisStyle& InStyle)
{
	Style = InStyle;
	ApplyStyle();
}

UE::Geometry::FFrame3d UGizmoElementTranslateAxis::MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const EAxisList::Type InAxisList) const
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;

	return MakeTransformedPlane(
		InTransform,
		InCoordinateSystem,
		ForwardDirection,
		UpDirection,
		SideDirection);
}

void UGizmoElementTranslateAxis::DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings)
{
	if (!bIsShowingDelta)
	{
		return;
	}
}

void UGizmoElementTranslateAxis::SetGizmoViewContext(UGizmoViewContext* InGizmoViewContext)
{
	GizmoViewContext = InGizmoViewContext;
}

void UGizmoElementTranslateAxis::SetWidgetHost(IToolkitHost* const InWidgetHost)
{
	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetWidgetHost(InWidgetHost);
	}
}

void UGizmoElementTranslateAxis::Setup(
	const uint32 InPartId,
	const EAxisList::Type InAxis,
	const FGizmoElementTranslateAxisStyle& InStyle)
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;
	using namespace UE::Editor::InteractiveToolsFramework::Private;
	using namespace GizmoElementTranslateAxisLocals;

	if (ArrowElement)
	{
		return;
	}

	Axis = InAxis;
	Style = InStyle;

	GetAxisBasis<FVector::FReal>(InAxis, ForwardDirection, UpDirection, SideDirection);

	// Primary Arrow Element
	if (!ArrowElement)
	{
		ArrowElement = NewObject<UGizmoElementArrow>();
		ArrowElement->SetPartIdentifier(InPartId, true, true);
		ArrowElement->SetHeadType(EGizmoElementArrowHeadType::Cone);
		ArrowElement->SetNumSides(Style.NumSegments);
		ArrowElement->SetViewDependentType(EGizmoElementViewDependentType::Axis);
		ArrowElement->SetViewDependentAxis(ForwardDirection);

		Add(ArrowElement);
	}

	// Apply common delta element settings
	auto ApplyDeltaElementSettings = [&, InPartId](UGizmoElementBase* InElement)
	{
		InElement->SetPartIdentifier(InPartId);

		InElement->SetHittableState(false);
		InElement->SetEnabledForDefaultState(false);
		InElement->SetEnabledForHoveringState(false);
		InElement->SetEnabledForInteractingState(true);
		InElement->SetEnabledForSelectedState(false);
		InElement->SetEnabledForSubduedState(false);
	};

	// Delta Widget Element
	if (!DeltaWidgetElement)
	{
		DeltaWidgetElement = NewObject<UGizmoElementValueWidget>();

		DeltaWidgetElement->SetPartIdentifier(InPartId);	
		DeltaWidgetElement->SetViewAlignType(EGizmoElementViewAlignType::PointScreen);
		DeltaWidgetElement->SetViewAlignAxis(FVector::UpVector);
		DeltaWidgetElement->SetViewAlignNormal(-FVector::ForwardVector);

		DeltaWidgetElement->SetLocation(FVector::ZeroVector);
		DeltaWidgetElement->SetClampToScreen(true);
		DeltaWidgetElement->SetClampToScreenFunction(
			[&](const FVector2D& InWidgetLocation, const FVector2D& InWidgetSize, const FIntRect& InViewRect) -> FVector2D
			{
				const FVector2d HalfViewSize = FVector2d(InViewRect.Size()) / 2.0;
				const FVector2d HalfWidgetSize = InWidgetSize / 2.0;

				// Assumes center alignment in canvas @see UGizmoElementWidget::UGizmoElementWidget()
				const FVector2d ExtraPadding(8.0f);
	
				const FVector2d PaddedViewMin = -HalfViewSize + HalfWidgetSize + ExtraPadding;
				const FVector2d PaddedViewMax = HalfViewSize - HalfWidgetSize - ExtraPadding;

				const bool bNeedsClamping = 
					(InWidgetLocation.X < PaddedViewMin.X) || (InWidgetLocation.X > PaddedViewMax.X)
					|| (InWidgetLocation.Y < PaddedViewMin.Y) || (InWidgetLocation.Y > PaddedViewMax.Y);

				const FVector2d Origin = StartState.TransformLocation2D - HalfViewSize;

				if (!bNeedsClamping)
				{
					return InWidgetLocation;
				}

				FBox2d PaddedViewBox(PaddedViewMin, PaddedViewMax);
				FVector2d::FReal ClippedStart, ClippedEnd;
				if (UE::Editor::GizmoMath::ClipLineToBox(
					Origin,
					InWidgetLocation,
					PaddedViewBox,
					ClippedStart,
					ClippedEnd))
				{
					return FVector2D(Origin + (InWidgetLocation - Origin) * ClippedEnd);
				}

				return InWidgetLocation;
			});

		ApplyDeltaElementSettings(DeltaWidgetElement);

		DeltaWidgetElement->ClearSubdueVertexColor();
		DeltaWidgetElement->ClearSelectVertexColor();

		DeltaWidgetElement->SetVisibleState(false);

		Add(DeltaWidgetElement);
	}

	SetPartIdentifier(InPartId, true);

	bIsValid = true;

	SetStyle(InStyle);

	UpdateElements();
}

void UGizmoElementTranslateAxis::UpdateElements()
{
	if (!bIsValid)
	{
		return;
	}

	// Delta Widget Element
	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetOffset2D(FVector2D(0.0f, 28.0f));
	}
}

void UGizmoElementTranslateAxis::BeginDelta(const FDeltaParameters& InParameters)
{
	bIsShowingDelta = true;
	StartState.Initialize(InParameters);
	CurrentState = StartState;

	if (DeltaWidgetElement && StartState.bIsSingleAxis)
	{
		DeltaWidgetElement->SetVisibleState(EnumHasAllFlags(Style.ShowFlags, EGizmoElementTranslateShowFlags::DeltaLabel));
		DeltaWidgetElement->SetClampToScreen(InParameters.bIsCursorInViewport); // Only set this on Begin, if the interaction starts within the bounds of the viewport
	}

	// Update State
	UpdateDelta(InParameters);
}

void UGizmoElementTranslateAxis::UpdateDelta(const FDeltaParameters& InParameters)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementTranslateAxisLocals;

	if (!DeltaWidgetElement)
	{
		return;
	}

	CurrentState.Update(InParameters);

	const FVector DeltaTranslation = CurrentState.Translation - StartState.Translation;
	const double Distance = FVector::DotProduct(DeltaTranslation, StartState.AxisDirection);

	// Delta Text Elements
	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetUnitText(Distance, EUnitType::Distance, TextNumberFormattingOptions);

		const FVector TextCenter = ForwardDirection * (Style.AxisOffsetFromCenter + ArrowElement->GetBodyLength()) + SideDirection;
		DeltaWidgetElement->SetLocation(TextCenter);
	}
}

void UGizmoElementTranslateAxis::EndDelta()
{
	if (DeltaWidgetElement)
	{
		DeltaWidgetElement->SetVisibleState(false);
	}
	
	bIsShowingDelta = false;
	StartState = CurrentState;
}

void UGizmoElementTranslateAxis::ApplyStyle()
{
	if (!bIsValid)
	{
		return;
	}

	SetVertexColor(Style.Colors.Default.Get(GetVertexColor()));

	const float SizeCoefficient = Style.SizeCoefficient.Get(1.0f);

	const float ArrowHeadRadius = Style.ArrowRadius * Style.ArrowSizeMultiplier * Style.HandleSizeMultiplier;
	const float MinLineRadius = Style.MinLineThickness * 0.5f;
	const float MaxLineRadius = ArrowHeadRadius; // Prevents the line radius becoming larger than and overshooting the axis "head"
	const float LineRadius = Style.LineThickness * 0.5f;

	// Primary Arrow Element
	if (ArrowElement)
	{
		ArrowElement->SetBase(ForwardDirection * Style.AxisOffsetFromCenter * SizeCoefficient);
		ArrowElement->SetDirection(ForwardDirection);
		ArrowElement->SetSideDirection(SideDirection);
		ArrowElement->SetBodyLength(Style.AxisLength * Style.AxisLengthMultiplier * SizeCoefficient);
		ArrowElement->SetBodyRadius(FMath::Clamp((LineRadius * Style.LineThicknessMultiplier), MinLineRadius, MaxLineRadius) * SizeCoefficient);
		ArrowElement->SetHeadLength(Style.ArrowHeight * Style.ArrowSizeMultiplier * Style.HandleSizeMultiplier * SizeCoefficient);
		ArrowElement->SetHeadRadius(ArrowHeadRadius * SizeCoefficient);

		UE::Editor::InteractiveToolsFramework::ApplyMaterialsToElement(ArrowElement, Style.Materials);
		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(ArrowElement, Style.Colors);

		ArrowElement->SetPixelHitDistanceThreshold(Style.PixelHitDistanceThreshold);
	}

	// Delta Widget Element
	if (DeltaWidgetElement)
	{
		const FLinearColor BackgroundColor = Style.DeltaTextBackgroundColor;

		DeltaWidgetElement->SetMaterial(Style.VertexColorMaterial);
		DeltaWidgetElement->SetHoverMaterial(Style.VertexColorMaterial);
		DeltaWidgetElement->SetInteractMaterial(Style.VertexColorMaterial);

		DeltaWidgetElement->SetVertexColor(BackgroundColor);
		DeltaWidgetElement->SetHoverVertexColor(BackgroundColor);
		DeltaWidgetElement->SetInteractVertexColor(BackgroundColor);

		DeltaWidgetElement->ClearSubdueVertexColor();
		DeltaWidgetElement->ClearSelectVertexColor();
	}
}

void UGizmoElementTranslateAxis::FState::Initialize(const FDeltaParameters& InParameters)
{
	Transform = InParameters.Transform;
	TransformLocation2D = InParameters.TransformLocation2D;
	Translation = InParameters.Translation;
	AxisDirection = InParameters.AxisDirection;
	PlaneNormal = InParameters.PlaneNormal;
	bIsSingleAxis = UE::Editor::InteractiveToolsFramework::Internal::IsAxisSingular(InParameters.AxisList);
}

void UGizmoElementTranslateAxis::FState::Update(const FDeltaParameters& InParameters)
{
	Transform = InParameters.Transform;
	TransformLocation2D = InParameters.TransformLocation2D;
	Translation = InParameters.Translation;
}
