// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseGizmos/GizmoElementGroup.h"
#include "EditorGizmoElementInterfaces.h"
#include "EditorGizmos/GizmoElementTranslate.h"
#include "FrameTypes.h"

#include "GizmoElementTranslateAxis.generated.h"

class IToolkitHost;
class UGizmoElementValueWidget;
class UGizmoElementArrow;
class UGizmoElementDashedLine;
class UGizmoElementRectangle;
class UGizmoElementRoundedRectangle;
class UGizmoElementBox;
class UGizmoElementCylinder;
class UGizmoElementLineStrip;
class UGizmoElementCircle;
class UGizmoElementSphere;
class UGizmoElementArc;
class UGizmoElementText;
class UGizmoElementTorus;
class UGizmoViewContext;

/**
 * Combines the visual elements of a single translation axis: an arrow handle,
 * a dashed delta line, and a value label shown during interaction.
 * Also implements IPlaneProvider to define the interaction plane for drag projection.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementTranslateAxis
	: public UGizmoElementGroupBase
	, public UE::Editor::InteractiveToolsFramework::IPlaneProvider
{
	GENERATED_BODY()

public:
	/** Parameters shared between BeginDelta and UpdateDelta calls, describing the current interaction state. */
	struct FDeltaParameters
	{
		/** The current world transform of the gizmo. */
		FTransform Transform = FTransform::Identity;

		/** The active coordinate system (World, Local, etc.). */
		EToolContextCoordinateSystem CoordinateSystem = EToolContextCoordinateSystem::World;

		/** Screen-space location of the gizmo origin, used for label placement. */
		FVector2D TransformLocation2D = FVector2D::ZeroVector;

		/** Direction of the translation axis in the current coordinate system. */
		FVector AxisDirection = FVector::ZeroVector;

		/** Normal of the interaction plane. */
		FVector PlaneNormal = FVector::UpVector;

		/** Accumulated translation delta from the start of the interaction. */
		FVector Translation = FVector::ZeroVector;

		/** Whether the cursor is currently within the viewport (affects delta label visibility). */
		bool bIsCursorInViewport = true;

		/** Whether this is an indirect interaction (e.g. input field rather than drag). */
		bool bIsIndirectInteraction = false;

		/** The axis or axes being translated. */
		EAxisList::Type AxisList = EAxisList::None;
	};

public:
	UGizmoElementTranslateAxis();

	//~ Begin UGizmoElementBase
	virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	//~ End UGizmoElementBase

	/** Returns the current visual style for this translate axis element. */
	const FGizmoElementTranslateAxisStyle& GetStyle() const;

	/** Sets and applies the given Style. */
	void SetStyle(const FGizmoElementTranslateAxisStyle& InStyle);

	//~ Begin IPlaneProvider
	/** Creates the interaction plane for this translate axis, transformed into the provided coordinate space. */
	virtual UE::Geometry::FFrame3d MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const EAxisList::Type InAxisList) const override;
	//~ End IPlaneProvider

	/** Renders various debug visualizations, if enabled. */
	void DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings);

	/** Sets the Gizmo view context, needed for screen space interactions */
	void SetGizmoViewContext(UGizmoViewContext* InGizmoViewContext);

	/** Sets the host to add/remove a widget to.
	* Currently, this is a Toolkit host, but it could be abstracted to an ISlateWidgetHost (see IEditorViewportClientProxy). */
	void SetWidgetHost(IToolkitHost* const InWidgetHost);

	/**
	 * Initialize the axis element with a part identifier, axis, and visual style.
	 * Must be called before the element can render or receive interactions.
	 * @param InPartId Unique identifier for hit-testing this axis.
	 * @param InAxis The axis this element represents (e.g. EAxisList::X).
	 * @param InStyle Visual style parameters for the axis.
	 */
	void Setup(
		const uint32 InPartId,
		const EAxisList::Type InAxis,
		const FGizmoElementTranslateAxisStyle& InStyle);

	/** Rebuilds child elements to reflect the current style. Call after modifying style properties. */
	void UpdateElements();

	/** Begin a delta interaction, recording the start state. */
	void BeginDelta(const FDeltaParameters& InParameters);

	/** Update the delta visualization with the current interaction state. */
	void UpdateDelta(const FDeltaParameters& InParameters);

	/** End the current delta interaction and hide delta visuals. */
	void EndDelta();

private:
	/** Style contains various uniform and varying properties that affect the appearance of this gizmo element. */
	UPROPERTY(Getter, Setter)
	FGizmoElementTranslateAxisStyle Style;

	/** Translate Axis. */
	UPROPERTY()
	TObjectPtr<UGizmoElementArrow> ArrowElement;

	/** Widget element that displays the delta value label during interaction. */
	UPROPERTY()
	TObjectPtr<UGizmoElementValueWidget> DeltaWidgetElement;

private:
	/** Propagates the current Style properties to all child visual elements. */
	void ApplyStyle();

private:
	/** Flag indicating element validity - generally true after Setup is called. */
	std::atomic_bool bIsValid = false;
	
	/** Gizmo view context, needed for screen space interactions */
	UPROPERTY(Setter)
	TObjectPtr<UGizmoViewContext> GizmoViewContext;

	/** The axis this element represents (e.g. EAxisList::X). */
	EAxisList::Type Axis;

	/** Forward direction along the axis in local space. */
	FVector ForwardDirection;

	/** Up direction perpendicular to the axis, used for interaction plane construction. */
	FVector UpDirection;

	/** Side direction perpendicular to the axis, used for interaction plane construction. */
	FVector SideDirection;

	/** Formatting options for the delta value label text. */
	FNumberFormattingOptions TextNumberFormattingOptions;

	/** Snapshot of interaction state at a point in time, used to track start and current states during a delta. */
	struct FState
	{
		/** The world transform of the gizmo at this point in the interaction. */
		FTransform Transform;

		/** The gizmo origin projected into 2D screen-space. */
		FVector2D TransformLocation2D = FVector2D::ZeroVector; // The transforms' location in 2D screen-space

		/** Accumulated translation offset from the interaction origin. */
		FVector Translation = FVector::ZeroVector;

		/** Direction of the translation axis in the current coordinate system. */
		FVector AxisDirection = FVector::ZeroVector;

		/** Normal of the interaction plane. */
		FVector PlaneNormal = FVector::UpVector;

		/** Whether the interaction is constrained to a single axis (as opposed to a plane). */
		bool bIsSingleAxis = false;

		/** Populates all fields from the given delta parameters at the start of an interaction. */
		void Initialize(const FDeltaParameters& InParameters);

		/** Updates fields from the given delta parameters during an ongoing interaction. */
		void Update(const FDeltaParameters& InParameters);
	};

	/** Interaction state captured at BeginDelta, used as the reference for computing deltas. */
	FState StartState;

	/** Interaction state updated each frame during an active delta interaction. */
	FState CurrentState;

	/** Whether the delta visualization (dashed line and value label) is currently visible. */
	bool bIsShowingDelta = false;
};
