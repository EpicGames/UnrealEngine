// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementTranslateGroup.h"

#include "BaseGizmos/GizmoElementBox.h"
#include "BaseGizmos/GizmoElementCircle.h"
#include "BaseGizmos/GizmoElementCylinder.h"
#include "BaseGizmos/GizmoElementRectangle.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.h"
#include "EditorGizmos/EditorGizmoElementSharedInternal.inl"
#include "GizmoElementRotateAxis.h"
#include "GizmoElementTranslateAxis.h"

DEFINE_LOG_CATEGORY_STATIC(LogGizmoElementTranslateAxisSet, Log, All);

namespace UE::Editor::InteractiveToolsFramework::Private
{
	namespace GizmoElementTranslateAxisSetLocals
	{
		UGizmoElementTranslateAxis::FDeltaParameters MakeAxisDeltaParameters(const UGizmoElementTranslateGroup::FDeltaParameters& InGroupParameters, const EAxis::Type InAxis)
		{			
			UGizmoElementTranslateAxis::FDeltaParameters AxisDeltaParameters;
			AxisDeltaParameters.Transform = InGroupParameters.Transform;
			AxisDeltaParameters.TransformLocation2D = InGroupParameters.TransformLocation2D;
			AxisDeltaParameters.Translation = InGroupParameters.Transform.GetLocation();
			AxisDeltaParameters.CoordinateSystem = InGroupParameters.CoordinateSystem;
			AxisDeltaParameters.PlaneNormal = InGroupParameters.PlaneNormal;
			AxisDeltaParameters.bIsIndirectInteraction = InGroupParameters.bIsIndirectInteraction;
			AxisDeltaParameters.AxisList = InGroupParameters.AxisList;

			AxisDeltaParameters.AxisDirection =
				InGroupParameters.CoordinateSystem == EToolContextCoordinateSystem::Local
				? InGroupParameters.Transform.GetUnitAxis(InAxis)
				: Internal::GetAxisVector(InAxis);

			return AxisDeltaParameters;
		}

		static bool ShouldShowElement(const bool bInIsAlternateElement,	const TFunction<FToolContextSnappingConfiguration()>& InGetCurrentSnappingSettingsFunction)
		{
			auto GetCVarValue = []() -> int32
			{
				constexpr const TCHAR* CVarName = TEXT("Editor.Gizmo.AlternateSnappingElements");
				if (const IConsoleVariable* DebugDrawCVar = IConsoleManager::Get().FindConsoleVariable(CVarName, false))
				{
					return FMath::Clamp(DebugDrawCVar->GetInt(), 0, 2);
				}

				constexpr int32 DefaultValue = 0;
				return DefaultValue;
			};

			const int32 AlternateSnappingElementsOption = GetCVarValue();

			// Snapping Elements disabled, only show non-alternates
			if (AlternateSnappingElementsOption == 0)
			{
				return !bInIsAlternateElement;
			}

			const FToolContextSnappingConfiguration& CurrentSnappingSettings = InGetCurrentSnappingSettingsFunction();
			const bool bIsSnappingEnabled =	CurrentSnappingSettings.bEnablePositionGridSnapping || CurrentSnappingSettings.ObjectTransform.bEnable;

			// Only show alternate when snapping on
			if (AlternateSnappingElementsOption == 1)
			{
				return bIsSnappingEnabled == bInIsAlternateElement;
			}

			// Inverse - only show alternate when snapping off, regular when on
			if (AlternateSnappingElementsOption == 2)
			{
				return bIsSnappingEnabled != bInIsAlternateElement;
			}

			// We shouldn't even be here
			return false;
		}
	}
}

void FGizmoElementTranslateAxisStyleOverride::ApplyTo(FGizmoElementTranslateAxisStyle& InOutStyle) const
{
	using namespace UE::Editor::InteractiveToolsFramework;

	if (Colors.IsSet())
	{
		ApplyColorOverrides(InOutStyle.Colors, Colors.GetValue());
	}

	if (Materials.IsSet())
	{
		ApplyMaterialOverrides(InOutStyle.Materials, Materials.GetValue());
	}

	if (VertexColorMaterial.IsSet())
	{
		InOutStyle.VertexColorMaterial = VertexColorMaterial.GetValue();
	}
}

void FGizmoElementTranslatePlanarStyleOverride::ApplyTo(FGizmoElementTranslatePlanarStyle& InOutStyle) const
{
	using namespace UE::Editor::InteractiveToolsFramework;

	if (Colors.IsSet())
	{
		ApplyColorOverrides(InOutStyle.Colors, Colors.GetValue());
	}

	if (Materials.IsSet())
	{
		ApplyMaterialOverrides(InOutStyle.Materials, Materials.GetValue());
	}

	if (VertexColorMaterial.IsSet())
	{
		InOutStyle.VertexColorMaterial = VertexColorMaterial.GetValue();
	}
}

void UGizmoElementTranslateGroup::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (!ensure(RenderAPI))
	{
		return;
	}

	// Non-Delta Elements
	FRenderTraversalState CurrentRenderState(RenderState);
	if (UpdateRenderState(RenderAPI, FVector::ZeroVector, CurrentRenderState))
	{
		ApplyUniformConstantScaleToTransform(CurrentRenderState.PixelToWorldScale, CurrentRenderState.LocalToWorldTransform);

		ForEachAxisElement([&](UGizmoElementTranslateAxis* InAxisElement)
		{
			if (InAxisElement)
			{
				InAxisElement->Render(RenderAPI, CurrentRenderState);
			}
		});

		if (PlanarXYElement)
		{
			PlanarXYElement->Render(RenderAPI, CurrentRenderState);
		}

		if (PlanarYZElement)
		{
			PlanarYZElement->Render(RenderAPI, CurrentRenderState);
		}

		if (PlanarXZElement)
		{
			PlanarXZElement->Render(RenderAPI, CurrentRenderState);
		}

		if (UniformElement)
		{
			UniformElement->Render(RenderAPI, CurrentRenderState);
		}

		if (UniformElementAlternate)
		{
			UniformElementAlternate->Render(RenderAPI, CurrentRenderState);
		}

		if (OriginElement)
		{
			OriginElement->Render(RenderAPI, CurrentRenderState);
		}
	}

	if (IsInteracting())
	{
		// Delta Elements are based on the Start Transform, not current
		FRenderTraversalState DeltaRenderState(RenderState);
		if (UpdateDeltaRenderState(RenderAPI, FVector::ZeroVector, DeltaRenderState))
		{
			if (FMath::IsNearlyZero(DeltaRenderState.PixelToWorldScale))
			{
				return;
			}

			FRenderTraversalState UnscaledDeltaRenderState = DeltaRenderState;
			UnscaledDeltaRenderState.LocalToWorldTransform.SetScale3D(UnscaledDeltaRenderState.LocalToWorldTransform.GetScale3D() / DeltaRenderState.PixelToWorldScale);

			ApplyUniformConstantScaleToTransform(DeltaRenderState.PixelToWorldScale, DeltaRenderState.LocalToWorldTransform, true);

			auto RenderDeltaElement = [&](UGizmoElementBase* InElement)
			{
				if (InElement)
				{
					InElement->Render(RenderAPI, DeltaRenderState);
				}
			};

			RenderDeltaElement(DeltaOriginElement);

			const float DeltaLineLength = static_cast<float>(CurrentState.DistanceFromStart) / static_cast<float>(DeltaRenderState.PixelToWorldScale);
			if (!FMath::IsNearlyZero(DeltaLineLength))
			{
				DeltaLineElement->SetHeight(DeltaLineLength);
				RenderDeltaElement(DeltaLineElement);
			}
		}
	}
}

bool UGizmoElementTranslateGroup::IsInteracting() const
{
	for (const UGizmoElementBase* Element : Elements)
	{
		if (Element)
		{
			if (Element->GetElementInteractionState() == EGizmoElementInteractionState::Interacting)
			{
				return true;
			}
		}
	}

	return false;
}

void UGizmoElementTranslateGroup::ApplyGetCurrentSnappingSettingsFunction()
{
	// Only applies if set
	if (!GetCurrentSnappingSettingsFunction.IsSet())
	{
		return;
	}
	
	using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementTranslateAxisSetLocals;

	if (UniformElement)
	{
		constexpr bool bIsAlternateElement = false;

		UniformElement->SetIsVisibleFunction([GetCurrentSnappingSettingsFunction = GetCurrentSnappingSettingsFunction](
			const FSceneView*, EViewInteractionState, EGizmoElementInteractionState, 
			const FTransform&, const FVector&) -> bool
		{
			return ShouldShowElement(bIsAlternateElement, GetCurrentSnappingSettingsFunction);
		});

		UniformElement->SetIsHittableFunction([GetCurrentSnappingSettingsFunction = GetCurrentSnappingSettingsFunction](const UGizmoViewContext*, const FTransform&, const FVector&) -> bool
		{
			return ShouldShowElement(bIsAlternateElement, GetCurrentSnappingSettingsFunction);
		});
	}

	if (UniformElementAlternate)
	{
		constexpr bool bIsAlternateElement = true;

		UniformElementAlternate->SetIsVisibleFunction([GetCurrentSnappingSettingsFunction = GetCurrentSnappingSettingsFunction](
			const FSceneView*, EViewInteractionState, EGizmoElementInteractionState, 
			const FTransform&, const FVector&) -> bool
		{
			return ShouldShowElement(bIsAlternateElement, GetCurrentSnappingSettingsFunction);
		});

		UniformElementAlternate->SetIsHittableFunction([GetCurrentSnappingSettingsFunction = GetCurrentSnappingSettingsFunction](const UGizmoViewContext*, const FTransform&, const FVector&) -> bool
		{
			return ShouldShowElement(bIsAlternateElement, GetCurrentSnappingSettingsFunction);
		});
	}
}

void UGizmoElementTranslateGroup::FState::Initialize(const FDeltaParameters& InParameters)
{
	Transform = InParameters.Transform;
	TransformLocation2D = InParameters.TransformLocation2D;
	Translation = InParameters.Transform.GetLocation();
	PlaneNormal = InParameters.PlaneNormal;
	AxisList = InParameters.AxisList;
	bIsSingleAxis = UE::Editor::InteractiveToolsFramework::Internal::IsAxisSingular(AxisList);
}

void UGizmoElementTranslateGroup::FState::Update(const FDeltaParameters& InParameters)
{
	Transform = InParameters.Transform;
	TransformLocation2D = InParameters.TransformLocation2D;
	Translation = InParameters.Transform.GetLocation();
}

void UGizmoElementTranslateGroup::DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings, const uint32 InPartId)
{
	if (InPartId == 0)
	{
		ForEachAxisElement([&](UGizmoElementTranslateAxis* InAxisElement)
		{
			if (InAxisElement)
			{
				InAxisElement->DrawDebug(RenderAPI, InSettings);
			}
		});
	}
	else
	{
		ForEachSubElementRecursive([&](UGizmoElementBase* InElement)
		{
			if (UGizmoElementTranslateAxis* TranslateAxisElement = Cast<UGizmoElementTranslateAxis>(InElement))
			{
				TranslateAxisElement->DrawDebug(RenderAPI, InSettings);
			}
		},
		InPartId);
	}
}

void UGizmoElementTranslateGroup::SetWidgetHost(IToolkitHost* const InWidgetHost)
{
	if (!bIsValid)
	{
		return;
	}

	ForEachAxisElement([&](UGizmoElementTranslateAxis* InAxisElement)
	{
		InAxisElement->SetWidgetHost(InWidgetHost);
	});
}

void UGizmoElementTranslateGroup::SetGetCurrentSnappingSettingsFunction(const TFunction<FToolContextSnappingConfiguration()>& InFunction)
{
	GetCurrentSnappingSettingsFunction = InFunction;

	if (!bIsValid)
	{
		return;
	}

	ApplyGetCurrentSnappingSettingsFunction();
}

void UGizmoElementTranslateGroup::SetAxisEnabled(const EAxisList::Type InAxisListToEnable)
{
	auto SetAxisEnabled = [&](UGizmoElementTranslateAxis* InAxisElement, const EAxisList::Type InSingleAxis) -> bool
	{
		if (!InAxisElement)
		{
			return false;
		}

		const bool bEnableAxis = static_cast<uint8>(InAxisListToEnable) & static_cast<uint8>(InSingleAxis);
		InAxisElement->SetEnabled(bEnableAxis);

		return bEnableAxis;
	};

	const bool bEnableX = SetAxisEnabled(AxisXElement, EAxisList::X);
	const bool bEnableY = SetAxisEnabled(AxisYElement, EAxisList::Y);
	const bool bEnableZ = SetAxisEnabled(AxisZElement, EAxisList::Z);
	const bool bEnableAny = bEnableX || bEnableY || bEnableZ;

	if (OriginElement)
	{
		OriginElement->SetEnabled(bEnableAny);
	}
}

void UGizmoElementTranslateGroup::SetPlanarEnabled(const EAxisList::Type InAxisListToEnable)
{
	auto SetPlanarEnabled = [&](UGizmoElementBox* InPlanarElement, const EAxisList::Type InPlaneAxisPair)
	{
		if (!ensure(InPlanarElement))
		{
			return;
		}

		const bool bEnablePlanar = static_cast<uint8>(InAxisListToEnable) & static_cast<uint8>(InPlaneAxisPair);
		InPlanarElement->SetEnabled(bEnablePlanar);
	};

	SetPlanarEnabled(PlanarXYElement, EAxisList::XY);
	SetPlanarEnabled(PlanarYZElement, EAxisList::YZ);
	SetPlanarEnabled(PlanarXZElement, EAxisList::XZ);
}

void UGizmoElementTranslateGroup::SetUniformEnabled(const bool bInEnable)
{
	if (UniformElement)
	{
		UniformElement->SetEnabled(bInEnable);
	}

	if (UniformElementAlternate)
	{
		UniformElementAlternate->SetEnabled(bInEnable);
	}
}

const FGizmoElementTranslateStyle& UGizmoElementTranslateGroup::GetStyle() const
{
	return Style;
}

void UGizmoElementTranslateGroup::SetStyle(
	const FGizmoElementTranslateStyle& InStyle,
	const FGizmoElementTranslateAxisStyleOverride& InAxisStyleX,
	const FGizmoElementTranslateAxisStyleOverride& InAxisStyleY,
	const FGizmoElementTranslateAxisStyleOverride& InAxisStyleZ,
	const FGizmoElementTranslatePlanarStyleOverride& InPlanarStyleXY,
	const FGizmoElementTranslatePlanarStyleOverride& InPlanarStyleYZ,
	const FGizmoElementTranslatePlanarStyleOverride& InPlanarStyleXZ)
{
	Style = InStyle;
	StyleX = InAxisStyleX;
	StyleY = InAxisStyleY;
	StyleZ = InAxisStyleZ;
	PlanarXYStyle = InPlanarStyleXY;
	PlanarYZStyle = InPlanarStyleYZ;
	PlanarXZStyle = InPlanarStyleXZ;

	ApplyStyle();
}

UE::Geometry::FFrame3d UGizmoElementTranslateGroup::MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const EAxisList::Type InAxisList) const
{
	using namespace UE::Editor::InteractiveToolsFramework::Internal;
	
	return MakeTransformedPlaneForAxisList(
		InTransform,
		InViewContext,
		InCoordinateSystem,
		InAxisList,
		[this](const EAxisList::Type AxisList) -> const UGizmoElementBox*
		{
			return GetPlanarElement(AxisList);
		},
		[this](const EAxisList::Type AxisList) -> const IPlaneProvider*
		{
			return GetAxisElement(EAxis::FromAxisList(AxisList));
		});
}

void UGizmoElementTranslateGroup::Setup(
	const uint32 InPartId,
	const EAxisList::Type InAxisList,
	const FGizmoElementTranslateStyle& InStyle,
	const FAxisParameters& InAxisX,
	const FAxisParameters& InAxisY,
	const FAxisParameters& InAxisZ,
	const FPlanarParameters& InPlanarXY,
	const FPlanarParameters& InPlanarYZ,
	const FPlanarParameters& InPlanarXZ,
	const uint32 InUniformPartId)
{
	using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementTranslateAxisSetLocals;

	if (AxisXElement && AxisYElement && AxisZElement)
	{
		return;
	}

	// Axis Elements
	{
		auto MakeAxisElement = [&](const FAxisParameters& InParameters)
		{
			FGizmoElementTranslateAxisStyle AxisStyle = InStyle.AxisStyle;
			InParameters.StyleOverride.ApplyTo(AxisStyle);

			UGizmoElementTranslateAxis* AxisElement = NewObject<UGizmoElementTranslateAxis>();
			AxisElement->Setup(
				InParameters.PartId,
				InParameters.Axis,
				AxisStyle);

			return AxisElement;
		};

		AxisXElement = MakeAxisElement(InAxisX);
		AxisYElement = MakeAxisElement(InAxisY);
		AxisZElement = MakeAxisElement(InAxisZ);

		Add(AxisXElement);
		Add(AxisYElement);
		Add(AxisZElement);
	}

	// Planar Elements
	{
		auto MakePlanarElement = [&](const FPlanarParameters& InParameters)
		{
			using namespace UE::Editor::InteractiveToolsFramework::Internal;

			FGizmoElementTranslatePlanarStyle PlanarStyle = InStyle.PlanarStyle;
			InParameters.StyleOverride.ApplyTo(PlanarStyle);

			const EAxisList::Type PlaneNormalAxis = GetPlaneNormalAxis(InParameters.Axis);

			FVector Normal;
			FVector Up;
			FVector Side;
			GetAxisBasis<FVector::FReal>(PlaneNormalAxis, Normal, Up, Side);

			UGizmoElementBox* BoxElement = NewObject<UGizmoElementBox>();
			BoxElement->SetPartIdentifier(InParameters.PartId, true);
			BoxElement->SetUpDirection(Up);
			BoxElement->SetSideDirection(Side);
			BoxElement->SetViewDependentType(EGizmoElementViewDependentType::Plane);
			BoxElement->SetViewDependentAxis(Normal);

			return BoxElement;
		};

		PlanarXYElement = MakePlanarElement(InPlanarXY);
		PlanarYZElement = MakePlanarElement(InPlanarYZ);
		PlanarXZElement = MakePlanarElement(InPlanarXZ);

		Add(PlanarXYElement);
		Add(PlanarYZElement);
		Add(PlanarXZElement);
	}

	// Uniform screen-space element
	if (!UniformElement)
	{
		UniformElement = NewObject<UGizmoElementRectangle>();
		UniformElement->SetPartIdentifier(InUniformPartId, true);
		UniformElement->SetHitPriority(10);
		UniformElement->SetUpDirection(FVector::UpVector);
		UniformElement->SetSideDirection(FVector::RightVector);
		UniformElement->SetCenter(FVector::ZeroVector);
		UniformElement->SetViewAlignType(EGizmoElementViewAlignType::PointScreen);
		UniformElement->SetViewAlignAxis(FVector::UpVector);
		UniformElement->SetViewAlignNormal(-FVector::ForwardVector);
		UniformElement->SetHitMesh(true);
		UniformElement->SetHitLine(false);
		UniformElement->SetDrawMesh(false);
		UniformElement->SetDrawLine(true);
		UniformElement->SetHoverLineThicknessMultiplier(1.5f);
		UniformElement->SetInteractLineThicknessMultiplier(1.5f);
		UniformElement->SetSelectLineThicknessMultiplier(1.0f);

		Add(UniformElement);
	}

	// Uniform (Alternative) screen-space element. Swaps between the normal one and this depending on snap state.
	if (!UniformElementAlternate)
	{
		UniformElementAlternate = NewObject<UGizmoElementCircle>();
		UniformElementAlternate->SetPartIdentifier(InUniformPartId, true);
		UniformElementAlternate->SetHitPriority(10);
		UniformElementAlternate->SetCenter(FVector::ZeroVector);
		UniformElementAlternate->SetAxisTangent(FVector::RightVector);
		UniformElementAlternate->SetAxisBitangent(FVector::UpVector);
		UniformElementAlternate->SetViewAlignType(EGizmoElementViewAlignType::PointScreen);
		UniformElementAlternate->SetViewAlignAxis(FVector::UpVector);
		UniformElementAlternate->SetViewAlignNormal(-FVector::ForwardVector);
		UniformElementAlternate->SetHitMesh(true);
		UniformElementAlternate->SetHitLine(false);
		UniformElementAlternate->SetDrawMesh(false);
		UniformElementAlternate->SetDrawLine(true);
		UniformElementAlternate->SetHoverLineThicknessMultiplier(1.5f);
		UniformElementAlternate->SetInteractLineThicknessMultiplier(1.5f);
		UniformElementAlternate->SetSelectLineThicknessMultiplier(1.0f);

		Add(UniformElementAlternate);
	}

	// Origin Sphere Element
	if (!OriginElement)
	{
		OriginElement = NewObject<UGizmoElementCircle>();
		OriginElement->SetCenter(FVector::ZeroVector);
		OriginElement->SetAxisTangent(FVector::RightVector);
		OriginElement->SetAxisBitangent(FVector::UpVector);
		OriginElement->SetViewAlignType(EGizmoElementViewAlignType::PointScreen);
		OriginElement->SetViewAlignAxis(FVector::UpVector);
		OriginElement->SetViewAlignNormal(-FVector::ForwardVector);

		OriginElement->SetDrawMesh(true);
		OriginElement->SetHitMesh(false);
		OriginElement->SetVertexColor(FLinearColor::Transparent);

		OriginElement->SetDrawLine(false);
		OriginElement->SetHitLine(false);

		Add(OriginElement);
	}

	// Apply common delta element settings
	auto ApplyDeltaElementSettings = [&](UGizmoElementBase* InElement)
	{
		InElement->SetHittableState(false);
		InElement->SetEnabledForDefaultState(false);
		InElement->SetEnabledForHoveringState(false);
		InElement->SetEnabledForInteractingState(true);
		InElement->SetEnabledForSelectedState(false);
		InElement->SetEnabledForSubduedState(false);
	};

	// Delta Origin Circle Element - this appears as the original "left behind" origin when moving the gizmo
	if (!DeltaOriginElement)
	{
		DeltaOriginElement = NewObject<UGizmoElementCircle>();
		DeltaOriginElement->SetCenter(FVector::ZeroVector);
		DeltaOriginElement->SetAxisTangent(FVector::RightVector);
		DeltaOriginElement->SetAxisBitangent(FVector::UpVector);
		DeltaOriginElement->SetViewAlignType(EGizmoElementViewAlignType::PointScreen);
		DeltaOriginElement->SetViewAlignNormal(-FVector::ForwardVector);

		DeltaOriginElement->SetDrawMesh(true);
		DeltaOriginElement->SetHitMesh(false);
		DeltaOriginElement->SetVertexColor(FLinearColor::Transparent);

		DeltaOriginElement->SetDrawLine(true);
		DeltaOriginElement->SetHitLine(false);

		constexpr float OtherStateLineThickness = 1.0f;
		DeltaOriginElement->SetHoverLineThicknessMultiplier(OtherStateLineThickness);
		DeltaOriginElement->SetInteractLineThicknessMultiplier(OtherStateLineThickness);

		ApplyDeltaElementSettings(DeltaOriginElement);

		DeltaOriginElement->SetVisibleState(false);

		Add(DeltaOriginElement);
	}

	if (!DeltaLineElement)
	{
		DeltaLineElement = NewObject<UGizmoElementCylinder>();
		DeltaLineElement->SetBase(FVector::ZeroVector);

		ApplyDeltaElementSettings(DeltaLineElement);

		DeltaLineElement->SetVisibleState(false);

		Add(DeltaLineElement);
	}

	SetPartIdentifier(InPartId, true);

	ApplyGetCurrentSnappingSettingsFunction();

	bIsValid = true;

	SetStyle(
		InStyle,
		InAxisX.StyleOverride,
		InAxisY.StyleOverride,
		InAxisZ.StyleOverride,
		InPlanarXY.StyleOverride,
		InPlanarYZ.StyleOverride,
		InPlanarXZ.StyleOverride);

	UpdateElements();
}

void UGizmoElementTranslateGroup::UpdateElements()
{
	if (!bIsValid)
	{
		return;
	}

	auto UpdateAxisElement = [&](UGizmoElementTranslateAxis* InAxisElement)
	{
		if (!ensure(InAxisElement))
		{
			return;
		}

		InAxisElement->UpdateElements();
	};

	UpdateAxisElement(AxisXElement);
	UpdateAxisElement(AxisYElement);
	UpdateAxisElement(AxisZElement);

	constexpr float OriginElementViewDepthOffset = 10.0f; // Shared between origin elements

	// Origin Circle Element
	if (OriginElement)
	{
		OriginElement->SetViewDepthOffset(-(static_cast<float>(OriginElement->GetRadius()) * 2.0f + OriginElementViewDepthOffset)); // Ensure the origin is always on top
	}

	// Origin Pivot Element
	if (DeltaOriginElement)
	{
		DeltaOriginElement->SetViewDepthOffset(-OriginElementViewDepthOffset); // Ensure the delta origin is always on top of others, but below the main origin element
	}

	// Delta Line Element
	if (DeltaLineElement)
	{
	}
}

UGizmoElementTranslateAxis* UGizmoElementTranslateGroup::GetAxisElement(const EAxis::Type InAxis) const
{
	switch (InAxis)
	{
	case EAxis::X:
	if (ensure(AxisXElement))
	{
		return AxisXElement;
	}
	break;

	case EAxis::Y:
	if (ensure(AxisYElement))
	{
		return AxisYElement;
	}
	break;

	case EAxis::Z:
	if (ensure(AxisZElement))
	{
		return AxisZElement;
	}
	break;

	default:
		return nullptr;
	}

	return nullptr;
}

UGizmoElementBox* UGizmoElementTranslateGroup::GetPlanarElement(const EAxisList::Type InAxis) const
{
	switch (InAxis)
	{
	case EAxisList::XY:
	if (ensure(PlanarXYElement))
	{
		return PlanarXYElement;
	}
	break;

	case EAxisList::YZ:
	if (ensure(PlanarYZElement))
	{
		return PlanarYZElement;
	}
	break;

	case EAxisList::XZ:
	if (ensure(PlanarXZElement))
	{
		return PlanarXZElement;
	}
	break;

	default:
		return nullptr;
	}

	return nullptr;
}

void UGizmoElementTranslateGroup::ApplyStyle()
{
	if (!bIsValid)
	{
		return;
	}

	if (Style.Colors.Default.IsSet())
	{
		SetVertexColor(Style.Colors.Default.GetValue());
	}

	auto UpdateAxisElement = [&](UGizmoElementTranslateAxis* InAxisElement, const FGizmoElementTranslateAxisStyleOverride& InAxisStyleOverride)
	{
		if (!ensure(InAxisElement))
		{
			return;
		}

		FGizmoElementTranslateAxisStyle AxisStyle = Style.AxisStyle;
		InAxisStyleOverride.ApplyTo(AxisStyle);

		InAxisElement->SetStyle(AxisStyle);
	};

	UpdateAxisElement(AxisXElement, StyleX);
	UpdateAxisElement(AxisYElement, StyleY);
	UpdateAxisElement(AxisZElement, StyleZ);

	const float SizeCoefficient = Style.SizeCoefficient.Get(1.0f);

	auto UpdatePlanarElement = [&, SizeCoefficient](UGizmoElementBox* InPlanarElement, const FGizmoElementTranslatePlanarStyleOverride& InPlanarStyleOverride, const EAxisList::Type InPlaneAxis)
	{
		if (!InPlanarElement)
		{
			return;
		}

		using namespace UE::Editor::InteractiveToolsFramework::Internal;

		FGizmoElementTranslatePlanarStyle PlanarStyle = Style.PlanarStyle;
		InPlanarStyleOverride.ApplyTo(PlanarStyle);

		const EAxisList::Type PlaneNormalAxis = GetPlaneNormalAxis(InPlaneAxis);

		// The two vectors below are used for positive offsets, so we convert them from cartesian (might be negative) to UE handedness, depending on the current coordinate system
		FVector UnusedForwardAxis;
		FVector PositiveUpDirection;
		FVector PositiveSideDirection;
		GetSignedAxisBasis(PlaneNormalAxis, UnusedForwardAxis, PositiveUpDirection, PositiveSideDirection);

		const float PlanarSize = PlanarStyle.Size * PlanarStyle.SizeMultiplier;

		// Prevent the handles from being centered (or negative)
		const float MinOffsetFromOrigin = PlanarSize * 0.5f;
		const FVector PlanarHandleCenter = (PositiveUpDirection + PositiveSideDirection) * FMath::Max(MinOffsetFromOrigin, PlanarStyle.OffsetFromOrigin) * SizeCoefficient;

		InPlanarElement->SetCenter(PlanarHandleCenter);
		InPlanarElement->SetDimensions(FVector(PlanarStyle.Thickness * PlanarStyle.LineThicknessMultiplier, PlanarSize, PlanarSize) * SizeCoefficient);

		using namespace UE::Editor::InteractiveToolsFramework;

		ApplyMaterialsToElement(InPlanarElement, PlanarStyle.Materials);
		ApplyColorsToElement(InPlanarElement, PlanarStyle.Colors);

		InPlanarElement->SetPixelHitDistanceThreshold(PlanarStyle.PixelHitDistanceThreshold);
	};

	UpdatePlanarElement(PlanarXYElement, PlanarXYStyle, EAxisList::XY);
	UpdatePlanarElement(PlanarYZElement, PlanarYZStyle, EAxisList::YZ);
	UpdatePlanarElement(PlanarXZElement, PlanarXZStyle, EAxisList::XZ);

	auto UpdateUniformElement = [Style = Style, SizeCoefficient](UGizmoElementLineBase* InElement)
	{
		const FGizmoElementTranslateUniformStyle& UniformStyle = Style.UniformStyle;

		UE::Editor::InteractiveToolsFramework::ApplyMaterialsToElement(InElement, UniformStyle.Materials);
		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(InElement, UniformStyle.Colors);
		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(InElement, UniformStyle.LineColors);

		InElement->SetLineThickness(
			FMath::Max(Style.MinLineThickness, UniformStyle.LineThickness * Style.LineThicknessMultiplier) * SizeCoefficient);

		InElement->SetPixelHitDistanceThreshold(UniformStyle.PixelHitDistanceThreshold);
		InElement->SetMinimumPixelHitDistanceThreshold(UniformStyle.MinimumPixelHitDistanceThreshold);
	};

	const FGizmoElementTranslateUniformStyle& UniformStyle = Style.UniformStyle;
	const float UniformSize = UniformStyle.Size * UniformStyle.SizeMultiplier * SizeCoefficient;

	// Uniform screen-space element
	if (UniformElement)
	{
		UpdateUniformElement(UniformElement);

		UniformElement->SetHeight(UniformSize);
		UniformElement->SetWidth(UniformSize);
	}

	if (UniformElementAlternate)
	{
		UpdateUniformElement(UniformElementAlternate);

		UniformElementAlternate->SetRadius(UniformSize * 0.5f);
	}

	// Origin Circle Element
	if (OriginElement)
	{
		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(OriginElement, Style.AxisStyle.Colors);

		OriginElement->SetMaterial(Style.AxisStyle.VertexColorMaterial);
		OriginElement->SetHoverMaterial(Style.AxisStyle.VertexColorMaterial);
		OriginElement->SetInteractMaterial(Style.AxisStyle.VertexColorMaterial);

		OriginElement->SetRadius(Style.OriginRadius * SizeCoefficient);
		OriginElement->SetVertexColor(Style.UniformStyle.Colors.Default.Get(FLinearColor::White));

		OriginElement->SetLineColor(FLinearColor::Transparent); // Effectively hides line for the default state
		OriginElement->SetHoverLineColor(FLinearColor::Black);
		OriginElement->SetInteractLineColor(FLinearColor::Black);

		constexpr float OtherStateRadiusMultiplier = 2.0f;
		OriginElement->SetHoverRadiusMultiplier(OtherStateRadiusMultiplier);
		OriginElement->SetInteractRadiusMultiplier(OtherStateRadiusMultiplier);
	}

	// Apply common delta element settings
	auto ApplyDeltaElementSettings = [&](UGizmoElementBase* InElement)
	{
		UE::Editor::InteractiveToolsFramework::ApplyMaterialsToElement(InElement, Style.AxisStyle.Materials);
		UE::Editor::InteractiveToolsFramework::ApplyColorsToElement(InElement, Style.AxisStyle.Colors);
	};

	// Origin Pivot Element
	if (DeltaOriginElement)
	{
		ApplyDeltaElementSettings(DeltaOriginElement);

		DeltaOriginElement->SetRadius(Style.OriginRadius * 2.0f * SizeCoefficient);

		DeltaOriginElement->SetMaterial(Style.AxisStyle.VertexColorMaterial);
		DeltaOriginElement->SetHoverMaterial(Style.AxisStyle.VertexColorMaterial);
		DeltaOriginElement->SetInteractMaterial(Style.AxisStyle.VertexColorMaterial);

		DeltaOriginElement->SetLineColor(FLinearColor::Transparent); // Effectively hides line for the default state
		DeltaOriginElement->SetHoverLineColor(FLinearColor::Black);
		DeltaOriginElement->SetInteractLineColor(FLinearColor::Black);

		DeltaOriginElement->SetLineThickness(FMath::Max(Style.MinLineThickness, Style.OriginLineThickness * SizeCoefficient));
		
		DeltaOriginElement->SetVisibleState(false);
	}

	// Delta Line Element
	if (DeltaLineElement)
	{
		DeltaLineElement->SetDirection(FVector::RightVector);
		DeltaLineElement->SetHeight(100.0f * SizeCoefficient);

		DeltaLineElement->SetRadius(FMath::Max(Style.MinLineThickness * 0.5f, Style.DeltaLineThickness.Get(Style.AxisStyle.LineThickness) * 0.5f * Style.AxisStyle.LineThicknessMultiplier) * SizeCoefficient);

		DeltaLineElement->SetIsDashed(true);
		DeltaLineElement->SetDashParameters(Style.DeltaLineDashSpacing, Style.DeltaLineDashGapSpacing);

		DeltaLineElement->SetScreenSpace(true);

		ApplyDeltaElementSettings(DeltaLineElement);

		DeltaLineElement->SetVisibleState(false);
	}
}

void UGizmoElementTranslateGroup::BeginDelta(const FDeltaParameters& InParameters)
{
	bIsShowingDelta = true;
	StartState.Initialize(InParameters);
	CurrentState = StartState;

	ForEachAxisElementByList([&](UGizmoElementTranslateAxis* InAxisElement, const EAxis::Type InAxis)
	{
		if (InAxisElement)
		{
			using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementTranslateAxisSetLocals;
			const UGizmoElementTranslateAxis::FDeltaParameters AxisDeltaParameters = MakeAxisDeltaParameters(InParameters, InAxis);

			InAxisElement->BeginDelta(AxisDeltaParameters);
		}
	},
	StartState.AxisList);

	if (DeltaOriginElement)
	{
		DeltaOriginElement->SetVisibleState(EnumHasAllFlags(Style.ShowFlags, EGizmoElementTranslateShowFlags::DeltaOrigin));
	}

	if (DeltaLineElement)
	{
		DeltaLineElement->SetVisibleState(EnumHasAllFlags(Style.ShowFlags, EGizmoElementTranslateShowFlags::DeltaLine));
	}

	// Update State
	UpdateDelta(InParameters);
}

void UGizmoElementTranslateGroup::UpdateDelta(const FDeltaParameters& InParameters)
{
	CurrentState.Update(InParameters);

	const FVector StartTranslation = StartState.Translation;
	const FVector CurrentTranslation = CurrentState.Translation;

	const FVector StartToCurrent = CurrentTranslation - StartTranslation;
	const double Length = StartToCurrent.Length();

	// Delta Line Element
	if (DeltaLineElement)
	{
		CurrentState.DistanceFromStart = Length;
		DeltaLineElement->SetHeight(static_cast<float>(Length));
		DeltaLineElement->SetDirection(StartToCurrent.GetSafeNormal());
	}

	ForEachAxisElementByList([&](UGizmoElementTranslateAxis* InAxisElement, const EAxis::Type InAxis)
	{
		if (InAxisElement)
		{
			using namespace UE::Editor::InteractiveToolsFramework::Private::GizmoElementTranslateAxisSetLocals;
			const UGizmoElementTranslateAxis::FDeltaParameters AxisDeltaParameters = MakeAxisDeltaParameters(InParameters, InAxis);

			InAxisElement->UpdateDelta(AxisDeltaParameters);
		}
	},
	StartState.AxisList);
}

void UGizmoElementTranslateGroup::EndDelta()
{
	bIsShowingDelta = false;
	StartState = CurrentState;

	if (DeltaOriginElement)
	{
		DeltaOriginElement->SetVisibleState(false);
	}

	if (DeltaLineElement)
	{
		DeltaLineElement->SetVisibleState(false);
	}

	if (StartState.bIsSingleAxis)
	{
		ForEachAxisElementByList([&](UGizmoElementTranslateAxis* InAxisElement, const EAxis::Type InAxis)
		{
			if (InAxisElement)
			{
				InAxisElement->EndDelta();
			}
		},
		StartState.AxisList);
	}
}

void UGizmoElementTranslateGroup::ForEachAxisElement(const TFunctionRef<void(UGizmoElementTranslateAxis* InElement)>& InFunc) const
{
	if (AxisXElement)
	{
		InFunc(AxisXElement);
	}

	if (AxisYElement)
	{
		InFunc(AxisYElement);
	}

	if (AxisZElement)
	{
		InFunc(AxisZElement);
	}
}

void UGizmoElementTranslateGroup::ForEachAxisElementByList(
	const TFunctionRef<void(UGizmoElementTranslateAxis* InElement, const EAxis::Type InAxis)>& InFunc,
	const EAxisList::Type InAxisList) const
{
	auto CallOnElement = [&](UGizmoElementTranslateAxis* InElement, const EAxis::Type InAxis, const EAxisList::Type InAxisXYZ, const EAxisList::Type InAxisLUF)
	{
		if ((EnumHasAnyFlags(InAxisList, InAxisXYZ)
			|| EnumHasAnyFlags(InAxisList, InAxisLUF))
			&& InElement)
		{
			InFunc(InElement, InAxis);
		}
	};

	CallOnElement(AxisXElement, EAxis::X, EAxisList::X, EAxisList::Forward);
	CallOnElement(AxisYElement, EAxis::Y, EAxisList::Y, EAxisList::Left);
	CallOnElement(AxisZElement, EAxis::Z, EAxisList::Z, EAxisList::Up);
}

bool UGizmoElementTranslateGroup::UpdateDeltaRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalOrigin, FRenderTraversalState& InOutRenderState)
{
	// Use the SceneView from the RenderAPI which represents the current viewport.
	// Not using GizmoViewContext, which might not have the right viewport info for the current viewport, in case of split views
	const FSceneView* SceneView = RenderAPI->GetSceneView();
	check(SceneView);
	const UE::GizmoRenderingUtil::FSceneViewWrapper SceneViewInterface(*SceneView);

	const FVector4 ScreenSpaceStart = SceneView->WorldToScreen(StartState.Translation);

	// If StartState.Translation is behind the camera we need to use CurrentState information
	// This might happen e.g. for Shift + Drag, camera moves while translating
	const bool bIsBehindCamera = ScreenSpaceStart.W <= 0.0;
	const FVector& ReferencePoint = bIsBehindCamera ? CurrentState.Translation : StartState.Translation;
	const double ScreenSpaceDepth = bIsBehindCamera ? SceneView->WorldToScreen(CurrentState.Translation).W : ScreenSpaceStart.W;

	InOutRenderState.PixelToWorldScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale<double>(&SceneViewInterface, ReferencePoint);
	InOutRenderState.DepthAtPixelToWorldReferencePoint = ScreenSpaceDepth;
	InOutRenderState.LocalToWorldTransform.SetRotation(FQuat::Identity);
	InOutRenderState.LocalToWorldTransform.SetTranslation(StartState.Translation);
	InOutRenderState.LocalToWorldTransform.SetScale3D(FVector::OneVector);

	return UpdateRenderState(RenderAPI, InLocalOrigin, InOutRenderState);
}
