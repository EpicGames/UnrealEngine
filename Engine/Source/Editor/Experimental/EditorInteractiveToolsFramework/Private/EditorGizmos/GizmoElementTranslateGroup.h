// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoElementGroup.h"
#include "EditorGizmoElementInterfaces.h"
#include "EditorGizmos/GizmoElementTranslate.h"
#include "FrameTypes.h"

#include "GizmoElementTranslateGroup.generated.h"

class IToolkitHost;
class UGizmoElementBox;
class UGizmoElementCircle;
class UGizmoElementCylinder;
class UGizmoElementRectangle;
class UGizmoElementSphere;
class UGizmoElementTranslateAxis;

/** This style contains optional per-axis overrides. FGizmoElementTranslateAxisStyle is used for the shared style. */
USTRUCT(MinimalAPI)
struct FGizmoElementTranslateAxisStyleOverride
{
	GENERATED_BODY()

	/** The Axis color, usually Red for X, etc. */
	UPROPERTY()
	TOptional<FGizmoPerStateValueLinearColor> Colors;

	/** Optional override for axis materials per interaction state. */
	UPROPERTY()
	TOptional<FGizmoPerStateValueMaterialVariant> Materials;

	/** Optional override for the delta visualization material. */
	UPROPERTY()
	TOptional<TObjectPtr<UMaterialInterface>> DeltaMaterial;

	/** Optional override for the vertex-color material. */
	UPROPERTY()
	TOptional<TObjectPtr<UMaterialInterface>> VertexColorMaterial;

	/** Applies the overrides, if set, to the given Style. */
	void ApplyTo(FGizmoElementTranslateAxisStyle& InOutStyle) const;
};

/** This style contains optional per-planar overrides. FGizmoElementTranslatePlanarStyle is used for the shared style. */
USTRUCT(MinimalAPI)
struct FGizmoElementTranslatePlanarStyleOverride
{
	GENERATED_BODY()

	/** The Plane color, usually Red for YZ, etc. */
	UPROPERTY()
	TOptional<FGizmoPerStateValueLinearColor> Colors;

	/** Optional override for planar materials per interaction state. */
	UPROPERTY()
	TOptional<FGizmoPerStateValueMaterialVariant> Materials;

	/** Optional override for the vertex-color material. */
	UPROPERTY()
	TOptional<TObjectPtr<UMaterialInterface>> VertexColorMaterial;

	/** Applies the overrides, if set, to the given Style. */
	void ApplyTo(FGizmoElementTranslatePlanarStyle& InOutStyle) const;
};

/**
 * Container for all translation gizmo elements: per-axis arrows (X, Y, Z),
 * planar handles (XY, YZ, XZ), and the uniform screen-space handle.
 * Manages style propagation, axis enable/disable, and delta visualization across all children.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementTranslateGroup
	: public UGizmoElementGroupBase
	, public UE::Editor::InteractiveToolsFramework::IPlaneProvider
{
	GENERATED_BODY()

public:
	/** Per-Axis parameters. */
	struct FAxisParameters
	{
		/** Hit-test part identifier for this axis element. */
		uint32 PartId = 0;
		/** Which axis this element represents. */
		EAxisList::Type Axis = EAxisList::None;
		/** Optional style overrides for this axis. */
		FGizmoElementTranslateAxisStyleOverride StyleOverride;
	};

	/** Per-Plane parameters. */
	struct FPlanarParameters
	{
		/** Hit-test part identifier for this planar element. */
		uint32 PartId = 0;
		/** Which plane (axis pair) this element represents. */
		EAxisList::Type Axis = EAxisList::None;
		/** Optional style overrides for this planar element. */
		FGizmoElementTranslatePlanarStyleOverride StyleOverride;
	};

	/** Parameters shared between BeginDelta and UpdateDelta calls, describing the current interaction state. */
	struct FDeltaParameters
	{
		/** The current world transform of the gizmo. */
		FTransform Transform = FTransform::Identity;

		/** Screen-space location of the gizmo origin, used for label placement. */
		FVector2D TransformLocation2D = FVector2D::ZeroVector;

		/** The active coordinate system (World, Local, etc.). */
		EToolContextCoordinateSystem CoordinateSystem = EToolContextCoordinateSystem::World;

		/** Normal of the interaction plane. */
		FVector PlaneNormal = FVector::UpVector;

		/** The axis or axes being translated. */
		EAxisList::Type AxisList = EAxisList::None;

		/** Whether this is an indirect interaction (e.g. input field rather than drag). */
		bool bIsIndirectInteraction = false;
	};

public:
	//~ Begin UGizmoElementBase
	virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	//~ End UGizmoElementBase

	/** Renders various debug visualizations, if enabled. */
	void DrawDebug(IToolsContextRenderAPI* RenderAPI, const FGizmoDebugSettings& InSettings, const uint32 InPartId = 0);

	/** Sets the host to add/remove a widget to.
	* Currently, this is a Toolkit host, but it could be abstracted to an ISlateWidgetHost (see IEditorViewportClientProxy). */
	void SetWidgetHost(IToolkitHost* const InWidgetHost);

	/** Sets a function that provides the current snapping settings to use during interaction. */
	void SetGetCurrentSnappingSettingsFunction(const TFunction<FToolContextSnappingConfiguration()>& InFunction);

	/** Enables the elements specified in the provided AxisList, and disables those that aren't. */
	void SetAxisEnabled(const EAxisList::Type InAxisListToEnable);

	/** Enables the planar elements specified in the provided AxisList and disables those that aren't. */
	void SetPlanarEnabled(const EAxisList::Type InAxisListToEnable);

	/** Enables the uniform screen-space element. */
	void SetUniformEnabled(const bool bInEnable);

	/** Returns the current shared translation style. */
	const FGizmoElementTranslateStyle& GetStyle() const;

	/** Sets and applies the given Style + Overrides. */
	void SetStyle(
		const FGizmoElementTranslateStyle& InStyle,
		const FGizmoElementTranslateAxisStyleOverride& InAxisStyleX,
		const FGizmoElementTranslateAxisStyleOverride& InAxisStyleY,
		const FGizmoElementTranslateAxisStyleOverride& InAxisStyleZ,
		const FGizmoElementTranslatePlanarStyleOverride& InPlanarStyleXY,
		const FGizmoElementTranslatePlanarStyleOverride& InPlanarStyleYZ,
		const FGizmoElementTranslatePlanarStyleOverride& InPlanarStyleXZ);

	//~ Begin IPlaneProvider
	/** Creates the interaction plane for the active translate axis or plane, transformed into the provided coordinate space. */
	virtual UE::Geometry::FFrame3d MakePlane(const FTransform& InTransform, const UGizmoViewContext* InViewContext, const EToolContextCoordinateSystem InCoordinateSystem, const EAxisList::Type InAxisList) const override;
	//~ End IPlaneProvider

	/**
	 * Initialize the translation group with all axis, planar, and uniform elements.
	 * Must be called before the group can render or receive interactions.
	 * @param InPartId Base part identifier for the group.
	 * @param InAxisList Which axes to initially enable.
	 * @param InStyle Shared visual style for all translation elements.
	 * @param InAxisX Parameters for the X axis element.
	 * @param InAxisY Parameters for the Y axis element.
	 * @param InAxisZ Parameters for the Z axis element.
	 * @param InPlanarXY Parameters for the XY planar element.
	 * @param InPlanarYZ Parameters for the YZ planar element.
	 * @param InPlanarXZ Parameters for the XZ planar element.
	 * @param InUniformPartId Part identifier for the uniform screen-space element.
	 */
	void Setup(
		const uint32 InPartId,
		const EAxisList::Type InAxisList,
		const FGizmoElementTranslateStyle& InStyle,
		const FAxisParameters& InAxisX,
		const FAxisParameters& InAxisY,
		const FAxisParameters& InAxisZ,
		const FPlanarParameters& InPlanarXY,
		const FPlanarParameters& InPlanarYZ,
		const FPlanarParameters& InPlanarXZ,
		const uint32 InUniformPartId);

	/** Rebuilds child elements to reflect the current style. Call after modifying style properties. */
	void UpdateElements();

	/** Begin a delta interaction, recording the start state. Propagates to the active axis element. */
	void BeginDelta(const FDeltaParameters& InParameters);

	/** Update the delta visualization with the current interaction state. */
	void UpdateDelta(const FDeltaParameters& InParameters);

	/** End the current delta interaction and hide delta visuals. */
	void EndDelta();

	/** Get the axis element for the given single axis (X, Y, or Z). Returns nullptr if not set up. */
	UGizmoElementTranslateAxis* GetAxisElement(const EAxis::Type InAxis) const;

	/** Get the planar element for the given axis pair (XY, YZ, or XZ). Returns nullptr if not set up. */
	UGizmoElementBox* GetPlanarElement(const EAxisList::Type InAxis) const;

private:
	/** Applies the current style and per-axis/planar overrides to all child elements. */
	void ApplyStyle();

	/** Invokes the given function on each axis element (X, Y, Z). */
	void ForEachAxisElement(const TFunctionRef<void(UGizmoElementTranslateAxis* InElement)>& InFunc) const;
	/** Invokes the given function on each axis element whose axis is included in InAxisList. */
	void ForEachAxisElementByList(const TFunctionRef<void(UGizmoElementTranslateAxis* InElement, const EAxis::Type InAxis)>& InFunc, const EAxisList::Type InAxisList) const;

	/** Updates the render state for delta elements during render traversal, using the start transform rather than the current. Returns true if visible in the current view. */
	bool UpdateDeltaRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalOrigin, FRenderTraversalState& InOutRenderState);

	/** Returns true if a delta interaction is currently active. */
	bool IsInteracting() const;

	/** Applies and propagates the current snapping-settings provider function, if set. */
	void ApplyGetCurrentSnappingSettingsFunction();

private:
	/** Flag indicating element validity - generally true after Setup is called. */
	std::atomic_bool bIsValid = false;

	/** Style contains various uniform and varying properties that affect the appearance of this gizmo element. */
	UPROPERTY()
	FGizmoElementTranslateStyle Style;

	/** Optional overrides for the X axis. */
	UPROPERTY()
	FGizmoElementTranslateAxisStyleOverride StyleX;

	/** Optional overrides for the Y axis. */
	UPROPERTY()
	FGizmoElementTranslateAxisStyleOverride StyleY;

	/** Optional overrides for the Z axis. */
	UPROPERTY()
	FGizmoElementTranslateAxisStyleOverride StyleZ;

	/** Optional overrides for the XY plane. */
	UPROPERTY()
	FGizmoElementTranslatePlanarStyleOverride PlanarXYStyle;

	/** Optional overrides for the YZ plane. */
	UPROPERTY()
	FGizmoElementTranslatePlanarStyleOverride PlanarYZStyle;

	/** Optional overrides for the XZ plane. */
	UPROPERTY()
	FGizmoElementTranslatePlanarStyleOverride PlanarXZStyle;

	/** Axis X element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementTranslateAxis> AxisXElement;

	/** Axis Y element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementTranslateAxis> AxisYElement;
	
	/** Axis Z element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementTranslateAxis> AxisZElement;

	/** Planar XY element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> PlanarXYElement;

	/** Planar YZ element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> PlanarYZElement;

	/** Planar XZ element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> PlanarXZElement;

	/** Uniform element. */
	UPROPERTY()
	TObjectPtr<UGizmoElementRectangle> UniformElement;

	/** Uniform element (alternate). Switches between this and the other Uniform element depending on snap state. */
	UPROPERTY()
	TObjectPtr<UGizmoElementCircle> UniformElementAlternate;

	/** Circle element drawn at the gizmo origin. */
	UPROPERTY()
	TObjectPtr<UGizmoElementCircle> OriginElement;

	/** Circle element drawn at the delta origin during translation. */
	UPROPERTY()
	TObjectPtr<UGizmoElementCircle> DeltaOriginElement;

	/** Line element visualizing the delta distance during translation. */
	UPROPERTY()
	TObjectPtr<UGizmoElementCylinder> DeltaLineElement;

	/** Callback that provides the current snapping configuration for interaction. */
	TFunction<FToolContextSnappingConfiguration()> GetCurrentSnappingSettingsFunction = nullptr;

private:
	/** Snapshot of the interaction state at a given point in time, used for delta visualization. */
	struct FState
	{
		/** World transform of the gizmo. */
		FTransform Transform;

		/** Screen-space location of the gizmo origin. */
		FVector2D TransformLocation2D = FVector2D::ZeroVector;

		/** Accumulated translation offset from the start of the interaction. */
		FVector Translation = FVector::ZeroVector;

		/** Normal of the interaction plane. */
		FVector PlaneNormal = FVector::UpVector;

		/** The axis or axes being translated. */
		EAxisList::Type AxisList = EAxisList::None;

		/** True if the translation is constrained to a single axis. */
		bool bIsSingleAxis = false;

		/** Signed distance moved from the start position along the translation direction. */
		double DistanceFromStart = 0.0;

		/** Initializes this state from the given delta parameters at interaction start. */
		void Initialize(const FDeltaParameters& InParameters);

		/** Updates this state from the given delta parameters during interaction. */
		void Update(const FDeltaParameters& InParameters);
	};

	/** Captured state at the beginning of the current delta interaction. */
	FState StartState;

	/** Most recently updated state during the current delta interaction. */
	FState CurrentState;

	/** True while delta visualization elements are being displayed. */
	bool bIsShowingDelta = false;
};
