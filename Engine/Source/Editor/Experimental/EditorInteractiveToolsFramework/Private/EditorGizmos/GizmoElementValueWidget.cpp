// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementValueWidget.h"

#include "Styling/SlateColor.h"
#include "Widgets/SGizmoElementValue.h"

UGizmoElementValueWidget::UGizmoElementValueWidget()
{
	SAssignNew(ValueWidget, UE::Editor::InteractiveToolsFramework::SGizmoElementValue)
	.Text_UObject(this, &UGizmoElementValueWidget::GetText);

	SetWidget(ValueWidget.ToSharedRef());
}

void UGizmoElementValueWidget::SetText(const FText& InText)
{
	Text = InText;
}

FText UGizmoElementValueWidget::GetText() const
{
	return Text;
}

void UGizmoElementValueWidget::SetUnitText(const double InValue, const EUnitType InUnitType, const FNumberFormattingOptions& InNumberFormattingOptions)
{
	switch (InUnitType)
	{
	case EUnitType::Angle:
		{
			SetText(
				FText::Format(FText::FromString(TEXT("{0} {1}")),
				FText::AsNumber(FMath::RadiansToDegrees(InValue), &InNumberFormattingOptions),
				FText::FromString(FUnitConversion::GetUnitDisplayString(EUnit::Degrees)))
			);
		}
		break;

	case EUnitType::Distance:
		{
			FNumericUnit<double> DisplayUnit = FUnitConversion::QuantizeUnitsToBestFit<double>(InValue, EUnit::Centimeters);

			SetText(
				FText::Format(FText::FromString(TEXT("{0} {1}")),
				FText::AsNumber(DisplayUnit.Value, &InNumberFormattingOptions),
				FText::FromString(FUnitConversion::GetUnitDisplayString(DisplayUnit.Units)))
			);
		}
		break;

	default:
		{
			SetText(
				FText::Format(FText::FromString(TEXT("{0}")),
				FText::AsNumber(InValue, &InNumberFormattingOptions))
			);
		}
		break;
	}
}

FSlateColor UGizmoElementValueWidget::GetBackgroundColorAndOpacity() const
{
	return MeshRenderAttributes.GetVertexColor(GetElementInteractionState());
}
