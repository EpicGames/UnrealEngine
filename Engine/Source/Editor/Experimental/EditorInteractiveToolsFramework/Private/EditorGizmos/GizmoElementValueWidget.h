// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GizmoElementWidget.h"
#include "Math/UnitConversion.h"
#include "Styling/SlateColor.h"

#include "GizmoElementValueWidget.generated.h"

class SWidget;

/**
 * A widget-based gizmo element that displays a formatted numeric value label.
 * Used to show delta values (e.g. translation distance, rotation angle, scale factor)
 * next to the gizmo during interaction.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementValueWidget : public UGizmoElementWidget
{
	GENERATED_BODY()

public:
	UGizmoElementValueWidget();

	/** Set the display text directly. */
	void SetText(const FText& InText);

	/** Get the current display text. */
	FText GetText() const;

	/**
	 * Format and set the display text from a numeric value with unit conversion.
	 * @param InValue The numeric value to display.
	 * @param InUnitType The unit type for display formatting (e.g. centimeters, degrees).
	 * @param InNumberFormattingOptions Formatting options controlling decimal places, grouping, etc.
	 */
	void SetUnitText(const double InValue, const EUnitType InUnitType, const FNumberFormattingOptions& InNumberFormattingOptions = FNumberFormattingOptions::DefaultNoGrouping());

private:
	/** Returns the background color and opacity for the value label, based on the current gizmo state. */
	FSlateColor GetBackgroundColorAndOpacity() const;

private:
	/** The display text shown in the value widget. */
	UPROPERTY(Getter, Setter)
	FText Text;

	/** The Slate widget instance used to render the value label in the viewport. */
	TSharedPtr<SWidget> ValueWidget = nullptr;
};
