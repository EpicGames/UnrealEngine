// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoElementWidget.h"

#include "CanvasTypes.h"
#include "IAssetViewport.h"
#include "Toolkits/IToolkitHost.h"
#include "Widgets/Layout/SBox.h"

UGizmoElementWidget::UGizmoElementWidget()
{
	ViewAlignType = EGizmoElementViewAlignType::PointScreen;
	ViewAlignAxis = FVector::UpVector;
	ViewAlignNormal = -FVector::ForwardVector;

	// We only draw in the single focused/active/primary viewport
	// This is both a limitation and design choice. If we need to show it in all viewports, the widget must handled for each individually.

	ResetClampToScreenFunction();

	SAssignNew(HostWidget, SBox)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	.Visibility_UObject(this, &UGizmoElementWidget::GetVisibility);
}

void UGizmoElementWidget::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	// @note: intentionally empty, we only draw in DrawHUD()
}

void UGizmoElementWidget::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	FRenderTraversalState CurrentRenderState(RenderState);
	if (UpdateRenderState(RenderAPI, Location, CurrentRenderState))
	{
		const FSceneView* SceneView = RenderAPI->GetSceneView();
		if (SceneView)
		{
			FVector2D ScreenLocation;
			if (SceneView->WorldToPixel(CurrentRenderState.LocalToWorldTransform.GetLocation(), ScreenLocation))
			{
				const double DPIScale = Canvas->GetDPIScale();

				// Compute ViewCenter using unconstrained view rectangle (prevents offsets when viewport has letterboxing)
				const FVector2D ViewCenter = FVector2D(SceneView->UnconstrainedViewRect.Width(), SceneView->UnconstrainedViewRect.Height()) / 2.0f;

				FVector2D WidgetLocation = FVector2D(ScreenLocation.X - ViewCenter.X, ScreenLocation.Y - ViewCenter.Y);

				const FVector2D ScaledOffset2D = Offset2D * Offset2DMultiplier;

				if (bClampToScreen)
				{
					const FVector2D WidgetSize = Widget.IsValid() ? FVector2D(Widget->GetDesiredSize().GetAbs()) / DPIScale : FVector2D::ZeroVector;

					if (ensure(ClampToScreenFunction.IsSet()))
					{
						WidgetLocation = ClampToScreenFunction(WidgetLocation + ScaledOffset2D, WidgetSize, SceneView->UnscaledViewRect);
						WidgetLocation -= ScaledOffset2D;
					}

					// Use the default clamp function for the offset
					WidgetLocation = ClampToScreenInternal(WidgetLocation + ScaledOffset2D, WidgetSize, SceneView->UnscaledViewRect);
				}
				else
				{
					WidgetLocation += ScaledOffset2D;
				}

				WidgetLocation /= DPIScale;

				UpdateWidgetTransform(WidgetLocation);
			}
		}
	}
}

bool UGizmoElementWidget::IsVisible(
	const FSceneView* View, EViewInteractionState InCurrentViewInteractionState, EGizmoElementInteractionState InCurrentInteractionState, const FTransform& InLocalToWorldTransform,
	const FVector& InLocalCenter) const
{
	if (!Super::IsVisible(View, InCurrentViewInteractionState, InCurrentInteractionState, InLocalToWorldTransform, InLocalCenter))
	{
		return false;
	}

	const bool bIsFocusedView = !!(InCurrentViewInteractionState & EViewInteractionState::Focused);

	// We only draw in the single focused/active/primary viewport
	return bIsFocusedView;
}

FInputRayHit UGizmoElementWidget::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	// @todo: test against widget rect
	return FInputRayHit();
}

const FVector UGizmoElementWidget::GetLocation() const
{
	return Location;
}

void UGizmoElementWidget::SetLocation(const FVector& InLocation)
{
	Location = InLocation;
}

const FVector2D UGizmoElementWidget::GetOffset2D() const
{
	return Offset2D;
}

void UGizmoElementWidget::SetOffset2D(const FVector2D& InOffset2D)
{
	Offset2D = InOffset2D;
}

bool UGizmoElementWidget::GetClampToScreen() const
{
	return bClampToScreen;
}

void UGizmoElementWidget::SetClampToScreen(bool bInClampToScreen)
{
	bClampToScreen = bInClampToScreen;
}

void UGizmoElementWidget::SetClampToScreenFunction(FClampToScreenFunction&& InClampToScreenFunction)
{
	ClampToScreenFunction = MoveTemp(InClampToScreenFunction);
}

void UGizmoElementWidget::ResetClampToScreenFunction()
{
	ClampToScreenFunction = [&](const FVector2D& InWidgetLocation, const FVector2D& InWidgetSize, const FIntRect& InViewRect) -> FVector2D
	{
		return ClampToScreenInternal(InWidgetLocation, InWidgetSize, InViewRect);
	};
}

const TSharedPtr<SWidget>& UGizmoElementWidget::GetWidget() const
{
	return Widget;
}

void UGizmoElementWidget::SetWidget(const TSharedPtr<SWidget>& InWidget)
{
	if (!ensure(HostWidget.IsValid()))
	{
		return;
	}

	if (Widget.IsValid())
	{
		RemoveWidget();
		Widget.Reset();
	}

	Widget = InWidget;
	HostWidget->SetContent(SNullWidget::NullWidget);

	if (InWidget.IsValid())
	{
		HostWidget->SetContent(Widget.ToSharedRef());	
	}
}

void UGizmoElementWidget::SetWidgetHost(IToolkitHost* const InWidgetHost)
{
	if (!ensure(HostWidget.IsValid()))
	{
		return;
	}

	if (WidgetHost)
	{
		WidgetHost->OnActiveViewportChanged().RemoveAll(this);
		WidgetHost->RemoveViewportOverlayWidget(HostWidget.ToSharedRef());
	}

	WidgetHost = InWidgetHost;
	if (InWidgetHost)
	{
		InWidgetHost->OnActiveViewportChanged().AddUObject(this, &UGizmoElementWidget::OnActiveViewportChanged);
		WidgetHost->AddViewportOverlayWidget(HostWidget.ToSharedRef());
	}
}

EVisibility UGizmoElementWidget::GetVisibility() const
{
	return GetVisibleState() && GetEnabledForInteractionState(GetElementInteractionState())
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

void UGizmoElementWidget::RemoveWidget()
{
	if (!Widget.IsValid())
	{
		return;
	}

	if (!ensure(HostWidget.IsValid()))
	{
		return;
	}

	if (WidgetHost)
	{
		WidgetHost->RemoveViewportOverlayWidget(HostWidget.ToSharedRef());
	}
}

void UGizmoElementWidget::UpdateWidgetTransform(const FVector2D& InLocation)
{
	if (HostWidget.IsValid())
	{
		FMargin Padding;
		Padding.Left = static_cast<float>(InLocation.X);
		Padding.Top = static_cast<float>(InLocation.Y);
		
		HostWidget->SetPadding(Padding);	
	}
}

void UGizmoElementWidget::OnActiveViewportChanged(TSharedPtr<IAssetViewport> InOldViewport, TSharedPtr<IAssetViewport> InNewViewport)
{
	if (!Widget.IsValid())
	{
		return;
	}

	if (!ensure(HostWidget.IsValid()))
	{
		return;
	}

	if (WidgetHost)
	{
		if (InOldViewport.IsValid())
		{
			WidgetHost->RemoveViewportOverlayWidget(HostWidget.ToSharedRef(), InOldViewport);
		}

		if (InNewViewport.IsValid())
		{
			WidgetHost->AddViewportOverlayWidget(HostWidget.ToSharedRef(), InNewViewport);
		}
	}
}

FVector2D UGizmoElementWidget::ClampToScreenInternal(const FVector2D& InWidgetLocation, const FVector2D& InWidgetSize, const FIntRect& InViewRect) const
{
	const FVector2D HalfViewSize = FVector2D(InViewRect.Size()) / 2.0;
	const FVector2D HalfWidgetSize = InWidgetSize / 2.0;

	// Assumes center alignment in canvas @see UGizmoElementWidget::UGizmoElementWidget()
	const FVector2D ExtraPadding(8.0);

	const FVector2D PaddedViewMin = -HalfViewSize + HalfWidgetSize + ExtraPadding;
	const FVector2D PaddedViewMax = HalfViewSize - HalfWidgetSize - ExtraPadding;

	return FVector2D::Clamp(InWidgetLocation, PaddedViewMin, PaddedViewMax);
}
