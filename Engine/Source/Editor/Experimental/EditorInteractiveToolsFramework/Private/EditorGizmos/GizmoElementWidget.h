// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "BaseGizmos/GizmoElementBase.h"
#include "Layout/Visibility.h"

#include "GizmoElementWidget.generated.h"

class IAssetViewport;
class IToolkitHost;
class SBox;

/** Represents a widget element in the Gizmo system.
 * This is used to add functionality and visual representation
 * associated with a widget in the context of a Gizmo. */
UCLASS(Transient, MinimalAPI)
class UGizmoElementWidget : public UGizmoElementBase
{
	GENERATED_BODY()

public:
	UGizmoElementWidget();

	//~ Begin UGizmoElementBase Interface.
	virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	virtual bool IsVisible(const FSceneView* View, EViewInteractionState InCurrentViewInteractionState, EGizmoElementInteractionState InCurrentInteractionState, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const override;
	//~ End UGizmoElementBase Interface.

	const FVector GetLocation() const;
	void SetLocation(const FVector& InLocation);

	const FVector2D GetOffset2D() const;
	void SetOffset2D(const FVector2D& InOffset2D);

	bool GetClampToScreen() const;
	void SetClampToScreen(bool bInClampToScreen);

	using FClampToScreenFunction = TUniqueFunction<FVector2D(const FVector2D& WidgetLocation, const FVector2D& WidgetSize, const FIntRect& ViewRect)>;

	/** Set a custom function to use when ClampToScreen is true. */
	void SetClampToScreenFunction(FClampToScreenFunction&& InClampToScreenFunction);

	/** Resets the ClampToScreen function to default behavior. */
	void ResetClampToScreenFunction();

	const TSharedPtr<SWidget>& GetWidget() const;
	void SetWidget(const TSharedPtr<SWidget>& InWidget);

	/** Sets the host to add/remove a widget to.
	 * Currently, this is a Toolkit host, but it could be abstracted to an ISlateWidgetHost (see IEditorViewportClientProxy). */
	void SetWidgetHost(IToolkitHost* const InWidgetHost);

protected:
	/** The location of the widget, in world space. Note that this is always projected into screen space. */
	UPROPERTY(Getter, Setter)
	FVector Location = FVector::ZeroVector;

	/** Offset to apply in screen space, applied after converting from world to screen. */
	UPROPERTY(Getter, Setter)
	FVector2D Offset2D = FVector2D::ZeroVector;

	/**
	 * Whether to clamp the widget to screen bounds.
	 * Note that this has the effect of ignoring culling, so may require manual visibility handling.
	 * The clamping is based on the viewport size and widgets' desired size.
	 */
	UPROPERTY(Getter = "GetClampToScreen", Setter = "SetClampToScreen")
	bool bClampToScreen = false;

	/** An overridable function to retrieve the clamped widget location. Useful for when the clamped location should be based on an intersection vs. simple clamp. */
	FClampToScreenFunction ClampToScreenFunction;

private:
	/** Get the widget visibility based on the element state. */
	EVisibility GetVisibility() const;

	/** Remove the widget from the current viewport. */
	void RemoveWidget();

	/** Update the widget's transform based on the current location within the viewport. */
	void UpdateWidgetTransform(const FVector2D& InLocation);

	/** Callback for when the active viewport changes, to re-add the widget to the new viewport. */
	void OnActiveViewportChanged(TSharedPtr<IAssetViewport> InOldViewport, TSharedPtr<IAssetViewport> InNewViewport);

	/** Internal implementation of the default ClampToScreen function. */
	FVector2D ClampToScreenInternal(const FVector2D& InWidgetLocation, const FVector2D& InWidgetSize, const FIntRect& InViewRect) const;

private:
	const FVector2D Offset2DMultiplier = FVector2D(1.0, -1.0); // Invert Y axis for screen space
	
	/** The outer widget that's transformed from world to screen space, with the user-specified Widget as the child. */
	TSharedPtr<SBox> HostWidget;

	/** The actual widget to display. */
	TSharedPtr<SWidget> Widget;

	/** The current toolkit the widget is added to, containing a viewport. */
	IToolkitHost* WidgetHost = nullptr;
};
