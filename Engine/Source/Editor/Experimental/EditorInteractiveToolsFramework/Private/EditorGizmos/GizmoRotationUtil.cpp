// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/GizmoRotationUtil.h"
#include "AnimationCoreLibrary.h"
#include "Components/SceneComponent.h"

#include "EditorGizmos/TransformGizmoInterfaces.h"

#include "Elements/Actor/ActorElementWorldInterface.h"
#include "Elements/Actor/ActorElementEditorWorldInterface.h"
#include "Elements/Actor/ActorElementData.h"

#include "Elements/Component/ComponentElementWorldInterface.h"
#include "Elements/Component/ComponentElementEditorWorldInterface.h"
#include "Elements/Component/ComponentElementData.h"

#include "GameFramework/Actor.h"

namespace UE::GizmoRotationUtil
{
	
void DecomposeRotations(const FTransform& InTransform, const FRotationContext& InRotationContext, FRotationDecomposition& OutDecomposition)
{
	const FQuat QX = FQuat(FRotator(0.0, 0.0, InRotationContext.Rotation.Roll));
	const FQuat QY = FQuat(FRotator(InRotationContext.Rotation.Pitch, 0.0, 0.0));
	const FQuat QZ = FQuat(FRotator(0.0, InRotationContext.Rotation.Yaw, 0.0));
		
	FQuat& RX = OutDecomposition.R[0];
	FQuat& RY = OutDecomposition.R[1];
	FQuat& RZ = OutDecomposition.R[2];

	switch (InRotationContext.RotationOrder)
	{
	case EEulerRotationOrder::XYZ:
		RZ = QZ;
		RY = QZ * QY;
		RX = QZ * QY * QX;
		break;
	case EEulerRotationOrder::XZY:
		RY = QY;
		RZ = QY * QZ;
		RX = QY * QZ * QX;
		break;
	case EEulerRotationOrder::YXZ:
		RZ = QZ;
		RX = QZ * QX;
		RY = QZ * QX * QY;
		break;
	case EEulerRotationOrder::YZX:
		RX = QX;
		RZ = QX * QZ;
		RY = QX * QZ * QY;
		break;
	case EEulerRotationOrder::ZXY:
		RY = QY;
		RX = QY * QX;
		RZ = QY * QX * QZ;
		break;
	case EEulerRotationOrder::ZYX:
		RX = QX;
		RY = QX * QY;
		RZ = QX * QY * QZ;
		break;
	default:
		RX = QX;
		RZ = QX * QZ;
		RY = QX * QZ * QY;
		break;
	}

	const FVector GlobalScale = InTransform.GetScale3D();
	const FVector OffsetScale = InRotationContext.Offset.GetScale3D();
	if (FTransform::AnyHasNegativeScale(GlobalScale, OffsetScale))
	{
		// use FTransform instead to compensate for scaling
		const FTransform TRX(RX), TRY(RY), TRZ(RZ);
		
		// compute global orientations
		const FTransform GlobalRX = TRX * InRotationContext.Offset;
		const FTransform GlobalRY = TRY * InRotationContext.Offset;
		const FTransform GlobalRZ = TRZ * InRotationContext.Offset;
		
		// switch them back to gizmo space
		const FTransform TransformNotRotation(FQuat::Identity, InTransform.GetTranslation(), GlobalScale);
		RX = GlobalRX.GetRelativeTransform(TransformNotRotation).GetRotation();
		RY = GlobalRY.GetRelativeTransform(TransformNotRotation).GetRotation();
		RZ = GlobalRZ.GetRelativeTransform(TransformNotRotation).GetRotation();	
	}
	else
	{
		const FQuat RotationOffset = InRotationContext.Offset.GetRotation();
		RX = RotationOffset * RX;
		RY = RotationOffset * RY;
		RZ = RotationOffset * RZ;
	}
}

FVector GetRotationAxis(const FTransform& InTransform, const FRotationContext& InRotationContext, const int32 InAxis)
{
	static const TArray RotateAxis({FVector::XAxisVector, FVector::YAxisVector, -FVector::ZAxisVector});
	if (!ensure(RotateAxis.IsValidIndex(InAxis)))
	{
		return FVector::Zero();
	}

	FRotationDecomposition Decomposition;
	DecomposeRotations(InTransform, InRotationContext, Decomposition);

	// handle negative scaling
	FVector ScaleSign = InRotationContext.Offset.GetScale3D().GetSignVector();
	if (ScaleSign[0] < 0.0 || ScaleSign[1] < 0.0 || ScaleSign[2] < 0.0)
	{
		const double GlobalScale = ScaleSign[0] * ScaleSign[1] * ScaleSign[2];
		ScaleSign *= GlobalScale;
		return Decomposition.R[InAxis] * (RotateAxis[InAxis] * ScaleSign);
	}

	return Decomposition.R[InAxis] * (RotateAxis[InAxis] * ScaleSign);
}

FRelativeTransformInterfaceRegistry& FRelativeTransformInterfaceRegistry::Get()
{
	static FRelativeTransformInterfaceRegistry Singleton;
	return Singleton;
}
	
IRelativeTransformInterface* FRelativeTransformInterfaceRegistry::FindRelativeTransformInterface(const TTypedElement<ITypedElementWorldInterface>& InElement) const
{
	if (UObject* ObjectWorldInterface = Cast<UObject>(InElement.GetInterface()))
	{
		return FindRelativeTransformInterface(ObjectWorldInterface->GetClass());
	}
	return nullptr;
}
	
IRelativeTransformInterface* FRelativeTransformInterfaceRegistry::FindRelativeTransformInterface(const UClass* InClass) const
{
	const TUniquePtr<IRelativeTransformInterface>* Interface = WorldInterfaceToRelativeTransformInterface.Find(InClass);
	if (!Interface)
	{
		LogOnce(InClass);		
	}
	return Interface ? Interface->Get() : nullptr;
}

void FRelativeTransformInterfaceRegistry::RegisterDefaultInterfaces()
{
	FRelativeTransformInterfaceRegistry& Registry = Get();
	
	Registry.RegisterRelativeTransformInterface<UActorElementWorldInterface>(MakeUnique<FActorRelativeTransformInterface>());
	Registry.RegisterRelativeTransformInterface<UActorElementEditorWorldInterface>(MakeUnique<FActorRelativeTransformInterface>());
	
	Registry.RegisterRelativeTransformInterface<UComponentElementWorldInterface>(MakeUnique<FComponentRelativeTransformInterface>());
	Registry.RegisterRelativeTransformInterface<UComponentElementEditorWorldInterface>(MakeUnique<FComponentRelativeTransformInterface>());
}

void FRelativeTransformInterfaceRegistry::LogOnce(const UClass* InClass) const
{
	if (InClass && !QueriedUnregisteredClass.Contains(InClass))
	{
		UE_LOGF(LogTemp, Warning, "No relative transform interface found for class %ls. Did you call RegisterRelativeTransformInterface<> for that class?", *InClass->GetName());
		QueriedUnregisteredClass.Add(InClass);
	}
}
	
	

bool GetRelativeTransform(const TTypedElement<ITypedElementWorldInterface>& InElement, FTransform& OutTransform, FRotationContext& InOutRotationContext)
{
	InOutRotationContext.RotationOrder = EEulerRotationOrder::XYZ;
	
	if (InElement.GetWorldTransform(OutTransform))
	{
		if (InOutRotationContext.bUseExplicitRotator)
		{
			bool bComputeRelativeFromTransform = true;

			if (UObject* ObjectWorldInterface = Cast<UObject>(InElement.GetInterface()))
			{
				const FRelativeTransformInterfaceRegistry& Registry = FRelativeTransformInterfaceRegistry::Get();
				if (IRelativeTransformInterface* Interface = Registry.FindRelativeTransformInterface(ObjectWorldInterface->GetClass()))
				{
					FEulerTransform RelativeEulerTransform;
					if (Interface->GetRelativeTransform(InElement, RelativeEulerTransform))
					{
						// explicit rotator value
						InOutRotationContext.Rotation = RelativeEulerTransform.Rotator();
						// parent's world 
						InOutRotationContext.Offset = RelativeEulerTransform.ToFTransform().Inverse() * OutTransform;
						
						bComputeRelativeFromTransform = false;
					}
				}
			}

			if (bComputeRelativeFromTransform)
			{
				// relative rotation
				FTransform RelativeTransform;
				if (InElement.GetRelativeTransform(RelativeTransform))
				{
					// rotator from FQuat
					InOutRotationContext.Rotation = RelativeTransform.Rotator();
					// parent's world 
					InOutRotationContext.Offset = RelativeTransform.Inverse() * OutTransform; 
				}
			}
			return true;
		}
	
		// parent space only, leave the rotation context as it is
		FTransform RelativeTransform;
		if (InElement.GetRelativeTransform(RelativeTransform))
		{
			const FTransform ParentWorld = RelativeTransform.Inverse() * OutTransform;
			OutTransform.SetRotation(ParentWorld.GetRotation());
		}
		return true;
	}
	
	return false;
}
	
void GetComponentRelativeTransform(const USceneComponent* InSceneComponent, FEulerTransform& OutRelativeTransform)
{
	OutRelativeTransform.Location = InSceneComponent->GetRelativeLocation();
	OutRelativeTransform.Rotation = InSceneComponent->GetRelativeRotation();
	OutRelativeTransform.Scale = InSceneComponent->GetRelativeScale3D();
}

void SetComponentRelativeTransform(USceneComponent* InSceneComponent, const FEulerTransform& InRelativeTransform)
{
	InSceneComponent->SetRelativeLocation_Direct(InRelativeTransform.GetLocation());
	InSceneComponent->SetRelativeRotationExact(InRelativeTransform.Rotator());
	InSceneComponent->SetRelativeScale3D_Direct(InRelativeTransform.GetScale3D());
}

bool FActorRelativeTransformInterface::GetRelativeTransform(const FTypedElementHandle& InElementHandle, FEulerTransform& OutRelativeTransform)
{
	if (USceneComponent* SceneComponent = GetSceneComponent(InElementHandle))
	{
		GetComponentRelativeTransform(SceneComponent, OutRelativeTransform);
		return true;	
	}
	
	return false;
}
	
bool FActorRelativeTransformInterface::SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FEulerTransform& InRelativeTransform)
{
	if (USceneComponent* SceneComponent = GetSceneComponent(InElementHandle))
	{
		SetComponentRelativeTransform(SceneComponent, InRelativeTransform);
		return true;	
	}
	
	return false;
}

USceneComponent* FActorRelativeTransformInterface::GetSceneComponent(const FTypedElementHandle& InElementHandle)
{
	const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
	return Actor ? Actor->GetRootComponent() : nullptr;
}
	
bool FComponentRelativeTransformInterface::GetRelativeTransform(const FTypedElementHandle& InElementHandle, FEulerTransform& OutRelativeTransform)
{
	if (const USceneComponent* SceneComponent = GetSceneComponent(InElementHandle))
	{
		OutRelativeTransform.Location = SceneComponent->GetRelativeLocation();
    	OutRelativeTransform.Rotation = SceneComponent->GetRelativeRotation();
    	OutRelativeTransform.Scale = SceneComponent->GetRelativeScale3D();
		return true;
	}

	return false;
}

bool FComponentRelativeTransformInterface::SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FEulerTransform& InRelativeTransform)
{
	if (USceneComponent* SceneComponent = GetSceneComponent(InElementHandle))
	{
		SetComponentRelativeTransform(SceneComponent, InRelativeTransform);
		return true;	
	}
	
	return false;
}

USceneComponent* FComponentRelativeTransformInterface::GetSceneComponent(const FTypedElementHandle& InElementHandle)
{
	UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);
	return Component ? Cast<USceneComponent>(Component) : nullptr;
}
	
}
