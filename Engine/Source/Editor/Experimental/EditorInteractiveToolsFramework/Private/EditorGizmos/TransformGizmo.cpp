// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/TransformGizmo.h"

#include "AnimationCoreLibrary.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/AxisSources.h"
#include "BaseGizmos/GizmoElementGroup.h"
#include "BaseGizmos/GizmoElementShapes.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/StateTargets.h"
#include "EditorGizmos/GizmoElementGimbal.h"
#include "EditorGizmos/GizmoRotationUtil.h"
#include "Engine/CollisionProfile.h"
#include "Engine/World.h"
#include "Materials/Material.h"
#include "Misc/AxisDisplayInfo.h"
#include "UnrealEngine.h"
#include "Behaviors/MultiButtonClickDragBehavior.h"
#include "Intersection/IntersectionUtil.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "ToolDataVisualizer.h"
#include "TransformGizmoEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformGizmo)

#define LOCTEXT_NAMESPACE "UTransformGizmo"

DEFINE_LOG_CATEGORY_STATIC(LogTransformGizmo, Log, All);

namespace GizmoLocals
{
	
// NOTE these variables are not intended to remain here indefinitely.
// Their purpose is to experiment the new behavior of rotation gizmos.

static float DotThreshold = 0.2f;
static FAutoConsoleVariableRef CVarDotThreshold(
	TEXT("Gizmos.DotThreshold"),
	DotThreshold,
	TEXT("Dot threshold for determining whether the rotation plane is perpendicular to the camera view [0.2, 1.0]"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
	{
		DotThreshold = FMath::Clamp(DotThreshold, 0.2f, 1.0f);
	})
);

static bool	bDebugDraw = false;
static FAutoConsoleVariableRef CVarDebugDraw(
	TEXT("Gizmos.DebugDraw"),
	bDebugDraw,
	TEXT("Displays debugging information.")
	);

static bool	ProjectIndirect = true;
static FAutoConsoleVariableRef CVarProjectIndirect(
	TEXT("Gizmos.ProjectIndirect"),
	ProjectIndirect,
	TEXT("Project to the nearest point of the curve when handling indirect rotation.")
	);
}

namespace UE::EditorInteractiveToolsFramework::Private
{
	namespace TransformGizmoLocals
	{
		template <EAxis::Type Axis>
		static constexpr FVector GetAxisVector()
		{
			return FVector::ZeroVector;
		}

		template <>
		constexpr FVector GetAxisVector<EAxis::X>()
		{
			return FVector::XAxisVector;
		}

		template <>
		FVector GetAxisVector<EAxis::Y>()
		{
			return AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward
				? -FVector::YAxisVector
				: FVector::YAxisVector;
		}

		template <>
		constexpr FVector GetAxisVector<EAxis::Z>()
		{
			return FVector::ZAxisVector;
		}

		static FVector GetAxisVector(EAxis::Type InAxis)
		{
			switch (InAxis)
			{
			case EAxis::X:
				return GetAxisVector<EAxis::X>();

			case EAxis::Y:
				return GetAxisVector<EAxis::Y>();

			case EAxis::Z:
				return GetAxisVector<EAxis::Z>();

			case EAxis::None:
			default:
				return FVector::ZeroVector;
			}
		}

		template <EAxis::Type Axis>
		static constexpr FVector GetAxisSideVector()
		{
			return FVector::ZeroVector;
		}

		template <>
		constexpr FVector GetAxisSideVector<EAxis::X>()
		{
			return FVector::YAxisVector;
		}

		template <>
		constexpr FVector GetAxisSideVector<EAxis::Y>()
		{
			return FVector::ZAxisVector;
		}

		template <>
		constexpr FVector GetAxisSideVector<EAxis::Z>()
		{
			return FVector::XAxisVector;
		}

		static FVector GetAxisSideVector(EAxis::Type InAxis)
		{
			switch (InAxis)
			{
			case EAxis::X:
				return GetAxisSideVector<EAxis::X>();

			case EAxis::Y:
				return GetAxisSideVector<EAxis::Y>();

			case EAxis::Z:
				return GetAxisSideVector<EAxis::Z>();

			case EAxis::None:
			default:
				return FVector::ZeroVector;
			}
		}
	}
};

FTransformGizmoStyle::FTransformGizmoStyle()
{
}

void FTransformGizmoStyle::MakeValid()
{
	constexpr float AxisSizeMultiplierMin = 0.1f;
	AxisSizeMultiplier = FMath::Max(AxisSizeMultiplierMin, AxisSizeMultiplier);
}

float FTransformGizmoStyle::GetModifiedAxisSizeMultiplier(const TOptional<TUniqueFunction<float(const float)>>& InModifierOverride) const
{
	if (InModifierOverride.IsSet())
	{
		return InModifierOverride.GetValue()(AxisSizeMultiplier);
	}

	return GetDefault<UTransformGizmoEditorSettings>()->GetTransformGizmoAxisMultiplier();
}

bool FTransformGizmoInteraction::IsCoordinateSystemEnabled(const ECoordSystem InCoordinateSystem) const
{
	return (AdditionalCoordinateSystems & (1 << InCoordinateSystem)) != 0;
}

void FTransformGizmoInteraction::SetCoordinateSystemEnabled(const ECoordSystem InCoordinateSystem, const bool bInIsEnabled)
{
	if (bInIsEnabled)
	{
		AdditionalCoordinateSystems |= (1 << InCoordinateSystem);
	}
	else
	{
		AdditionalCoordinateSystems &= ~(1 << InCoordinateSystem);
	}
}

void UTransformGizmo::SetDisallowNegativeScaling(bool bDisallow)
{
	bDisallowNegativeScaling = bDisallow;
}

void UTransformGizmo::Setup()
{
	if (IsValid(GizmoElementRoot))
	{
		return;
	}
	
	UInteractiveGizmo::Setup();

	SetupBehaviors();
	SetupIndirectBehaviors();
	SetupMaterials();
	SetupOnClickFunctions();

	// @todo: Gizmo element construction will be moved to the UEditorTransformGizmoBuilder to decouple
	// the rendered elements from the transform gizmo.
	GizmoElementRoot = NewObject<UGizmoElementGroup>();
	GizmoElementRoot->SetConstantScale(true);
	GizmoElementRoot->SetHoverMaterial(CurrentAxisMaterial);
	GizmoElementRoot->SetInteractMaterial(CurrentAxisMaterial);
	GizmoElementRoot->SetHoverLineColor(Style.CurrentColor);
	GizmoElementRoot->SetInteractLineColor(Style.CurrentColor);

	// the main gimbal rotation element that manages the three gimbal rotation axis as a group
	RotateGimbalElement = NewObject<UGizmoElementGimbal>();
	RotateGimbalElement->SetHoverMaterial(CurrentAxisMaterial);
	RotateGimbalElement->SetInteractMaterial(CurrentAxisMaterial);
	RotateGimbalElement->SetHoverLineColor(Style.CurrentColor);
	RotateGimbalElement->SetInteractLineColor(Style.CurrentColor);
	GizmoElementRoot->Add(RotateGimbalElement);

	bInInteraction = false;

	SetModeLastHitPart(EGizmoTransformMode::None, ETransformGizmoPartIdentifier::Default);
	SetModeLastHitPart(EGizmoTransformMode::Translate, ETransformGizmoPartIdentifier::TranslateScreenSpace);
	SetModeLastHitPart(EGizmoTransformMode::Rotate, ETransformGizmoPartIdentifier::RotateArcball);
	SetModeLastHitPart(EGizmoTransformMode::Scale, ETransformGizmoPartIdentifier::ScaleUniform);
}

void UTransformGizmo::SetupBehaviors()
{
	// Add default mouse hover behavior
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(HoverBehavior);

	// Add default mouse input behavior
	UClickDragInputBehavior* MouseBehavior = NewObject<UClickDragInputBehavior>();
	MouseBehavior->Initialize(this);
	MouseBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(MouseBehavior);
}

void UTransformGizmo::SetupIndirectBehaviors()
{
	static constexpr int INDIRECT_GIZMO_PRIORITY = FInputCapturePriority::DEFAULT_GIZMO_PRIORITY-1;
	
	// Add middle mouse input behavior for indirect manipulation
	ULocalClickDragInputBehavior* MiddleClickDragBehavior = NewObject<ULocalClickDragInputBehavior>();
	MiddleClickDragBehavior->Initialize();
	MiddleClickDragBehavior->SetDefaultPriority(INDIRECT_GIZMO_PRIORITY);
	MiddleClickDragBehavior->SetUseMiddleMouseButton();
	MiddleClickDragBehavior->CanBeginClickDragFunc = [this](const FInputDeviceRay& PressPos)
	{
		if (OnPreCanInteractDelegate.IsBound())
		{
			FGizmoInteractionDescription Desc({.Ray=PressPos, .bIndirect=true});
			OnPreCanInteractDelegate.Broadcast(Desc);
		}
		
		static const FInputRayHit InvalidRayHit;
		static const FInputRayHit ValidRayHit(TNumericLimits<double>::Max());
		return CanInteract() ? ValidRayHit : InvalidRayHit;
	};
	MiddleClickDragBehavior->OnClickPressFunc = [this](const FInputDeviceRay& InPressPos)
	{
		bIndirectManipulation = true;
		if (LastHitPart != ETransformGizmoPartIdentifier::Default && LastHitPart != GetCurrentModeLastHitPart())
		{
			UpdateInteractingState(false, LastHitPart, true);
		}
		LastHitPart = GetCurrentModeLastHitPart();
		return OnClickPress(InPressPos);
	};
	MiddleClickDragBehavior->OnClickDragFunc = [this](const FInputDeviceRay& InDragPos)
	{
		bIndirectManipulation = true;
		return OnClickDrag(InDragPos);
	};
	MiddleClickDragBehavior->OnClickReleaseFunc = [this](const FInputDeviceRay& InReleasePos)
	{
		bIndirectManipulation = false;
		return OnClickRelease(InReleasePos);
	};
	MiddleClickDragBehavior->OnTerminateFunc = [this]()
	{
		bIndirectManipulation = false;
		return OnTerminateDragSequence();
	};
	// disable ctrl + mmb for that behavior?
	MiddleClickDragBehavior->ModifierCheckFunc = [this](const FInputDeviceState& InputState)
	{
		if (FInputDeviceState::IsAltKeyDown(InputState))
		{
			// Do not capture alt-clicks so that they can be handled by camera movements
			return false;
		}
		return !bSequentialIndirectAxesButtons || !FInputDeviceState::IsCtrlKeyDown(InputState);
	};
	AddInputBehavior(MiddleClickDragBehavior);

	// Add left/right mouse input behavior for indirect manipulation
	MultiIndirectClickDragBehavior = NewObject<UMultiButtonClickDragBehavior>();
	MultiIndirectClickDragBehavior->Initialize();
	MultiIndirectClickDragBehavior->SetDefaultPriority(INDIRECT_GIZMO_PRIORITY);
	MultiIndirectClickDragBehavior->EnableButton(EKeys::LeftMouseButton);
	if (bSequentialIndirectAxesButtons)
	{
		MultiIndirectClickDragBehavior->EnableButton(EKeys::MiddleMouseButton);
	}
	MultiIndirectClickDragBehavior->EnableButton(EKeys::RightMouseButton);
	MultiIndirectClickDragBehavior->ModifierCheckFunc = FInputDeviceState::IsCtrlKeyDown;
	MultiIndirectClickDragBehavior->CanBeginClickDragFunc = [this](const FInputDeviceRay& PressPos)
	{
		if (OnPreCanInteractDelegate.IsBound())
		{
			FGizmoInteractionDescription Desc({.Ray=PressPos, .bIndirect=true});
			OnPreCanInteractDelegate.Broadcast(Desc);
		}
		
		static const FInputRayHit InvalidRayHit;
		static const FInputRayHit ValidRayHit(TNumericLimits<double>::Max());
		return CanInteract() ? ValidRayHit : InvalidRayHit;
	};
	MultiIndirectClickDragBehavior->OnClickPressFunc = [this](const FInputDeviceRay& InPressPos)
	{
		bIndirectManipulation = true;
		if (LastHitPart == ETransformGizmoPartIdentifier::Default)
		{
			LastHitPart = GetCurrentModeLastHitPart();
		}
		return OnClickPress(InPressPos);
	};
	MultiIndirectClickDragBehavior->OnClickDragFunc = [this](const FInputDeviceRay& InDragPos)
	{
		bIndirectManipulation = true;
		return OnClickDrag(InDragPos);
	};
	MultiIndirectClickDragBehavior->OnClickReleaseFunc = [this](const FInputDeviceRay& InReleasePos)
	{
		bIndirectManipulation = false;
		return OnClickRelease(InReleasePos);
	};
	MultiIndirectClickDragBehavior->OnTerminateFunc = [this]()
	{
		bIndirectManipulation = false;
		return OnTerminateDragSequence();
	};
	
	auto GetAxis = [this](const FInputDeviceState& InInput)
	{
		const bool bAddX = InInput.Mouse.Left.bDown;
		if (bSequentialIndirectAxesButtons)
		{
			const bool bAddY = InInput.Mouse.Middle.bDown;
			const bool bAddZ = InInput.Mouse.Right.bDown;
			return bAddX ? EAxis::X : bAddY ? EAxis::Y : bAddZ ? EAxis::Z : EAxis::None;
		}
		const bool bAddY = InInput.Mouse.Right.bDown;
		return InInput.Mouse.Middle.bDown || (bAddX && bAddY) ? EAxis::Z : bAddX ? EAxis::X : bAddY ? EAxis::Y : EAxis::None;
	};

	auto GetHitPart = [this](const EAxis::Type InAxis)
	{
		static constexpr ETransformGizmoPartIdentifier TranslateIds [4] = {
			ETransformGizmoPartIdentifier::Default,
			ETransformGizmoPartIdentifier::TranslateXAxis,
			ETransformGizmoPartIdentifier::TranslateYAxis,
			ETransformGizmoPartIdentifier::TranslateZAxis};
			
		static constexpr ETransformGizmoPartIdentifier RotateIds [4] = {
			ETransformGizmoPartIdentifier::Default,
			ETransformGizmoPartIdentifier::RotateXAxis,
			ETransformGizmoPartIdentifier::RotateYAxis,
			ETransformGizmoPartIdentifier::RotateZAxis};

		static constexpr ETransformGizmoPartIdentifier ScaleIds [4] = {
			ETransformGizmoPartIdentifier::Default,
			ETransformGizmoPartIdentifier::ScaleXAxis,
			ETransformGizmoPartIdentifier::ScaleYAxis,
			ETransformGizmoPartIdentifier::ScaleZAxis};

		static constexpr ETransformGizmoPartIdentifier GimbalIds [4] = {
			ETransformGizmoPartIdentifier::Default,
			ETransformGizmoPartIdentifier::RotateXGimbal,
			ETransformGizmoPartIdentifier::RotateYGimbal,
			ETransformGizmoPartIdentifier::RotateZGimbal};

		switch (CurrentMode)
		{
		case EGizmoTransformMode::Translate: return TranslateIds[InAxis];
		case EGizmoTransformMode::Rotate: return bGimbalRotationMode ? GimbalIds[InAxis] : RotateIds[InAxis];
		case EGizmoTransformMode::Scale: return ScaleIds[InAxis];
		default: break;
		}
		return ETransformGizmoPartIdentifier::Default;
	};
	
	MultiIndirectClickDragBehavior->OnStateUpdated = [this, GetAxis, GetHitPart](const FInputDeviceState& Input)
	{
		// disable indirect if the current axis is none 
		const EAxis::Type Axis = GetAxis(Input);
		if (Axis == EAxis::None)
		{
			bIndirectManipulation = false;
			return;
		}

		bIndirectManipulation = true;
		
		const ETransformGizmoPartIdentifier HitPart = GetHitPart(Axis);
		if (HitPart != GetCurrentModeLastHitPart())
		{
			// update interaction state
			UpdateInteractingState(false, GetCurrentModeLastHitPart(), true);
			SetModeLastHitPart(CurrentMode, HitPart);
			UpdateInteractingState(true, HitPart, true);

			// reinitialize OnClickPress data
			LastHitPart = HitPart;
			const uint8 HitPartIndex = static_cast<uint8>(LastHitPart);
			if (OnClickPressFunctions.IsValidIndex(HitPartIndex) && OnClickPressFunctions[HitPartIndex])
			{
				OnClickPressFunctions[HitPartIndex](this, MultiIndirectClickDragBehavior->GetDeviceRay(Input));
			}
		}
	};
	AddInputBehavior(MultiIndirectClickDragBehavior);
}

void UTransformGizmo::SetupMaterials()
{
	auto GetBaseMaterial = [this]() -> UMaterial*
	{
		if (CustomizationFunction)
		{
			const FGizmoCustomization& GizmoCustomization = CustomizationFunction();
			if (IsValid(GizmoCustomization.Material))
			{
				return GizmoCustomization.Material.Get();
			}
		}

		static const FString MaterialName = TEXT("/Engine/EditorMaterials/TransformGizmoMaterial_PseudoLitMasked.TransformGizmoMaterial_PseudoLitMasked");
		UMaterial* Material = FindObject<UMaterial>(nullptr, *MaterialName);
		if (!Material)
		{
			Material = LoadObject<UMaterial>(nullptr, *MaterialName, nullptr, LOAD_None, nullptr);
		}
		
		return Material ? Material : GEngine->ArrowMaterial.Get();
	};
	
	UMaterial* AxisMaterialBase = GetBaseMaterial(); 

	AxisMaterialX = UMaterialInstanceDynamic::Create(AxisMaterialBase, NULL);
	AxisMaterialX->SetVectorParameterValue("GizmoColor", AxisDisplayInfo::GetAxisColor(EAxisList::X));

	AxisMaterialY = UMaterialInstanceDynamic::Create(AxisMaterialBase, NULL);
	AxisMaterialY->SetVectorParameterValue("GizmoColor", AxisDisplayInfo::GetAxisColor(EAxisList::Y));

	AxisMaterialZ = UMaterialInstanceDynamic::Create(AxisMaterialBase, NULL);
	AxisMaterialZ->SetVectorParameterValue("GizmoColor", AxisDisplayInfo::GetAxisColor(EAxisList::Z));

	GreyMaterial = UMaterialInstanceDynamic::Create(AxisMaterialBase, NULL);
	GreyMaterial->SetVectorParameterValue("GizmoColor", Style.GreyColor);

	WhiteMaterial = UMaterialInstanceDynamic::Create(AxisMaterialBase, NULL);
	WhiteMaterial->SetVectorParameterValue("GizmoColor", Style.WhiteColor);

	CurrentAxisMaterial = UMaterialInstanceDynamic::Create(AxisMaterialBase, NULL);
	CurrentAxisMaterial->SetVectorParameterValue("GizmoColor", Style.CurrentColor);

	OpaquePlaneMaterialXY = UMaterialInstanceDynamic::Create(AxisMaterialBase, NULL);
	OpaquePlaneMaterialXY->SetVectorParameterValue("GizmoColor", FLinearColor::White);

	TransparentVertexColorMaterial = (UMaterial*)StaticLoadObject(
		UMaterial::StaticClass(), NULL,
		TEXT("/Engine/EditorMaterials/WidgetVertexColorMaterial.WidgetVertexColorMaterial"), NULL, LOAD_None, NULL);

	GridMaterial = (UMaterial*)StaticLoadObject(
		UMaterial::StaticClass(), NULL,
		TEXT("/Engine/EditorMaterials/WidgetGridVertexColorMaterial_Ma.WidgetGridVertexColorMaterial_Ma"), NULL,
		LOAD_None, NULL);
	if (!GridMaterial)
	{
		GridMaterial = TransparentVertexColorMaterial;
	}
}

void UTransformGizmo::Shutdown()
{
	ClearActiveTarget();
	OnSetActiveTarget.Clear();
	OnAboutToClearActiveTarget.Clear();
}

FTransform UTransformGizmo::GetGizmoTransform() const
{
	const float Scale = TransformGizmoSource ? TransformGizmoSource->GetGizmoScale() : 1.0f;
	
	auto GetCoordinateSystem = [&]()
	{
		if (TransformGizmoSource)
		{
			return TransformGizmoSource->GetGizmoCoordSystemSpace(); 
		}
		return GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem();
	};
	const bool bLocal = GetCoordinateSystem() == EToolContextCoordinateSystem::Local;
	
	FTransform GizmoLocalToWorldTransform(CurrentTransform.GetTranslation());
	if (bLocal)
	{
		GizmoLocalToWorldTransform.SetRotation(CurrentTransform.GetRotation());
	}
	GizmoLocalToWorldTransform.SetScale3D(FVector(Scale, Scale, Scale));

	return GizmoLocalToWorldTransform;
}

bool UTransformGizmo::CanRender(IToolsContextRenderAPI* InRenderAPI) const
{
	return CanInteract() && GizmoElementRoot && InRenderAPI;
}

void UTransformGizmo::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (CanRender(RenderAPI))
	{
		CurrentTransform = ActiveTarget->GetTransform();

		if (IsVisible())
		{
			const UE::GizmoRenderingUtil::FSceneViewWrapper WrappedSceneView(*RenderAPI->GetSceneView());
		
			UGizmoElementBase::FRenderTraversalState RenderState;
			RenderState.Initialize(WrappedSceneView, GetGizmoTransform(), RenderAPI->GetCameraState().DPIScale);
			GizmoElementRoot->Render(RenderAPI, RenderState);
		}


		// Allow debug rendering regardless of actual gizmo visibility (it can still be interacted with, so we want to visualize it)
		RenderDebug(RenderAPI);
	}
}

void UTransformGizmo::RenderDebug(IToolsContextRenderAPI* RenderAPI)
{
	if (!GizmoLocals::bDebugDraw)
	{
		return;
	}

	DebugData.TransformCurrent = CurrentTransform;

	FToolDataVisualizer Draw;
	Draw.BeginFrame(RenderAPI);

	static constexpr float ScreenSpaceHandleRadius = 15.0f;

	const FVector CameraFacingNormal = -RenderAPI->GetSceneView()->GetViewDirection();

	static const FLinearColor ActorStartColor = FLinearColor::Blue.CopyWithNewOpacity(0.5f);
	static const FLinearColor ActorCurrentColor = FLinearColor(0.0f, 1.0f, 1.0f).CopyWithNewOpacity(0.5f);

	static const FLinearColor InteractionStartColor = FLinearColor::Red.CopyWithNewOpacity(0.5f);
	static const FLinearColor InteractionCurrentColor = FLinearColor::Yellow.CopyWithNewOpacity(0.5f);

	// From initial cursor pos to current for drag operation
	if (DebugData.bIsEditing)
	{
		// Actual Object
		Draw.DrawCircle(DebugData.TransformStart.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, ActorStartColor, 0.0f, false);
		Draw.DrawCircle(DebugData.TransformCurrent.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, ActorCurrentColor, 0.0f, false);
		Draw.DrawLine(DebugData.TransformStart.GetLocation(), DebugData.TransformCurrent.GetLocation(), ActorCurrentColor, 0.0f, false);

		// Rotate mode displays as arcs, etc.
		if (CurrentMode == EGizmoTransformMode::Rotate)
		{
			const float InteractionRadius = static_cast<float>(DebugData.InteractionRadius);
			float ArcAngleMin = static_cast<float>(FMath::RadiansToDegrees(DebugData.InteractionAngleStart));
			float ArcAngleMax = static_cast<float>(FMath::RadiansToDegrees(DebugData.InteractionAngleCurrent));

			FVector StartNormal = (DebugData.InteractionStart.GetLocation() - DebugData.TransformStart.GetLocation()).GetSafeNormal();
			FVector InteractionStartOvershoot = DebugData.TransformStart.GetLocation() + (StartNormal * InteractionRadius);

			// Cursor
			Draw.DrawCircle(DebugData.InteractionStart.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionStartColor, 0.0f, false);
			Draw.DrawCircle(InteractionStartOvershoot, CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionStartColor, 0.0f, false);
			Draw.DrawCircle(DebugData.InteractionCurrent.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionCurrentColor, 0.0f, false);

			Draw.DrawCircle(DebugData.TransformStart.GetLocation(), DebugData.InteractionPlaneNormal, static_cast<float>(DebugData.InteractionRadius), 128, InteractionCurrentColor, 0.0f, false);

			// @todo: min/max angles should be relative (to InteractionPlanarNormal?), also doesn't seem to be centered
			// Draw.DrawArc(DebugData.TransformStart.GetLocation(), InteractionPlanarNormal, static_cast<float>(DebugData.InteractionRadius * 1.1f), 128, ArcAngleMin, ArcAngleMax, FLinearColor::Green, 0.0f, false);

			Draw.DrawLine(DebugData.TransformStart.GetLocation(), InteractionStartOvershoot, InteractionStartColor.CopyWithNewOpacity(0.5f), 0.0f, false);
			Draw.DrawLine(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation(), InteractionCurrentColor, 0.0f, false);
		}
		else
		{
			// Cursor
			Draw.DrawCircle(DebugData.InteractionStart.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionStartColor, 0.0f, false);
			Draw.DrawCircle(DebugData.InteractionCurrent.GetLocation(), CameraFacingNormal, ScreenSpaceHandleRadius, 16, InteractionCurrentColor, 0.0f, false);
			Draw.DrawLine(DebugData.InteractionStart.GetLocation(), DebugData.InteractionCurrent.GetLocation(), InteractionCurrentColor, 0.0f, false);
		}
	}

	Draw.EndFrame();
}

FInputRayHit UTransformGizmo::BeginHoverSequenceHitTest(const FInputDeviceRay& DevicePos)
{
	return UpdateHoveredPart(DevicePos);
}

void UTransformGizmo::OnBeginHover(const FInputDeviceRay& DevicePos)
{
}

bool UTransformGizmo::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	const FInputRayHit RayHit = UpdateHoveredPart(DevicePos);
	return RayHit.bHit;
}

void UTransformGizmo::OnEndHover()
{
	if (HitTarget)
	{
		if (LastHitPart != ETransformGizmoPartIdentifier::Default)
		{
			UpdateHoverState(false, LastHitPart);
		}

		const ETransformGizmoPartIdentifier ModeHitPart = GetCurrentModeLastHitPart();
		if (ModeHitPart != ETransformGizmoPartIdentifier::Default)
		{
			UpdateInteractingState(true, ModeHitPart, true);
		}
	}
}

FInputRayHit UTransformGizmo::UpdateHoveredPart(const FInputDeviceRay& PressPos)
{
	if (!HitTarget || !IsVisible(EViewportContext::Hovered))
	{
		return FInputRayHit();
	}

	FInputRayHit RayHit = HitTarget->IsHit(PressPos);

	ETransformGizmoPartIdentifier HitPart;
	if (RayHit.bHit && VerifyPartIdentifier(RayHit.HitIdentifier))
	{
		HitPart = static_cast<ETransformGizmoPartIdentifier>(RayHit.HitIdentifier);
	}
	else
	{
		HitPart = ETransformGizmoPartIdentifier::Default;
	}

	if (HitPart != LastHitPart)
	{
		if (LastHitPart != ETransformGizmoPartIdentifier::Default)
		{
			UpdateHoverState(false, LastHitPart);
		}

		if (HitPart != ETransformGizmoPartIdentifier::Default)
		{
			UpdateHoverState(true, HitPart);
		}

		LastHitPart = HitPart;
	}

	const ETransformGizmoPartIdentifier ModeHitPart = GetCurrentModeLastHitPart();
	if (ModeHitPart != ETransformGizmoPartIdentifier::Default)
	{
		UpdateInteractingState(true, ModeHitPart, true);
	}

	return RayHit;
}

uint32 UTransformGizmo::GetMaxPartIdentifier() const
{
	return static_cast<uint32>(ETransformGizmoPartIdentifier::Max);
}

bool UTransformGizmo::VerifyPartIdentifier(uint32 InPartIdentifier) const
{
	if (InPartIdentifier >= GetMaxPartIdentifier())
	{
		UE_LOGF(LogTransformGizmo, Warning, "Unrecognized transform gizmo part identifier %d, valid identifiers are between 0-%d.", 
			InPartIdentifier, GetMaxPartIdentifier());
		return false;
	}

	return true;
}

void UTransformGizmo::SetModeLastHitPart(const EGizmoTransformMode InMode, const ETransformGizmoPartIdentifier InIdentifier)
{
	if (InMode >= EGizmoTransformMode::None && InMode < EGizmoTransformMode::Max)
	{
		LastHitPartPerMode[static_cast<uint8>(InMode)] = InIdentifier;
	}
}

EAxisList::Type UTransformGizmo::GetInteractionAxisList() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return InteractionAxisList;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UTransformGizmo::SetInteractionAxisList(const EAxisList::Type InAxisList)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InteractionAxisList = InAxisList;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

ETransformGizmoPartIdentifier UTransformGizmo::GetCurrentModeLastHitPart() const
{
	auto GetTransformMode = [&]()
	{
		if (TransformGizmoSource)
		{
			return TransformGizmoSource->GetGizmoMode();
		}
		
		const EToolContextTransformGizmoMode ActiveGizmoMode = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentTransformGizmoMode();
		switch (ActiveGizmoMode)
		{
		case EToolContextTransformGizmoMode::NoGizmo:
			return EGizmoTransformMode::None;
		case EToolContextTransformGizmoMode::Translation:
			return EGizmoTransformMode::Translate;
		case EToolContextTransformGizmoMode::Rotation:
			return EGizmoTransformMode::Rotate;
		case EToolContextTransformGizmoMode::Scale:
			return EGizmoTransformMode::Scale;
		case EToolContextTransformGizmoMode::Combined:
			return EGizmoTransformMode::None;
		}
		
		return EGizmoTransformMode::None;
	};

	const EGizmoTransformMode GizmoTransformMode = GetTransformMode();
	return (GizmoTransformMode < EGizmoTransformMode::Max) ?
		LastHitPartPerMode[static_cast<uint8>(GizmoTransformMode)] :
		ETransformGizmoPartIdentifier::Default; 
}

FInputRayHit UTransformGizmo::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FInputRayHit RayHit;
		
	if (IsVisible(EViewportContext::Hovered) && CanInteract() && HitTarget)
	{
		RayHit = HitTarget->IsHit(PressPos);
		ETransformGizmoPartIdentifier HitPart;
		if (RayHit.bHit && VerifyPartIdentifier(RayHit.HitIdentifier))
		{
			HitPart = static_cast<ETransformGizmoPartIdentifier>(RayHit.HitIdentifier);
		}
		else
		{
			HitPart = ETransformGizmoPartIdentifier::Default;
		}

		if (HitPart != ETransformGizmoPartIdentifier::Default)
		{
			LastHitPart = static_cast<ETransformGizmoPartIdentifier>(RayHit.HitIdentifier);
		}
	}

	return RayHit;
}

void UTransformGizmo::UpdateMode()
{
	auto GetTransformMode = [&]()
	{
		if (TransformGizmoSource)
		{
			return TransformGizmoSource->GetGizmoMode();
		}
		
		const EToolContextTransformGizmoMode ActiveGizmoMode = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentTransformGizmoMode();
		switch (ActiveGizmoMode)
		{
			case EToolContextTransformGizmoMode::Translation: return EGizmoTransformMode::Translate;
			case EToolContextTransformGizmoMode::Rotation: return EGizmoTransformMode::Rotate;
			case EToolContextTransformGizmoMode::Scale: return EGizmoTransformMode::Scale;
		}
		return EGizmoTransformMode::None;
	};

	auto GetAxisToDraw = [&]()
	{
		if (TransformGizmoSource)
		{
			return TransformGizmoSource->GetGizmoAxisToDraw(TransformGizmoSource->GetGizmoMode());
		}
		return EAxisList::Type::All;
	};

	const EGizmoTransformMode NewMode = GetTransformMode();
	const EAxisList::Type NewAxisToDraw = GetAxisToDraw();
	
	if (NewMode != CurrentMode)
	{
		EnableMode(CurrentMode, EAxisList::None);
		EnableMode(NewMode, NewAxisToDraw);

		CurrentMode = NewMode;
		CurrentAxisToDraw = NewAxisToDraw;
	}
	else if (NewAxisToDraw != CurrentAxisToDraw)
	{
		EnableMode(CurrentMode, NewAxisToDraw);
		CurrentAxisToDraw = NewAxisToDraw;
	}
	else if (CurrentMode == EGizmoTransformMode::Rotate && bGimbalRotationMode != GetRotationContext().bUseExplicitRotator)
	{
		UpdateRotationMode();
	}
}

void UTransformGizmo::UpdateRotationMode()
{
	EnableMode(EGizmoTransformMode::Rotate, CurrentAxisToDraw);

	if (GetCurrentModeLastHitPart() != ETransformGizmoPartIdentifier::Default)
	{
		const ETransformGizmoPartIdentifier PreviousHitPart = GetCurrentModeLastHitPart();
		UpdateInteractingState(false, PreviousHitPart, true);
		
		ETransformGizmoPartIdentifier NewHitPart = ETransformGizmoPartIdentifier::RotateArcball;

		constexpr uint8 RotateId = static_cast<uint8>(EGizmoTransformMode::Rotate);
		if (bGimbalRotationMode)
		{
			switch (LastHitPartPerMode[RotateId])
			{
			case ETransformGizmoPartIdentifier::RotateXAxis:
				NewHitPart = ETransformGizmoPartIdentifier::RotateXGimbal;
				break;
			case ETransformGizmoPartIdentifier::RotateYAxis:
				NewHitPart = ETransformGizmoPartIdentifier::RotateYGimbal;
				break;
			case ETransformGizmoPartIdentifier::RotateZAxis:
				NewHitPart = ETransformGizmoPartIdentifier::RotateZGimbal;
				break;
			case ETransformGizmoPartIdentifier::RotateScreenSpace:
				NewHitPart = ETransformGizmoPartIdentifier::RotateArcball;
				break;
			default:
				break;
			}
		}
		else
		{
			switch (LastHitPartPerMode[RotateId])
			{
			case ETransformGizmoPartIdentifier::RotateXGimbal:
				NewHitPart = ETransformGizmoPartIdentifier::RotateXAxis;
				break;
			case ETransformGizmoPartIdentifier::RotateYGimbal:
				NewHitPart = ETransformGizmoPartIdentifier::RotateYAxis;
				break;
			case ETransformGizmoPartIdentifier::RotateZGimbal:
				NewHitPart = ETransformGizmoPartIdentifier::RotateZAxis;
				break;
			default:
				break;
			}
		}

		SetModeLastHitPart(CurrentMode, NewHitPart);

		static constexpr bool bInInteracting = true, bIdOnly = true;
		UpdateInteractingState(bInInteracting, NewHitPart, bIdOnly);
	}
}

void UTransformGizmo::EnableMode(EGizmoTransformMode InMode, EAxisList::Type InAxisListToDraw)
{
	if (InMode == EGizmoTransformMode::Translate)
	{
		EnableTranslate(InAxisListToDraw);
	}
	else if (InMode == EGizmoTransformMode::Rotate)
	{
		EnableRotate(InAxisListToDraw);
	}
	else if (InMode == EGizmoTransformMode::Scale)
	{
		EnableScale(InAxisListToDraw);
	}
}

void UTransformGizmo::EnableTranslate(EAxisList::Type InAxisListToDraw)
{
	check(GizmoElementRoot);

	const bool bEnableX = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::X);
	const bool bEnableY = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Y);
	const bool bEnableZ = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Z);
	const bool bEnableAny = bEnableX || bEnableY || bEnableZ;

	if (bEnableX && TranslateXAxisElement == nullptr)
	{
		TranslateXAxisElement = MakeTranslateAxis(ETransformGizmoPartIdentifier::TranslateXAxis, FVector::XAxisVector, FVector::YAxisVector, AxisMaterialX);
		GizmoElementRoot->Add(TranslateXAxisElement);
	}

	if (bEnableY && TranslateYAxisElement == nullptr)
	{
		TranslateYAxisElement = MakeTranslateAxis(ETransformGizmoPartIdentifier::TranslateYAxis, FVector::YAxisVector, FVector::ZAxisVector, AxisMaterialY);
		GizmoElementRoot->Add(TranslateYAxisElement);
	}

	if (bEnableZ && TranslateZAxisElement == nullptr)
	{
		TranslateZAxisElement = MakeTranslateAxis(ETransformGizmoPartIdentifier::TranslateZAxis, FVector::ZAxisVector, FVector::XAxisVector, AxisMaterialZ);
		GizmoElementRoot->Add(TranslateZAxisElement);
	}

	if (bEnableAny && TranslateScreenSpaceElement == nullptr)
	{
		TranslateScreenSpaceElement = MakeTranslateScreenSpaceHandle();
		GizmoElementRoot->Add(TranslateScreenSpaceElement);
	}

	if (TranslateXAxisElement)
	{
		TranslateXAxisElement->SetEnabled(bEnableX);
		TranslateXAxisElement->SetHitMask(TranslateScreenSpaceElement);
	}

	if (TranslateYAxisElement)
	{
		TranslateYAxisElement->SetEnabled(bEnableY);
		TranslateYAxisElement->SetHitMask(TranslateScreenSpaceElement);
	}

	if (TranslateZAxisElement)
	{
		TranslateZAxisElement->SetEnabled(bEnableZ);
		TranslateZAxisElement->SetHitMask(TranslateScreenSpaceElement);
	}

	if (TranslateScreenSpaceElement)
	{
		TranslateScreenSpaceElement->SetEnabled(bEnableAny);
	}

	EnablePlanarObjects(true, bEnableX, bEnableY, bEnableZ);
}

void UTransformGizmo::EnablePlanarObjects(bool bTranslate, bool bEnableX, bool bEnableY, bool bEnableZ)
{
	check(GizmoElementRoot);

	auto EnablePlanarElement = [this](
		TObjectPtr<UGizmoElementBox>& PlanarElement,
		ETransformGizmoPartIdentifier PartId,
		const FVector& Axis0,
		const FVector& Axis1,
		const FVector& Axis2,
		UMaterialInterface* Material,
		bool bEnable)
	{
		if (bEnable && PlanarElement == nullptr)
		{
			PlanarElement = MakePlanarHandle(PartId, Axis0, Axis1, Axis2, Material);
			GizmoElementRoot->Add(PlanarElement);
		}

		if (PlanarElement)
		{
			PlanarElement->SetEnabled(bEnable);
		}
	};

	const bool bEnableXY = bEnableX && bEnableY;
	const bool bEnableYZ = bEnableY && bEnableZ;
	const bool bEnableXZ = bEnableX && bEnableZ;

	const FVector XAxis = FVector::XAxisVector;
	const FVector YAxis = FVector::YAxisVector;
	const FVector ZAxis = FVector::ZAxisVector;

	if (bTranslate)
	{
		EnablePlanarElement(TranslatePlanarXYElement, ETransformGizmoPartIdentifier::TranslateXYPlanar, XAxis, YAxis, ZAxis, AxisMaterialZ, bEnableXY);
		EnablePlanarElement(TranslatePlanarYZElement, ETransformGizmoPartIdentifier::TranslateYZPlanar, YAxis, ZAxis, XAxis, AxisMaterialX, bEnableYZ);
		EnablePlanarElement(TranslatePlanarXZElement, ETransformGizmoPartIdentifier::TranslateXZPlanar, ZAxis, XAxis, YAxis, AxisMaterialY, bEnableXZ);
	}
	else
	{
		EnablePlanarElement(ScalePlanarXYElement, ETransformGizmoPartIdentifier::ScaleXYPlanar, XAxis, YAxis, ZAxis, AxisMaterialZ, bEnableXY);
		EnablePlanarElement(ScalePlanarYZElement, ETransformGizmoPartIdentifier::ScaleYZPlanar, YAxis, ZAxis, XAxis, AxisMaterialX, bEnableYZ);
		EnablePlanarElement(ScalePlanarXZElement, ETransformGizmoPartIdentifier::ScaleXZPlanar, ZAxis, XAxis, YAxis, AxisMaterialY, bEnableXZ);
	}
}

void UTransformGizmo::EnableRotate(EAxisList::Type InAxisListToDraw)
{
	const bool bEnableX = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::X);
	const bool bEnableY = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Y);
	const bool bEnableZ = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Z);
	const bool bEnableAll = bEnableX && bEnableY && bEnableZ;

	bGimbalRotationMode = (bEnableX || bEnableY || bEnableZ) && GetRotationContext().bUseExplicitRotator;

	// default rotation handles
	{
		auto EnableRotateElement = [this](TObjectPtr<UGizmoElementTorus>& InOutElement, const ETransformGizmoPartIdentifier InPartId, const bool bEnableAxis)
		{
			if (bEnableAxis)
			{
				if (!InOutElement)
				{
					InOutElement = MakeDefaultRotateAxis(InPartId);
					GizmoElementRoot->Add(InOutElement);
				}
			}
	
			if (InOutElement)
			{
				InOutElement->SetEnabled(bEnableAxis && !bGimbalRotationMode);
			}
		};

		EnableRotateElement(RotateXAxisElement, ETransformGizmoPartIdentifier::RotateXAxis, bEnableX);
		EnableRotateElement(RotateYAxisElement, ETransformGizmoPartIdentifier::RotateYAxis, bEnableY);
		EnableRotateElement(RotateZAxisElement, ETransformGizmoPartIdentifier::RotateZAxis, bEnableZ);
	}

	// gimbal rotation handles
	{
		auto EnableGimbalRotateElement = [this](TObjectPtr<UGizmoElementTorus>& InOutElement, const ETransformGizmoPartIdentifier InPartId, const bool bEnable)
		{
			if (bEnable)
			{
				if (!InOutElement)
				{
					InOutElement = MakeDefaultRotateAxis(InPartId);
					RotateGimbalElement->Add(InOutElement);
				}
			}
	
			if (InOutElement)
			{
				InOutElement->SetEnabled(bEnable && bGimbalRotationMode);
			}
		};

		EnableGimbalRotateElement(RotateXGimbalElement, ETransformGizmoPartIdentifier::RotateXGimbal, bEnableX);
		EnableGimbalRotateElement(RotateYGimbalElement, ETransformGizmoPartIdentifier::RotateYGimbal, bEnableY);
		EnableGimbalRotateElement(RotateZGimbalElement, ETransformGizmoPartIdentifier::RotateZGimbal, bEnableZ);

		if (RotateGimbalElement)
		{
			RotateGimbalElement->SetEnabled(bEnableAll && bGimbalRotationMode);
		}
	}

	// screen space & arc ball handles
	if (bEnableAll)
	{
		if (!RotateScreenSpaceElement)
		{
			RotateScreenSpaceElement = MakeRotateCircleHandle(
				ETransformGizmoPartIdentifier::RotateScreenSpace,
				Style.RotateCircleRadius + Style.RotateScreenSpaceRadiusOffset,
				Style.RotateScreenSpaceCircleColor,
				EGizmoElementDrawType::Line);
			GizmoElementRoot->Add(RotateScreenSpaceElement);
		}

		if (!RotateArcballElement)
		{
			RotateArcballElement = MakeArcballCircleHandle(ETransformGizmoPartIdentifier::RotateArcball, Style.RotateArcballSphereRadius, Style.RotateArcballCircleColor);
			GizmoElementRoot->Add(RotateArcballElement);
		}
	}
	
	if (RotateScreenSpaceElement)
    {
        RotateScreenSpaceElement->SetEnabled(bEnableAll && !bGimbalRotationMode);
    }
    if (RotateArcballElement)
    { 
        RotateArcballElement->SetEnabled(bEnableAll);
    }
}

void UTransformGizmo::EnableScale(EAxisList::Type InAxisListToDraw)
{
	check(GizmoElementRoot);

	using namespace UE::EditorInteractiveToolsFramework::Private::TransformGizmoLocals;

	const bool bEnableX = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::X);
	const bool bEnableY = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Y);
	const bool bEnableZ = static_cast<uint8>(InAxisListToDraw) & static_cast<uint8>(EAxisList::Z);
	
	if (bEnableX && ScaleXAxisElement == nullptr)
	{
		ScaleXAxisElement = MakeScaleAxis(ETransformGizmoPartIdentifier::ScaleXAxis, FVector::XAxisVector, GetAxisSideVector<EAxis::X>(), AxisMaterialX);
		GizmoElementRoot->Add(ScaleXAxisElement);
	}

	if (bEnableY && ScaleYAxisElement == nullptr)
	{
		ScaleYAxisElement = MakeScaleAxis(ETransformGizmoPartIdentifier::ScaleYAxis, FVector::YAxisVector, GetAxisSideVector<EAxis::Y>(), AxisMaterialY);
		GizmoElementRoot->Add(ScaleYAxisElement);
	}

	if (bEnableZ && ScaleZAxisElement == nullptr)
	{
		ScaleZAxisElement = MakeScaleAxis(ETransformGizmoPartIdentifier::ScaleZAxis, FVector::ZAxisVector, GetAxisSideVector<EAxis::Z>(), AxisMaterialZ);
		GizmoElementRoot->Add(ScaleZAxisElement);
	}

	if ((bEnableX || bEnableY || bEnableZ) && ScaleUniformElement == nullptr)
	{
		ScaleUniformElement = MakeUniformScaleHandle();
		GizmoElementRoot->Add(ScaleUniformElement);
	}

	if (ScaleXAxisElement)
	{
		ScaleXAxisElement->SetEnabled(bEnableX);
	}

	if (ScaleYAxisElement)
	{
		ScaleYAxisElement->SetEnabled(bEnableY);
	}

	if (ScaleZAxisElement)
	{
		ScaleZAxisElement->SetEnabled(bEnableZ);
	}

	if (ScaleUniformElement)
	{
		ScaleUniformElement->SetEnabled(bEnableX || bEnableY || bEnableZ);
	}

	EnablePlanarObjects(false, bEnableX, bEnableY, bEnableZ);
}

void UTransformGizmo::UpdateCameraAxisSource()
{
	FViewCameraState CameraState;
	GetGizmoManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
	if (CameraAxisSource != nullptr)
	{
		CameraAxisSource->Origin = ActiveTarget ? ActiveTarget->GetTransform().GetLocation() : FVector::ZeroVector;
		CameraAxisSource->Direction = -CameraState.Forward();
		CameraAxisSource->TangentX = CameraState.Right();
		CameraAxisSource->TangentY = CameraState.Up();
	}
}

bool UTransformGizmo::IsVisible(const EViewportContext InViewportContext) const
{
	if (TransformGizmoSource)
	{
		return bVisible && TransformGizmoSource->GetVisible(InViewportContext);
	}
	return bVisible;
}

bool UTransformGizmo::CanInteract(const EViewportContext InViewportContext) const
{
	const bool bValidMode = CurrentMode > EGizmoTransformMode::None && CurrentMode < EGizmoTransformMode::Max;
	if (TransformGizmoSource)
	{
		return bValidMode && TransformGizmoSource->CanInteract(InViewportContext);
	}
	return bValidMode && bVisible;
}

const FRotationContext& UTransformGizmo::GetRotationContext() const
{
	static const FRotationContext Context;
	return TransformGizmoSource ? TransformGizmoSource->GetRotationContext() : Context;
}

void UTransformGizmo::Tick(float DeltaTime)
{
	if (PendingDragFunction)
	{
		PendingDragFunction();
		PendingDragFunction.Reset();
	}
	
	UpdateMode();

	UpdateCameraAxisSource();

	// update gimbal handle's rotation context
	if (RotateGimbalElement)
	{
		RotateGimbalElement->RotationContext = GetRotationContext();
	}
}

void UTransformGizmo::SetActiveTarget(UTransformProxy* Target, IToolContextTransactionProvider* TransactionProvider, IGizmoStateTarget* InStateTarget)
{
	if (ActiveTarget != nullptr)
	{
		ClearActiveTarget();
	}

	ActiveTarget = Target;

	// Set current mode to none, mode will be updated next Tick()
	CurrentMode = EGizmoTransformMode::None;

	if (!ActiveTarget)
	{
		return;
	}

	// This state target emits an explicit FChange that moves the GizmoActor root component during undo/redo.
	// It also opens/closes the Transaction that saves/restores the target object locations.
	if (TransactionProvider == nullptr)
	{
		TransactionProvider = GetGizmoManager();
	}

	if (InStateTarget)
	{
		StateTarget.SetInterface(InStateTarget);
		StateTarget.SetObject(CastChecked<UObject>(InStateTarget));
	}
	else
	{
		StateTarget = UGizmoObjectModifyStateTarget::Construct(Target,
			LOCTEXT("UTransformGizmoTransaction", "Transform"), TransactionProvider, this);	
	}

	CameraAxisSource = NewObject<UGizmoConstantFrameAxisSource>(this);

	OnSetActiveTarget.Broadcast(this, ActiveTarget);
}

// @todo: This should either be named to "SetScale" or removed, since it can be done with ReinitializeGizmoTransform
void UTransformGizmo::SetNewChildScale(const FVector& NewChildScale)
{
	FTransform NewTransform = ActiveTarget->GetTransform();
	NewTransform.SetScale3D(NewChildScale);

	TGuardValue GuardValue(ActiveTarget->bSetPivotMode, true);
	ActiveTarget->SetTransform(NewTransform);
}

void UTransformGizmo::SetVisibility(bool bVisibleIn)
{
	bVisible = bVisibleIn;
}

void UTransformGizmo::SetCustomizationFunction(const TFunction<const FGizmoCustomization()>& InFunction)
{
	CustomizationFunction = InFunction;
}

void UTransformGizmo::HandleWidgetModeChanged(UE::Widget::EWidgetMode InWidgetMode)
{
	auto GetTransformMode = [InWidgetMode]()
	{
		switch (InWidgetMode)
		{
		case UE::Widget::EWidgetMode::WM_Translate: return EGizmoTransformMode::Translate;
		case UE::Widget::EWidgetMode::WM_Rotate: return EGizmoTransformMode::Rotate;
		case UE::Widget::EWidgetMode::WM_Scale: return EGizmoTransformMode::Scale;
		default: return EGizmoTransformMode::None;
		}
	};
	const EGizmoTransformMode NewMode = GetTransformMode();

	if (CurrentMode != EGizmoTransformMode::None && NewMode == CurrentMode)
	{
		const ETransformGizmoPartIdentifier CurrentModeLastHitPart = GetCurrentModeLastHitPart();
		auto GetModeDefaultHitPart = [NewMode, CurrentModeLastHitPart, bGimbal = bGimbalRotationMode]()
		{
			const bool bIsRotateArcBall = (CurrentModeLastHitPart == ETransformGizmoPartIdentifier::RotateArcball);
			switch (NewMode)
			{
			case EGizmoTransformMode::Translate:
				return ETransformGizmoPartIdentifier::TranslateScreenSpace;
			case EGizmoTransformMode::Rotate:
				return bGimbal ? ETransformGizmoPartIdentifier::RotateArcball : bIsRotateArcBall ? ETransformGizmoPartIdentifier::RotateScreenSpace : ETransformGizmoPartIdentifier::RotateArcball;
			case EGizmoTransformMode::Scale:
				return ETransformGizmoPartIdentifier::ScaleUniform;
			default:
				return ETransformGizmoPartIdentifier::Default;
			}
		};

		const ETransformGizmoPartIdentifier DefaultHitPart = GetModeDefaultHitPart();
		if (DefaultHitPart != CurrentModeLastHitPart)
		{
			// reset indirect manipulation to default
			ResetInteractingStates(CurrentMode);
			ResetHoverStates(CurrentMode);
			
			SetModeLastHitPart(CurrentMode, DefaultHitPart);
			UpdateInteractingState(true, DefaultHitPart, true);
		}
	}
	
	LastHitPart = ETransformGizmoPartIdentifier::Default;
}

void UTransformGizmo::OnParametersChanged(const FGizmosParameters& InParameters)
{
	if (InParameters.bSequentialIndirectAxesButtons != bSequentialIndirectAxesButtons)
	{
		bSequentialIndirectAxesButtons = InParameters.bSequentialIndirectAxesButtons;

		// update CTRL + LMB/MMB/RMB indirect behavior
		if (MultiIndirectClickDragBehavior)
		{
			if (bSequentialIndirectAxesButtons)
			{
				MultiIndirectClickDragBehavior->EnableButton(EKeys::MiddleMouseButton);
			}
			else
			{
				MultiIndirectClickDragBehavior->DisableButton(EKeys::MiddleMouseButton);
			}
		}
	}
	
	bScreenSpaceIndirectOrthographicManipulation = InParameters.bScreenSpaceIndirectOrthographicManipulation;

	DefaultRotateMode = InParameters.RotateMode;
	
	// Unlike the EditorTRSGizmo, these are fixed (used to be in the now-deleted FGizmoDevParameters)
	constexpr float LineThicknessMultiplier = 1.0f;
	constexpr float TranslateAxisLength = 70.0f;
	constexpr float ScaleAxisLength = 70.0f;
	constexpr float RotateAxisRadius = 70.0f;

	// @note: The clamp values should match the property metadata (ClampMin)
	Style.AxisSizeMultiplier = FMath::Max(0.01f, InParameters.AxisSizeMultiplier);
	Style.LineThicknessMultiplier = FMath::Max(0.01f, LineThicknessMultiplier);
	Style.TranslateAxisLength = FMath::Max(1.0f, TranslateAxisLength);
	Style.RotateArcballSphereRadius = FMath::Max(1.0f, RotateAxisRadius);
	Style.RotateAxisRadius = Style.RotateArcballSphereRadius;
	Style.RotateCircleRadius = Style.RotateArcballSphereRadius;
	Style.ScaleAxisLength = FMath::Max(1.0f, ScaleAxisLength);

	UpdateElements();
}

UGizmoElementArrow* UTransformGizmo::MakeTranslateAxis(ETransformGizmoPartIdentifier InPartId, const FVector& InAxisDir, const FVector& InSideDir, UMaterialInterface* InMaterial)
{
	UGizmoElementArrow* ArrowElement = NewObject<UGizmoElementArrow>();
	ArrowElement->SetPartIdentifier(static_cast<uint32>(InPartId));
	ArrowElement->SetHeadType(EGizmoElementArrowHeadType::Cone);
	ArrowElement->SetEndCaps(true);
	ArrowElement->SetNumSides(32);
	ArrowElement->SetMaterial(InMaterial);
	ArrowElement->SetViewDependentType(EGizmoElementViewDependentType::Axis);
	ArrowElement->SetViewDependentAxis(InAxisDir);

	FVector AxisDir = InAxisDir;
	if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
	{
		if (InPartId == ETransformGizmoPartIdentifier::TranslateYAxis)
		{
			AxisDir = -AxisDir;
		}
	}

	UpdateTranslateAxis(ArrowElement, AxisDir, InSideDir);

	return ArrowElement;
}

void UTransformGizmo::UpdateTranslateAxis(UGizmoElementArrow* InElement)
{
	if (!InElement)
	{
		return;
	}

	UpdateTranslateAxis(InElement, InElement->GetDirection(), InElement->GetSideDirection());
}

void UTransformGizmo::UpdateTranslateAxis(UGizmoElementArrow* InElement, const EAxis::Type InAxis)
{
	if (!InElement)
	{
		return;
	}

	using namespace UE::EditorInteractiveToolsFramework::Private::TransformGizmoLocals;

	UpdateTranslateAxis(InElement, GetAxisVector(InAxis), GetAxisSideVector(InAxis));
}

void UTransformGizmo::UpdateTranslateAxis(UGizmoElementArrow* InElement, const FVector& InAxisDir, const FVector& InSideDir)
{
	if (!InElement)
	{
		return;
	}

	const float SizeCoeff = GetSizeCoefficient();

	InElement->SetBase(InAxisDir * Style.AxisLengthOffset * Style.AxisSizeMultiplier);
	InElement->SetDirection(InAxisDir);
	InElement->SetSideDirection(InSideDir);
	InElement->SetBodyLength(Style.TranslateAxisLength * Style.AxisSizeMultiplier * SizeCoeff);
	InElement->SetBodyRadius(Style.AxisRadius);
	InElement->SetHeadLength(Style.TranslateAxisConeHeight * SizeCoeff);
	InElement->SetHeadRadius(Style.TranslateAxisConeRadius * SizeCoeff);
}

UGizmoElementArrow* UTransformGizmo::MakeScaleAxis(ETransformGizmoPartIdentifier InPartId, const FVector& InAxisDir, const FVector& InSideDir, UMaterialInterface* InMaterial)
{
	UGizmoElementArrow* ArrowElement = NewObject<UGizmoElementArrow>();
	ArrowElement->SetPartIdentifier(static_cast<uint32>(InPartId));
	ArrowElement->SetHeadType(EGizmoElementArrowHeadType::Cube);
	ArrowElement->SetNumSides(32);
	ArrowElement->SetMaterial(InMaterial);
	ArrowElement->SetViewDependentType(EGizmoElementViewDependentType::Axis);
	ArrowElement->SetViewDependentAxis(InAxisDir);

	FVector AxisDir = InAxisDir;
    if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
    {
    	if (InPartId == ETransformGizmoPartIdentifier::ScaleYAxis)
    	{
    		AxisDir = -AxisDir;
    	}
    }

	UpdateScaleAxis(ArrowElement, AxisDir, InSideDir);

	return ArrowElement;
}

void UTransformGizmo::UpdateScaleAxis(UGizmoElementArrow* InElement)
{
	if (!InElement)
	{
		return;
	}

	UpdateScaleAxis(InElement, InElement->GetDirection(), InElement->GetSideDirection());
}

void UTransformGizmo::UpdateScaleAxis(UGizmoElementArrow* InElement, const EAxis::Type InAxis)
{
	if (!InElement)
	{
		return;
	}

	using namespace UE::EditorInteractiveToolsFramework::Private::TransformGizmoLocals;

	UpdateScaleAxis(InElement, GetAxisVector(InAxis), GetAxisSideVector(InAxis));
}

void UTransformGizmo::UpdateScaleAxis(UGizmoElementArrow* InElement, const FVector& InAxisDir, const FVector& InSideDir)
{
	if (!InElement)
	{
		return;
	}

	const float SizeCoeff = GetSizeCoefficient();

	InElement->SetBase(InAxisDir * Style.AxisLengthOffset * Style.AxisSizeMultiplier);
	InElement->SetDirection(InAxisDir);
	InElement->SetSideDirection(InSideDir);
	InElement->SetBodyLength(Style.ScaleAxisLength * Style.AxisSizeMultiplier * SizeCoeff);
	InElement->SetBodyRadius(Style.AxisRadius);
	InElement->SetHeadLength(Style.ScaleAxisCubeDim * SizeCoeff);
}

void UTransformGizmo::UpdateAllScaleAxis()
{
	if (ScaleXAxisElement)
	{
		UpdateScaleAxis(ScaleXAxisElement, EAxis::X);
	}

	if (ScaleYAxisElement)
	{
		UpdateScaleAxis(ScaleYAxisElement, EAxis::Y);
	}

	if (ScaleZAxisElement)
	{
		UpdateScaleAxis(ScaleZAxisElement, EAxis::Z);
	}
}

UGizmoElementBox* UTransformGizmo::MakeUniformScaleHandle()
{
	UGizmoElementBox* BoxElement = NewObject<UGizmoElementBox>();
	BoxElement->SetPartIdentifier(static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleUniform));
	BoxElement->SetCenter(FVector::ZeroVector);
	BoxElement->SetUpDirection(FVector::UpVector);
	BoxElement->SetSideDirection(FVector::RightVector);
	BoxElement->SetMaterial(GreyMaterial);
	UpdateUniformScaleHandle(BoxElement);

	return BoxElement;
}

void UTransformGizmo::UpdateUniformScaleHandle(UGizmoElementBox* InElement)
{
	if (!InElement)
	{
		return;
	}

	const float SizeCoeff = GetSizeCoefficient();

	InElement->SetDimensions(FVector( Style.ScaleAxisCubeDim, Style.ScaleAxisCubeDim, Style.ScaleAxisCubeDim) * SizeCoeff);
}

UGizmoElementBox* UTransformGizmo::MakePlanarHandle(ETransformGizmoPartIdentifier InPartId, const FVector& InUpDirection, const FVector& InSideDirection, const FVector& InPlaneNormal,
	UMaterialInterface* InMaterial)
{
	FVector UpDirection = InUpDirection;
	FVector SideDirection = InSideDirection;
	if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
	{
		if (InPartId == ETransformGizmoPartIdentifier::TranslateXYPlanar || InPartId == ETransformGizmoPartIdentifier::ScaleXYPlanar)
		{
			SideDirection = -SideDirection;
		}
		if (InPartId == ETransformGizmoPartIdentifier::TranslateYZPlanar || InPartId == ETransformGizmoPartIdentifier::ScaleYZPlanar)
		{
			UpDirection = -UpDirection;
		}
	}

	UGizmoElementBox* BoxElement = NewObject<UGizmoElementBox>();
	BoxElement->SetPartIdentifier(static_cast<uint32>(InPartId));
	BoxElement->SetUpDirection(UpDirection);
	BoxElement->SetSideDirection(SideDirection);
	BoxElement->SetMaterial(InMaterial);
	BoxElement->SetViewDependentType(EGizmoElementViewDependentType::Plane);
	BoxElement->SetViewDependentAxis(InPlaneNormal);
	UpdatePlanarHandle(BoxElement, UpDirection, SideDirection);

	return BoxElement;
}

void UTransformGizmo::UpdatePlanarHandle(UGizmoElementBox* InElement, const FVector& InUpDirection, const FVector& InSideDirection)
{
	if (!InElement)
    {
    	return;
    }

    const float SizeCoeff = GetSizeCoefficient();

    FVector PlanarHandleCenter = (InUpDirection + InSideDirection) * ((Style.TranslateAxisLength * Style.AxisSizeMultiplier) + Style.PlanarHandleOffset) * SizeCoeff;

	InElement->SetCenter(PlanarHandleCenter);
	InElement->SetDimensions(FVector(Style.PlanarHandleThickness, Style.PlanarHandleSize, Style.PlanarHandleSize) * SizeCoeff);
}

UGizmoElementRectangle* UTransformGizmo::MakeTranslateScreenSpaceHandle()
{
	UGizmoElementRectangle* RectangleElement = NewObject<UGizmoElementRectangle>();
	RectangleElement->SetPartIdentifier(static_cast<uint32>(ETransformGizmoPartIdentifier::TranslateScreenSpace));
	RectangleElement->SetUpDirection(FVector::UpVector);
	RectangleElement->SetSideDirection(FVector::RightVector);
	RectangleElement->SetCenter(FVector::ZeroVector);
	RectangleElement->SetViewAlignType(EGizmoElementViewAlignType::PointScreen);
	RectangleElement->SetViewAlignAxis(FVector::UpVector);
	RectangleElement->SetViewAlignNormal(-FVector::ForwardVector);
	RectangleElement->SetMaterial(TransparentVertexColorMaterial);
	RectangleElement->SetLineColor(Style.ScreenSpaceColor);
	RectangleElement->SetHitMesh(true);
	RectangleElement->SetDrawMesh(false);
	RectangleElement->SetDrawLine(true);
	RectangleElement->SetLineThickness(Style.AxisRadius * 2.0f);
	RectangleElement->SetHoverLineThicknessMultiplier(1.5f);
	RectangleElement->SetInteractLineThicknessMultiplier(1.5f);
	UpdateTranslateScreenSpaceHandle(RectangleElement);

	return RectangleElement;
}

void UTransformGizmo::UpdateTranslateScreenSpaceHandle(UGizmoElementRectangle* InElement)
{
	if (!InElement)
	{
		return;
	}

	const float SizeCoeff = GetSizeCoefficient();

	InElement->SetHeight(Style.TranslateScreenSpaceHandleSize * SizeCoeff);
	InElement->SetWidth(Style.TranslateScreenSpaceHandleSize * SizeCoeff);
}

UGizmoElementTorus* UTransformGizmo::MakeDefaultRotateAxis(const ETransformGizmoPartIdentifier InPartId)
{
	switch (InPartId)
	{
	case ETransformGizmoPartIdentifier::RotateXAxis:
	case ETransformGizmoPartIdentifier::RotateXGimbal:
		return MakeRotateAxis(InPartId, FVector::YAxisVector, FVector::ZAxisVector, AxisMaterialX, CurrentAxisMaterial);
		
	case ETransformGizmoPartIdentifier::RotateYAxis:
	case ETransformGizmoPartIdentifier::RotateYGimbal:
		return MakeRotateAxis(InPartId, FVector::ZAxisVector, FVector::XAxisVector, AxisMaterialY, CurrentAxisMaterial);
		
	case ETransformGizmoPartIdentifier::RotateZAxis:
	case ETransformGizmoPartIdentifier::RotateZGimbal:
		return MakeRotateAxis(InPartId, FVector::XAxisVector, FVector::YAxisVector, AxisMaterialZ, CurrentAxisMaterial);
		
	default:
		break;
	}

	ensure(false);
	return nullptr;
}

UGizmoElementTorus* UTransformGizmo::MakeRotateAxis(ETransformGizmoPartIdentifier InPartId, const FVector& TorusAxis0, const FVector& TorusAxis1,
	UMaterialInterface* InMaterial, UMaterialInterface* InCurrentMaterial)
{
	UGizmoElementTorus* RotateAxisElement = NewObject<UGizmoElementTorus>();
	RotateAxisElement->SetPartIdentifier(static_cast<uint32>(InPartId));
	RotateAxisElement->SetCenter(FVector::ZeroVector);
	RotateAxisElement->SetNumSegments(Style.RotateAxisNumSegments);
	RotateAxisElement->SetNumInnerSlices(Style.RotateAxisInnerSlices);
	RotateAxisElement->SetAxisBitangent(TorusAxis0);
	RotateAxisElement->SetAxisTangent(TorusAxis1);
	const FVector TorusNormal = RotateAxisElement->GetAxisTangent() ^ RotateAxisElement->GetAxisBitangent();
	RotateAxisElement->SetPartialType(EGizmoElementPartialType::PartialViewDependent);
	RotateAxisElement->SetPartialStartAngle(0.0f);
	RotateAxisElement->SetPartialEndAngle(UE_PI);
	RotateAxisElement->SetViewDependentAxis(TorusNormal);
	RotateAxisElement->SetViewAlignType(EGizmoElementViewAlignType::Axial);
	RotateAxisElement->SetViewAlignAxialAngleTol(static_cast<float>(UE_DOUBLE_SMALL_NUMBER));
	RotateAxisElement->SetViewAlignAxis(TorusNormal);
	RotateAxisElement->SetViewAlignNormal(TorusAxis1);
	RotateAxisElement->SetMaterial(InMaterial);
	UpdateRotateAxis(RotateAxisElement);

	return RotateAxisElement;
}

void UTransformGizmo::UpdateRotateAxis(UGizmoElementTorus* InElement)
{
	if (!InElement)
	{
		return;
	}

	const float SizeCoeff = GetSizeCoefficient();

	InElement->SetRadius(((Style.RotateAxisRadius * Style.AxisSizeMultiplier) + Style.RotateAxisOuterRadiusOffset) * SizeCoeff);
	InElement->SetInnerRadius((Style.RotateAxisInnerRadius * Style.LineThicknessMultiplier) * SizeCoeff);
}

UGizmoElementCircle* UTransformGizmo::MakeArcballCircleHandle(ETransformGizmoPartIdentifier InPartId, float InRadius, const FLinearColor& InColor)
{
	const FLinearColor RotateArcballCircleLineColor = 0.25f * Style.RotateArcballCircleColor.CopyWithNewOpacity(1.0f);

	UGizmoElementCircle* CircleElement = MakeRotateCircleHandle(ETransformGizmoPartIdentifier::RotateArcball, Style.RotateArcballSphereRadius, Style.RotateArcballCircleColor,
		EGizmoElementDrawType::FillAndLine, RotateArcballCircleLineColor);

	// Effectively hides the line when not hovering/interacting
	CircleElement->SetLineColor(FLinearColor::Transparent);

	return CircleElement;
}

UGizmoElementCircle* UTransformGizmo::MakeRotateCircleHandle(ETransformGizmoPartIdentifier InPartId, float InRadius, const FLinearColor& InColor,
	const EGizmoElementDrawType InDrawType, const TOptional<FLinearColor>& InLineColorOverride)
{
	UGizmoElementCircle* CircleElement = NewObject<UGizmoElementCircle>();
	CircleElement->SetPartIdentifier(static_cast<uint32>(InPartId));
	CircleElement->SetCenter(FVector::ZeroVector);
	CircleElement->SetAxisBitangent(FVector::UpVector);
	CircleElement->SetAxisTangent(-FVector::RightVector);
	CircleElement->SetLineColor(InColor);
	CircleElement->SetViewAlignType(EGizmoElementViewAlignType::PointOnly);
	CircleElement->SetViewAlignNormal(-FVector::ForwardVector);

	// Disable Draw/Hit by default, and toggle below based on the provided flags
	CircleElement->SetDrawMesh(false);
	CircleElement->SetHitMesh(false);
	CircleElement->SetDrawLine(false);
	CircleElement->SetHitLine(false);

	if (EnumHasAnyFlags(InDrawType, EGizmoElementDrawType::Fill))
	{
		CircleElement->SetDrawMesh(true);
		CircleElement->SetHitMesh(true);

		CircleElement->SetVertexColor(FLinearColor::Transparent);
		CircleElement->SetMaterial(TransparentVertexColorMaterial);

		const FLinearColor LightColor(InColor.R, InColor.G, InColor.B, 0.15f);

		CircleElement->SetHoverVertexColor(LightColor);
		CircleElement->SetHoverMaterial(TransparentVertexColorMaterial);

		CircleElement->SetInteractVertexColor(LightColor);
		CircleElement->SetInteractMaterial(TransparentVertexColorMaterial);
	}

	if (EnumHasAnyFlags(InDrawType, EGizmoElementDrawType::Line))
	{
		CircleElement->SetDrawLine(true);
		CircleElement->SetHitLine(true);

		if (InLineColorOverride.IsSet())
		{
			const FLinearColor LineColor = InLineColorOverride.GetValue();

			CircleElement->SetLineColor(LineColor);
			CircleElement->SetHoverLineColor(LineColor);
			CircleElement->SetInteractLineColor(LineColor);
		}
	}

	UpdateRotateCircleHandle(CircleElement, InRadius);

	return CircleElement;
}

void UTransformGizmo::UpdateRotateCircleHandle(UGizmoElementCircle* InElement, float InRadius)
{
	if (!InElement)
	{
		return;
	}

	const float SizeCoeff = GetSizeCoefficient();

	InElement->SetRadius((InRadius * Style.AxisSizeMultiplier) * SizeCoeff);
	InElement->SetLineThickness(Style.LineThicknessMultiplier);
}

void UTransformGizmo::UpdateElements()
{
	const FVector XAxis = FVector::XAxisVector;
	const FVector YAxis = FVector::YAxisVector;
	const FVector ZAxis = FVector::ZAxisVector;

	UpdateTranslateAxis(TranslateXAxisElement);
	UpdateTranslateAxis(TranslateYAxisElement);
	UpdateTranslateAxis(TranslateZAxisElement);

	UpdatePlanarHandle(TranslatePlanarXYElement, XAxis, YAxis);
	UpdatePlanarHandle(TranslatePlanarYZElement, YAxis, ZAxis);
	UpdatePlanarHandle(TranslatePlanarXZElement, XAxis, ZAxis);

	UpdateTranslateScreenSpaceHandle(TranslateScreenSpaceElement);

	UpdateScaleAxis(ScaleXAxisElement);
	UpdateScaleAxis(ScaleYAxisElement);
	UpdateScaleAxis(ScaleZAxisElement);

	UpdatePlanarHandle(ScalePlanarXYElement, XAxis, YAxis);
	UpdatePlanarHandle(ScalePlanarYZElement, YAxis, ZAxis);
	UpdatePlanarHandle(ScalePlanarXZElement, XAxis, ZAxis);

	UpdateUniformScaleHandle(ScaleUniformElement);

	UpdateRotateAxis(RotateXAxisElement);
	UpdateRotateAxis(RotateYAxisElement);
	UpdateRotateAxis(RotateZAxisElement);

	UpdateRotateCircleHandle(RotateArcballElement, Style.RotateArcballSphereRadius);
	UpdateRotateCircleHandle(RotateScreenSpaceElement, Style.RotateCircleRadius + Style.RotateScreenSpaceRadiusOffset);
}

void UTransformGizmo::ClearActiveTarget()
{
	StateTarget = nullptr;

	if (ActiveTarget)
	{
		OnAboutToClearActiveTarget.Broadcast(this, ActiveTarget);
		
		ActiveTarget->OnBeginTransformEdit.RemoveAll(this);
		ActiveTarget->OnEndTransformEdit.RemoveAll(this);
		ActiveTarget = nullptr;
	}
}


bool UTransformGizmo::PositionSnapFunction(const FVector& WorldPosition, FVector& SnappedPositionOut) const
{
	SnappedPositionOut = WorldPosition;
#if 0
	// only snap if we want snapping obvs
	if (bSnapToWorldGrid == false)
	{
		return false;
	}

	// only snap to world grid when using world axes
	if (GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem() != EToolContextCoordinateSystem::World)
	{
		return false;
	}

	FSceneSnapQueryRequest Request;
	Request.RequestType = ESceneSnapQueryType::Position;
	Request.TargetTypes = ESceneSnapQueryTargetType::Grid;
	Request.Position = WorldPosition;
	if ( bGridSizeIsExplicit )
	{
		Request.GridSize = ExplicitGridSize;
	}
	TArray<FSceneSnapQueryResult> Results;
	if (GetGizmoManager()->GetContextQueriesAPI()->ExecuteSceneSnapQuery(Request, Results))
	{
		SnappedPositionOut = Results[0].Position;
		return true;
	};
#endif
	return false;
}


FQuat UTransformGizmo::RotationSnapFunction(const FQuat& DeltaRotation) const
{
	FQuat SnappedDeltaRotation = DeltaRotation;
#if 0
	// only snap if we want snapping 
	if (bSnapToWorldRotGrid)
	{
		FSceneSnapQueryRequest Request;
		Request.RequestType   = ESceneSnapQueryType::Rotation;
		Request.TargetTypes   = ESceneSnapQueryTargetType::Grid;
		Request.DeltaRotation = DeltaRotation;
		if ( bRotationGridSizeIsExplicit )
		{
			Request.RotGridSize = ExplicitRotationGridSize;
		}
		TArray<FSceneSnapQueryResult> Results;
		if (GetGizmoManager()->GetContextQueriesAPI()->ExecuteSceneSnapQuery(Request, Results))
		{
			SnappedDeltaRotation = Results[0].DeltaRotation;
		};
	}
#endif	
	return SnappedDeltaRotation;
}

FVector UTransformGizmo::GetWorldAxis(const FVector& InAxis) const
{
	auto GetCoordinateSystem = [&]()
	{
		if (TransformGizmoSource)
		{
			return TransformGizmoSource->GetGizmoCoordSystemSpace(); 
		}
		return GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem();
	};
	
	if (GetCoordinateSystem() == EToolContextCoordinateSystem::Local)
	{
		return CurrentTransform.TransformVectorNoScale(InAxis);
	}
	
	return InAxis;
}

FVector UTransformGizmo::GetGimbalRotationAxis(const int32 InAxis) const
{
	return UE::GizmoRotationUtil::GetRotationAxis(CurrentTransform, GetRotationContext(), InAxis);
}

void UTransformGizmo::SetupOnClickFunctions()
{
	int NumParts = static_cast<int>(ETransformGizmoPartIdentifier::Max);
	OnClickPressFunctions.SetNum(NumParts);
	OnClickDragFunctions.SetNum(NumParts);
	OnClickReleaseFunctions.SetNum(NumParts);

	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXAxis)] = &UTransformGizmo::OnClickPressTranslateXAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYAxis)] = &UTransformGizmo::OnClickPressTranslateYAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateZAxis)] = &UTransformGizmo::OnClickPressTranslateZAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXYPlanar)] = &UTransformGizmo::OnClickPressTranslateXYPlanar;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYZPlanar)] = &UTransformGizmo::OnClickPressTranslateYZPlanar;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXZPlanar)] = &UTransformGizmo::OnClickPressTranslateXZPlanar;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateScreenSpace)] = &UTransformGizmo::OnClickPressScreenSpaceTranslate;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXAxis)] = &UTransformGizmo::OnClickPressScaleXAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYAxis)] = &UTransformGizmo::OnClickPressScaleYAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleZAxis)] = &UTransformGizmo::OnClickPressScaleZAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXYPlanar)] = &UTransformGizmo::OnClickPressScaleXYPlanar;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYZPlanar)] = &UTransformGizmo::OnClickPressScaleYZPlanar;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXZPlanar)] = &UTransformGizmo::OnClickPressScaleXZPlanar;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleUniform)] = &UTransformGizmo::OnClickPressScaleXYZ;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXAxis)] = &UTransformGizmo::OnClickPressRotateXAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYAxis)] = &UTransformGizmo::OnClickPressRotateYAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZAxis)] = &UTransformGizmo::OnClickPressRotateZAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateScreenSpace)] = &UTransformGizmo::OnClickPressScreenSpaceRotate;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateArcball)] = &UTransformGizmo::OnClickPressArcBallRotate;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXGimbal)] = &UTransformGizmo::OnClickPressGimbalRotateAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYGimbal)] = &UTransformGizmo::OnClickPressGimbalRotateAxis;
	OnClickPressFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZGimbal)] = &UTransformGizmo::OnClickPressGimbalRotateAxis;

	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXAxis)] = &UTransformGizmo::OnClickDragTranslateAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYAxis)] = &UTransformGizmo::OnClickDragTranslateAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateZAxis)] = &UTransformGizmo::OnClickDragTranslateAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXYPlanar)] = &UTransformGizmo::OnClickDragTranslatePlanar;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYZPlanar)] = &UTransformGizmo::OnClickDragTranslatePlanar;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXZPlanar)] = &UTransformGizmo::OnClickDragTranslatePlanar;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateScreenSpace)] = &UTransformGizmo::OnClickDragScreenSpaceTranslate;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXAxis)] = &UTransformGizmo::OnClickDragScaleAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYAxis)] = &UTransformGizmo::OnClickDragScaleAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleZAxis)] = &UTransformGizmo::OnClickDragScaleAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXYPlanar)] = &UTransformGizmo::OnClickDragScalePlanar;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYZPlanar)] = &UTransformGizmo::OnClickDragScalePlanar;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXZPlanar)] = &UTransformGizmo::OnClickDragScalePlanar;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleUniform)] = &UTransformGizmo::OnClickDragScaleXYZ;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXAxis)] = &UTransformGizmo::OnClickDragRotateAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYAxis)] = &UTransformGizmo::OnClickDragRotateAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZAxis)] = &UTransformGizmo::OnClickDragRotateAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateScreenSpace)] = &UTransformGizmo::OnClickDragScreenSpaceRotate;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateArcball)] = &UTransformGizmo::OnClickDragArcBallRotate;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXGimbal)] = &UTransformGizmo::OnClickDragGimbalRotateAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYGimbal)] = &UTransformGizmo::OnClickDragGimbalRotateAxis;
	OnClickDragFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZGimbal)] = &UTransformGizmo::OnClickDragGimbalRotateAxis;

	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXAxis)] = &UTransformGizmo::OnClickReleaseTranslateAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYAxis)] = &UTransformGizmo::OnClickReleaseTranslateAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateZAxis)] = &UTransformGizmo::OnClickReleaseTranslateAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXYPlanar)] = &UTransformGizmo::OnClickReleaseTranslatePlanar;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateYZPlanar)] = &UTransformGizmo::OnClickReleaseTranslatePlanar;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateXZPlanar)] = &UTransformGizmo::OnClickReleaseTranslatePlanar;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::TranslateScreenSpace)] = &UTransformGizmo::OnClickReleaseScreenSpaceTranslate;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXAxis)] = &UTransformGizmo::OnClickReleaseScaleAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYAxis)] = &UTransformGizmo::OnClickReleaseScaleAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleZAxis)] = &UTransformGizmo::OnClickReleaseScaleAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXYPlanar)] = &UTransformGizmo::OnClickReleaseScalePlanar;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleYZPlanar)] = &UTransformGizmo::OnClickReleaseScalePlanar;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleXZPlanar)] = &UTransformGizmo::OnClickReleaseScalePlanar;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::ScaleUniform)] = &UTransformGizmo::OnClickReleaseScaleXYZ;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXAxis)] = &UTransformGizmo::OnClickReleaseRotateAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYAxis)] = &UTransformGizmo::OnClickReleaseRotateAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZAxis)] = &UTransformGizmo::OnClickReleaseRotateAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateScreenSpace)] = &UTransformGizmo::OnClickReleaseScreenSpaceRotate;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateArcball)] = &UTransformGizmo::OnClickReleaseArcBallRotate;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateXGimbal)] = &UTransformGizmo::OnClickReleaseRotateAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateYGimbal)] = &UTransformGizmo::OnClickReleaseRotateAxis;
	OnClickReleaseFunctions[static_cast<int>(ETransformGizmoPartIdentifier::RotateZGimbal)] = &UTransformGizmo::OnClickReleaseRotateAxis;
}


float UTransformGizmo::GetNearestRayParamToInteractionAxis(const FInputDeviceRay& InRay)
{
	float RayNearestParam, AxisNearestParam;
	FVector RayNearestPt, AxisNearestPoint;
	GizmoMath::NearestPointOnLineToRay(InteractionAxisOrigin, InteractionAxisDirection,
		InRay.WorldRay.Origin, InRay.WorldRay.Direction, AxisNearestPoint, AxisNearestParam,
		RayNearestPt, RayNearestParam);
	return AxisNearestParam;
}

bool UTransformGizmo::GetRayParamIntersectionWithInteractionPlane(const FInputDeviceRay& InRay, FVector::FReal& OutHitParam)
{
	// if ray is parallel to plane, nothing has been hit
	if (FMath::IsNearlyZero(FVector::DotProduct(InteractionPlanarNormal, InRay.WorldRay.Direction)))
	{
		return false;
	}

	FPlane Plane(InteractionPlanarOrigin, InteractionPlanarNormal);
	OutHitParam = FMath::RayPlaneIntersectionParam(InRay.WorldRay.Origin, InRay.WorldRay.Direction, Plane);
	if (OutHitParam < 0)
	{
		return false;
	}

	return true;
}

void UTransformGizmo::UpdateHoverState(const bool bInHover, const ETransformGizmoPartIdentifier InHitPartId)
{
	if (!ensure(HitTarget))
	{
		return;
	}
	
	HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(InHitPartId));

	switch (InHitPartId)
	{
	case ETransformGizmoPartIdentifier::ScaleXAxis:
		HitTarget->UpdateHoverState(!bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleYAxis));
		HitTarget->UpdateHoverState(!bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleZAxis));
		break;
	case ETransformGizmoPartIdentifier::ScaleYAxis:
		HitTarget->UpdateHoverState(!bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleXAxis));
		HitTarget->UpdateHoverState(!bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleZAxis));
		break;
	case ETransformGizmoPartIdentifier::ScaleZAxis:
		HitTarget->UpdateHoverState(!bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleXAxis));
		HitTarget->UpdateHoverState(!bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleYAxis));
		break;

	case ETransformGizmoPartIdentifier::ScaleUniform:
		HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleXAxis));
		HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleYAxis));
		HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleZAxis));
		break;
	case ETransformGizmoPartIdentifier::ScaleXYPlanar:
		HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleXAxis));
		HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleYAxis));
		break;
	case ETransformGizmoPartIdentifier::ScaleYZPlanar:
		HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleYAxis));
		HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleZAxis));
		break;
	case ETransformGizmoPartIdentifier::ScaleXZPlanar:
		HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleXAxis));
		HitTarget->UpdateHoverState(bInHover, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleZAxis));
		break;
	default:
		break;
	}
}

void UTransformGizmo::ResetHoverStates(const EGizmoTransformMode InMode)
{
	ETransformGizmoPartIdentifier IdBegin = ETransformGizmoPartIdentifier::Default;
	ETransformGizmoPartIdentifier IdEnd = ETransformGizmoPartIdentifier::Max;

	switch (InMode)
	{
	case EGizmoTransformMode::Translate:
		IdBegin = ETransformGizmoPartIdentifier::TranslateAll;
		IdEnd = ETransformGizmoPartIdentifier::RotateAll;
		break;
	case EGizmoTransformMode::Rotate:
		IdBegin = ETransformGizmoPartIdentifier::RotateAll;
		IdEnd = ETransformGizmoPartIdentifier::ScaleAll;
		break;
	case EGizmoTransformMode::Scale:
		IdBegin = ETransformGizmoPartIdentifier::ScaleAll;
		IdEnd = ETransformGizmoPartIdentifier::Max;
		break;
	default:
		break;
	}

	static constexpr bool bInHover = false;
	for (uint32 Id = static_cast<uint32>(IdBegin); Id < static_cast<uint32>(IdEnd); ++Id)
	{
		UpdateHoverState(bInHover, static_cast<ETransformGizmoPartIdentifier>(Id));
	}
}

void UTransformGizmo::UpdateInteractingState(const bool bInInteracting, const ETransformGizmoPartIdentifier InHitPartId, const bool bIdOnly)
{
	if (!ensure(HitTarget))
	{
		return;
	}
	
	HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(InHitPartId));

	if (!bIdOnly)
	{
		switch (InHitPartId)
		{
		case ETransformGizmoPartIdentifier::RotateArcball:
			{
				HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::RotateArcball));
				GizmoElementRoot->UpdatePartVisibleState(!bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::RotateScreenSpace));
				break;
			}

		case ETransformGizmoPartIdentifier::ScaleUniform:
			{
				HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleXAxis));
				HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleYAxis));
				HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleZAxis));
				HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleUniform));
				GizmoElementRoot->UpdatePartVisibleState(!bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleXYPlanar));
				GizmoElementRoot->UpdatePartVisibleState(!bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleYZPlanar));
				GizmoElementRoot->UpdatePartVisibleState(!bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleXZPlanar));
				break;
			}
		case ETransformGizmoPartIdentifier::ScaleXYPlanar:
			HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleXAxis));
			HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleYAxis));
			break;
		case ETransformGizmoPartIdentifier::ScaleYZPlanar:
			HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleYAxis));
			HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleZAxis));
			break;
		case ETransformGizmoPartIdentifier::ScaleXZPlanar:
			HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleXAxis));
			HitTarget->UpdateInteractingState(bInInteracting, static_cast<uint32>(ETransformGizmoPartIdentifier::ScaleZAxis));
			break;
		default:
			break;
		}
	}
}

void UTransformGizmo::ResetInteractingStates(const EGizmoTransformMode InMode)
{
	ETransformGizmoPartIdentifier IdBegin = ETransformGizmoPartIdentifier::Default;
	ETransformGizmoPartIdentifier IdEnd = ETransformGizmoPartIdentifier::Max;
	bool bIdOnly = true;

	switch (InMode)
	{
	case EGizmoTransformMode::Translate:
		IdBegin = ETransformGizmoPartIdentifier::TranslateAll;
		IdEnd = ETransformGizmoPartIdentifier::RotateAll;
		break;
	case EGizmoTransformMode::Rotate:
		IdBegin = ETransformGizmoPartIdentifier::RotateAll;
		IdEnd = ETransformGizmoPartIdentifier::ScaleAll;
		break;
	case EGizmoTransformMode::Scale:
		IdBegin = ETransformGizmoPartIdentifier::ScaleAll;
		IdEnd = ETransformGizmoPartIdentifier::Max;
		bIdOnly = false; 
		break;
	default:
		break;
	}

	static constexpr bool bInInteracting = false;
	for (uint32 Id = static_cast<uint32>(IdBegin); Id < static_cast<uint32>(IdEnd); ++Id)
	{
		UpdateInteractingState(bInInteracting, static_cast<ETransformGizmoPartIdentifier>(Id), bIdOnly);
	}
}

void UTransformGizmo::BeginTransformEditSequence()
{
	// Update Debug
	{
		DebugData.bIsEditing = true;
		DebugData.TransformStart = CurrentTransform;
	}

	if (ensure(ActiveTarget))
	{
		if (ActiveTarget->bSetPivotMode)
		{
			ActiveTarget->BeginPivotEditSequence();
		}
		else
		{
			ActiveTarget->BeginTransformEditSequence();
		}
	}

	if (ensure(StateTarget))
	{
		StateTarget->BeginUpdate();
	}
}

void UTransformGizmo::EndTransformEditSequence()
{
	// Update Debug
	{
		DebugData.bIsEditing = false;
	}

	if (ensure(StateTarget))
	{
		StateTarget->EndUpdate();
	}

	if (ensure(ActiveTarget))
	{
		if (ActiveTarget->bSetPivotMode)
		{
			ActiveTarget->EndPivotEditSequence();
		}
		else
		{
			ActiveTarget->EndTransformEditSequence();
		}
	}
}

void UTransformGizmo::OnClickPress(const FInputDeviceRay& PressPos)
{
	PendingDragFunction.Reset();
	
	check(OnClickPressFunctions.Num() == static_cast<int>(ETransformGizmoPartIdentifier::Max));

	const ETransformGizmoPartIdentifier ModeLastHitPart = GetCurrentModeLastHitPart();
	
	if (OnClickPressFunctions[static_cast<int>(LastHitPart)])
	{
		OnClickPressFunctions[static_cast<int>(LastHitPart)](this, PressPos);
	}

	if (bInInteraction)
	{
		if (HitTarget && LastHitPart != ETransformGizmoPartIdentifier::Default)
		{
			if (LastHitPart != ModeLastHitPart)
			{
				UpdateInteractingState(false, ModeLastHitPart, true);	
			}
			UpdateInteractingState(true, LastHitPart);
		}

		BeginTransformEditSequence();
	}
}

void UTransformGizmo::OnClickDrag(const FInputDeviceRay& DragPos)
{
	if (!bInInteraction)
	{
		return;
	}

	int HitPartIndex = static_cast<int>(LastHitPart);
	check(HitPartIndex < OnClickDragFunctions.Num());

	if (OnClickDragFunctions[HitPartIndex])
	{
		if (bDeferDrag)
		{ // defer drag function on next tick
			PendingDragFunction = [this, DragPos, HitPartIndex]()
			{
				OnClickDragFunctions[HitPartIndex](this, DragPos);
			};
		}
		else
		{ // do drag function
			OnClickDragFunctions[HitPartIndex](this, DragPos);
		}
	}
}

void UTransformGizmo::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	if (!bInInteraction)
	{
		return;
	}

	const int HitPartIndex = static_cast<int>(LastHitPart);
	check(HitPartIndex < OnClickReleaseFunctions.Num());

	if (OnClickReleaseFunctions[HitPartIndex])
	{
		OnClickReleaseFunctions[HitPartIndex](this, ReleasePos);
	}

	EndTransformEditSequence();

	bInInteraction = false;

	if (HitTarget && LastHitPart != ETransformGizmoPartIdentifier::Default)
	{
		UpdateInteractingState(false, LastHitPart);
		UpdateInteractingState(true, GetCurrentModeLastHitPart(), true);
	}

	PendingDragFunction.Reset();
	
	if (OnPostInteractionDelegate.IsBound())
	{
		FGizmoInteractionDescription Desc({.Ray=ReleasePos, .bIndirect=bIndirectManipulation});
		OnPostInteractionDelegate.Broadcast(Desc);
	}
}

void UTransformGizmo::OnTerminateDragSequence()
{
	if (!bInInteraction)
	{
		return;
	}

	if (StateTarget)
	{
		StateTarget->EndUpdate();
	}
	bInInteraction = false;

	if (HitTarget && LastHitPart != ETransformGizmoPartIdentifier::Default)
	{
		HitTarget->UpdateInteractingState(false, static_cast<uint32>(LastHitPart));
	}
}

void UTransformGizmo::OnClickPressTranslateXAxis(const FInputDeviceRay& PressPos)
{
	InteractionAxisOrigin = CurrentTransform.GetLocation();
	InteractionAxisDirection = GetWorldAxis(FVector::XAxisVector);
	SetInteractionAxisList(EAxisList::X);
	OnClickPressAxis(PressPos);
}

void UTransformGizmo::OnClickPressTranslateYAxis(const FInputDeviceRay& PressPos)
{
	InteractionAxisOrigin = CurrentTransform.GetLocation();
	InteractionAxisDirection = GetWorldAxis(FVector::YAxisVector);
	SetInteractionAxisList(EAxisList::Y);
	OnClickPressAxis(PressPos);
}

void UTransformGizmo::OnClickPressTranslateZAxis(const FInputDeviceRay& PressPos)
{
	InteractionAxisOrigin = CurrentTransform.GetLocation();
	InteractionAxisDirection = GetWorldAxis(FVector::ZAxisVector);
	SetInteractionAxisList(EAxisList::Z);
	OnClickPressAxis(PressPos);
}

void UTransformGizmo::OnClickPressAxis(const FInputDeviceRay& PressPos)
{
	InteractionPlanarOrigin = CurrentTransform.GetLocation();
	InteractionAxisStartParam = GetNearestRayParamToInteractionAxis(PressPos);
	InteractionAxisCurrParam = InteractionAxisStartParam;

	// indirect manipulation uses a 2D approach instead as there's no guaranty to intersect a plane 
	if (bIndirectManipulation)
	{
		InteractionScreenCurrPos = PressPos.ScreenPosition;
		StartRotation = CurrentRotation = ActiveTarget->GetTransform().GetRotation();
		bInInteraction = true;
		SetModeLastHitPart(EGizmoTransformMode::Translate, LastHitPart);
		return;
	}

	// compute plane and axis to mute
	const FVector XAxis = GetWorldAxis(FVector::XAxisVector);
	const FVector YAxis = GetWorldAxis(FVector::YAxisVector);
	const FVector ZAxis = GetWorldAxis(FVector::ZAxisVector);

	const FVector ViewDirection = GizmoViewContext->GetViewDirection();
	const FVector::FReal XDot = FMath::Abs(FVector::DotProduct(ViewDirection, XAxis));
	const FVector::FReal YDot = FMath::Abs(FVector::DotProduct(ViewDirection, YAxis));
	const FVector::FReal ZDot = FMath::Abs(FVector::DotProduct(ViewDirection, ZAxis));

	if (FVector::DotProduct(InteractionAxisDirection, XAxis) > 0.1)
	{
		InteractionPlanarNormal = (YDot > ZDot) ? YAxis : ZAxis;
		NormalToRemove = (YDot > ZDot) ? ZAxis : YAxis;
	}
	else if (FVector::DotProduct(InteractionAxisDirection, YAxis) > 0.1)
	{
		InteractionPlanarNormal = (XDot > ZDot) ? XAxis : ZAxis;
		NormalToRemove = (XDot > ZDot) ? ZAxis : XAxis;
	}
	else
	{
		InteractionPlanarNormal = (XDot > YDot) ? XAxis : YAxis;
		NormalToRemove = (XDot > YDot) ? YAxis : XAxis;
	}
	
	FVector::FReal HitDepth;
	if (GetRayParamIntersectionWithInteractionPlane(PressPos, HitDepth))
	{
		InteractionPlanarStartPoint = PressPos.WorldRay.Origin + PressPos.WorldRay.Direction * HitDepth;
		InteractionPlanarCurrPoint = InteractionPlanarStartPoint;
		
		// Update Debug
		if (GizmoLocals::bDebugDraw)
		{
			DebugData.TransformStart.SetLocation(InteractionPlanarOrigin);
			DebugData.InteractionStart.SetLocation(InteractionPlanarStartPoint);
			DebugData.InteractionCurrent.SetLocation(InteractionPlanarStartPoint);
			DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
		}
	}

	bInInteraction = true;
	SetModeLastHitPart(EGizmoTransformMode::Translate, LastHitPart);
}

void UTransformGizmo::SnapTranslateDelta(FVector& InOutWorldDelta) const
{
	UInteractiveGizmoManager* GizmoManager = GetGizmoManager();
	if (!GizmoManager)
	{
		return;
	}
	
	IToolsContextQueriesAPI* QueriesAPI = GizmoManager->GetContextQueriesAPI();
	if (!QueriesAPI)
	{
		return;
	}

	if (const USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GizmoManager))
	{
		FSceneSnapQueryRequest SnapRequest;
		SnapRequest.RequestCoordinateSpace = QueriesAPI->GetCurrentCoordinateSystem();
		SnapRequest.RequestType = ESceneSnapQueryType::Position;
		SnapRequest.AxisList = GetInteractionAxisList();

		FVector PositionToSnap = InOutWorldDelta;
		if (SnapRequest.RequestCoordinateSpace == EToolContextCoordinateSystem::Local)
		{
			// Convert the delta to local space
			const FQuat Rotation = ActiveTarget->GetTransform().GetRotation();
			PositionToSnap = Rotation.Inverse() * InOutWorldDelta;
		}

		SnapRequest.Position = PositionToSnap;

		TArray<FSceneSnapQueryResult> SnapResults;
		if (SnapManager->ExecuteSceneSnapQuery(SnapRequest, SnapResults))
		{
			check(!SnapResults.IsEmpty());
			const FSceneSnapQueryResult& SnapResult = SnapResults[0];

			// Convert the delta back to the desired space
			if (SnapResult.GetCoordinateSpace(SnapRequest) == EToolContextCoordinateSystem::Local)
			{
				const FQuat Rotation = ActiveTarget->GetTransform().GetRotation();
				InOutWorldDelta = Rotation * SnapResult.Position;
			}
			else
			{
				InOutWorldDelta = SnapResult.Position;
			}
		}
	}
}

void UTransformGizmo::SnapRotateDelta(FQuat& InOutWorldDelta) const
{
	UInteractiveGizmoManager* GizmoManager = GetGizmoManager();
	if (!GizmoManager)
	{
		return;
	}

	IToolsContextQueriesAPI* QueriesAPI = GizmoManager->GetContextQueriesAPI();
	if (!QueriesAPI)
	{
		return;
	}

	if (const USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GizmoManager))
	{
		FSceneSnapQueryRequest SnapRequest;
		SnapRequest.RequestCoordinateSpace = QueriesAPI->GetCurrentCoordinateSystem();
		SnapRequest.RequestType = ESceneSnapQueryType::Rotation;
		SnapRequest.AxisList = GetInteractionAxisList();
		SnapRequest.DeltaRotation = InOutWorldDelta;

		TArray<FSceneSnapQueryResult> SnapResults;
		if (SnapManager->ExecuteSceneSnapQuery(SnapRequest, SnapResults))
		{
			check(!SnapResults.IsEmpty());

			InOutWorldDelta = SnapResults[0].DeltaRotation;
		}
	}
}

void UTransformGizmo::SnapRotateAngleDelta(double& InOutAngleDelta) const
{
	UInteractiveGizmoManager* GizmoManager = GetGizmoManager();
	if (!GizmoManager)
	{
		return;
	}

	IToolsContextQueriesAPI* QueriesAPI = GizmoManager->GetContextQueriesAPI();
	if (!QueriesAPI)
	{
		return;
	}

	if (const USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GizmoManager))
	{
		FSceneSnapQueryRequest SnapRequest;
		SnapRequest.RequestCoordinateSpace = QueriesAPI->GetCurrentCoordinateSystem();
		SnapRequest.RequestType = ESceneSnapQueryType::RotationAngle;
		SnapRequest.AxisList = GetInteractionAxisList();
		SnapRequest.RotationAngle = InOutAngleDelta;

		TArray<FSceneSnapQueryResult> SnapResults;
		if (SnapManager->ExecuteSceneSnapQuery(SnapRequest, SnapResults))
		{
			check(!SnapResults.IsEmpty());

			InOutAngleDelta = SnapResults[0].RotationAngle;
		}
	}
}

void UTransformGizmo::SnapScaleDelta(FVector& InOutLocalScaleDelta) const
{
	UInteractiveGizmoManager* GizmoManager = GetGizmoManager();
	if (!GizmoManager)
	{
		return;
	}

	IToolsContextQueriesAPI* QueriesAPI = GizmoManager->GetContextQueriesAPI();
	if (!QueriesAPI)
	{
		return;
	}

	if (const USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GizmoManager))
	{
		FSceneSnapQueryRequest SnapRequest;

		// Scale should always be in local space
		SnapRequest.RequestCoordinateSpace = EToolContextCoordinateSystem::Local;
		SnapRequest.RequestType = ESceneSnapQueryType::Scale;
		SnapRequest.AxisList = GetInteractionAxisList();
		SnapRequest.Scale = InOutLocalScaleDelta;

		TArray<FSceneSnapQueryResult> SnapResults;
		if (SnapManager->ExecuteSceneSnapQuery(SnapRequest, SnapResults))
		{
			check(!SnapResults.IsEmpty());

			InOutLocalScaleDelta = SnapResults[0].Scale;
		}
	}
}

void UTransformGizmo::OnClickDragTranslateAxis(const FInputDeviceRay& DragPos)
{
	// indirect manipulation uses a 2D projection approach instead of plane intersection
	if (bIndirectManipulation)
	{
		const FVector2D DragDir = DragPos.ScreenPosition - InteractionScreenCurrPos;

		const FVector2D XAxisDir = GetScreenProjectedAxis(GizmoViewContext, FVector::XAxisVector, CurrentTransform);
		const FVector2D YAxisDir = GetScreenProjectedAxis(GizmoViewContext, FVector::YAxisVector, CurrentTransform);
		const FVector2D ZAxisDir = GetScreenProjectedAxis(GizmoViewContext, FVector::ZAxisVector, CurrentTransform);

		const float PixelToWorldRatio = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(GizmoViewContext, CurrentTransform.GetLocation(), true);

		const EAxisList::Type AxisList = GetInteractionAxisList();
		FVector Delta((AxisList == EAxisList::X) ? PixelToWorldRatio * FVector2D::DotProduct(XAxisDir, DragDir) : 0.0,
					  (AxisList == EAxisList::Y) ? PixelToWorldRatio * FVector2D::DotProduct(YAxisDir, DragDir) : 0.0,
					  (AxisList == EAxisList::Z) ? PixelToWorldRatio * FVector2D::DotProduct(ZAxisDir, DragDir) : 0.0);
		Delta = CurrentRotation * Delta;
		
		ApplyTranslateDelta(Delta);

		InteractionScreenCurrPos = DragPos.ScreenPosition;
		return;
	}

	FVector::FReal HitDepth;
	if (GetRayParamIntersectionWithInteractionPlane(DragPos, HitDepth))
	{
		FVector HitPoint = DragPos.WorldRay.Origin + DragPos.WorldRay.Direction * HitDepth;

		const FVector DeltaToStart = HitPoint-InteractionPlanarStartPoint;
		const FVector AxisToRemove = NormalToRemove * FVector::DotProduct(DeltaToStart, NormalToRemove);
		
		HitPoint -= AxisToRemove;
	
		FVector Delta = ComputePlanarTranslateDelta(InteractionPlanarCurrPoint, HitPoint);

		SnapTranslateDelta(Delta);
		ApplyTranslateDelta(Delta);

		InteractionPlanarCurrPoint += Delta;

		// Update Debug
		if (GizmoLocals::bDebugDraw)
		{
			DebugData.InteractionCurrent.SetLocation(InteractionPlanarCurrPoint);
		}
	}
}

void UTransformGizmo::OnClickReleaseTranslateAxis(const FInputDeviceRay& InReleasePos)
{
	bInInteraction = false;
}

void UTransformGizmo::OnClickPressTranslateXYPlanar(const FInputDeviceRay& PressPos)
{
	InteractionPlanarOrigin = CurrentTransform.GetLocation();
	InteractionPlanarNormal = GetWorldAxis(FVector::ZAxisVector);
	InteractionPlanarAxisX = GetWorldAxis(FVector::XAxisVector);
	InteractionPlanarAxisY = GetWorldAxis(FVector::YAxisVector);
	SetInteractionAxisList(EAxisList::XY);
	OnClickPressTranslatePlanar(PressPos);
}

void UTransformGizmo::OnClickPressTranslateYZPlanar(const FInputDeviceRay& PressPos)
{
	InteractionPlanarOrigin = CurrentTransform.GetLocation();
	InteractionPlanarNormal = GetWorldAxis(FVector::XAxisVector);
	InteractionPlanarAxisX = GetWorldAxis(FVector::YAxisVector);
	InteractionPlanarAxisY = GetWorldAxis(FVector::ZAxisVector);
	SetInteractionAxisList(EAxisList::YZ);
	OnClickPressTranslatePlanar(PressPos);
}

void UTransformGizmo::OnClickPressTranslateXZPlanar(const FInputDeviceRay& PressPos)
{
	InteractionPlanarOrigin = CurrentTransform.GetLocation();
	InteractionPlanarNormal = GetWorldAxis(FVector::YAxisVector);
	InteractionPlanarAxisX = GetWorldAxis(FVector::ZAxisVector);
	InteractionPlanarAxisY = GetWorldAxis(FVector::XAxisVector);
	SetInteractionAxisList(EAxisList::XZ);
	OnClickPressTranslatePlanar(PressPos);
}

void UTransformGizmo::OnClickPressPlanar(const FInputDeviceRay& PressPos)
{
	FVector::FReal HitDepth;
	if (GetRayParamIntersectionWithInteractionPlane(PressPos, HitDepth))
	{
		InteractionPlanarStartPoint = PressPos.WorldRay.Origin + PressPos.WorldRay.Direction * HitDepth;
		InteractionPlanarCurrPoint = InteractionPlanarStartPoint;
		bInInteraction = true;

		// Update Debug
		if (GizmoLocals::bDebugDraw)
		{
			DebugData.TransformStart.SetLocation(InteractionPlanarOrigin);
			DebugData.InteractionStart.SetLocation(InteractionPlanarStartPoint);
			DebugData.InteractionCurrent.SetLocation(InteractionPlanarStartPoint);
			DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
		}

		SetModeLastHitPart(EGizmoTransformMode::Translate, LastHitPart);
	}
}

void UTransformGizmo::OnClickPressTranslatePlanar(const FInputDeviceRay& PressPos)
{
	OnClickPressPlanar(PressPos);
}

void UTransformGizmo::OnClickDragTranslatePlanar(const FInputDeviceRay& DragPos)
{
	FVector::FReal HitDepth;
	if (GetRayParamIntersectionWithInteractionPlane(DragPos, HitDepth))
	{
		const FVector HitPoint = DragPos.WorldRay.Origin + DragPos.WorldRay.Direction * HitDepth;
		FVector Delta = ComputePlanarTranslateDelta(InteractionPlanarCurrPoint, HitPoint);
		SnapTranslateDelta(Delta);
		ApplyTranslateDelta(Delta);
		InteractionPlanarCurrPoint += Delta;
		
		// Update Debug
		if (GizmoLocals::bDebugDraw)
		{
			DebugData.InteractionCurrent.SetLocation(InteractionPlanarCurrPoint);
		}
	}
}

void UTransformGizmo::OnClickReleaseTranslatePlanar(const FInputDeviceRay& InReleasePos)
{
	bInInteraction = false;
}

FVector UTransformGizmo::ComputeAxisTranslateDelta(double InStartParam, double InEndParam)
{
	const double ParamDelta = InEndParam - InStartParam;
	return InteractionAxisDirection * ParamDelta;
}

FVector UTransformGizmo::ComputePlanarTranslateDelta(const FVector& InStartPoint, const FVector& InEndPoint)
{
	return InEndPoint - InStartPoint;
}

void UTransformGizmo::OnClickPressScreenSpaceTranslate(const FInputDeviceRay& PressPos)
{
	check(GizmoViewContext);

	InteractionPlanarOrigin = CurrentTransform.GetLocation();
	InteractionPlanarNormal = -GizmoViewContext->GetViewDirection();
	InteractionPlanarAxisX = GizmoViewContext->GetViewUp();
	InteractionPlanarAxisY = GizmoViewContext->GetViewRight();
	SetInteractionAxisList(EAxisList::Screen);
	OnClickPressPlanar(PressPos);
}

void UTransformGizmo::OnClickDragScreenSpaceTranslate(const FInputDeviceRay& DragPos)
{
	OnClickDragTranslatePlanar(DragPos);
}

void UTransformGizmo::OnClickReleaseScreenSpaceTranslate(const FInputDeviceRay& InReleasePos)
{
	bInInteraction = false;
}

void UTransformGizmo::OnClickPressScaleXAxis(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::X);
	OnClickPressScale(PressPos);
}

void UTransformGizmo::OnClickPressScaleYAxis(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::Y);
	OnClickPressScale(PressPos);
}

void UTransformGizmo::OnClickPressScaleZAxis(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::Z);
	OnClickPressScale(PressPos);
}

void UTransformGizmo::OnClickPressScaleXYPlanar(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::XY);
	OnClickPressScale(PressPos);
}

void UTransformGizmo::OnClickPressScaleYZPlanar(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::YZ);
	OnClickPressScale(PressPos);
}

void UTransformGizmo::OnClickPressScaleXZPlanar(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::XZ);
	OnClickPressScale(PressPos);
}

void UTransformGizmo::OnClickPressScaleXYZ(const FInputDeviceRay& PressPos)
{
	SetInteractionAxisList(EAxisList::XYZ);
	OnClickPressScale(PressPos);
}

void UTransformGizmo::OnClickPressScale(const FInputDeviceRay& PressPos)
{
	FVector2D AxisDir(0.0, 0.0);

	const EAxisList::Type AxisList = GetInteractionAxisList();
	if (AxisList & EAxisList::X)
	{
		AxisDir += GetScreenProjectedAxis(GizmoViewContext, FVector::XAxisVector, CurrentTransform);
	}
	if (AxisList & EAxisList::Y)
	{
		AxisDir += GetScreenProjectedAxis(GizmoViewContext, FVector::YAxisVector, CurrentTransform);
	}
	if (AxisList & EAxisList::Z)
	{
		AxisDir += GetScreenProjectedAxis(GizmoViewContext, FVector::ZAxisVector, CurrentTransform);
	}

	InteractionScreenAxisDirection = AxisDir.GetSafeNormal();
	InteractionScreenStartPos = InteractionScreenEndPos = InteractionScreenCurrPos = PressPos.ScreenPosition;
	bInInteraction = true;

	// Update Debug
	if (GizmoLocals::bDebugDraw)
	{
		FVector::FReal HitDepth;
		if (GetRayParamIntersectionWithInteractionPlane(PressPos, HitDepth))
		{
			const FVector HitPoint = PressPos.WorldRay.Origin + PressPos.WorldRay.Direction * HitDepth;
			DebugData.InteractionStart.SetLocation(HitPoint);
			DebugData.InteractionCurrent.SetLocation(HitPoint);
			DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
			DebugData.InteractionScreenDirection = InteractionScreenAxisDirection;
		}
	}

	SetModeLastHitPart(EGizmoTransformMode::Scale, LastHitPart);
}

void UTransformGizmo::OnClickDragScaleAxis(const FInputDeviceRay& DragPos)
{
	OnClickDragScale(DragPos);
}

void UTransformGizmo::OnClickDragScalePlanar(const FInputDeviceRay& DragPos)
{
	OnClickDragScale(DragPos);
}

void UTransformGizmo::OnClickDragScaleXYZ(const FInputDeviceRay& DragPos)
{
	OnClickDragScale(DragPos);
}

void UTransformGizmo::OnClickDragScale(const FInputDeviceRay& DragPos)
{
	FVector2D ScreenDelta = DragPos.ScreenPosition - InteractionScreenCurrPos;

	auto GetScaleType = [&]()
	{
		if (TransformGizmoSource)
		{
			return TransformGizmoSource->GetScaleType();
		}
		return EGizmoTransformScaleType::Default;
	};
	
	if (GetScaleType() != EGizmoTransformScaleType::PercentageBased)
	{
		ScreenDelta *= ScaleMultiplier;
	}

	InteractionScreenEndPos += ScreenDelta;

	FVector ScaleDelta = ComputeScaleDelta(InteractionScreenStartPos, InteractionScreenEndPos, ScreenDelta);
	SnapScaleDelta(ScaleDelta);

	if (ScaleDelta.X != 0.0 || ScaleDelta.Y != 0.0 || ScaleDelta.Z != 0.0)
	{
		ApplyScaleDelta(ScaleDelta);
		InteractionScreenEndPos -= ScreenDelta;
		InteractionScreenCurrPos = DragPos.ScreenPosition;
	}

	// Update Debug
	if (GizmoLocals::bDebugDraw)
	{
		FVector::FReal HitDepth;
		if (GetRayParamIntersectionWithInteractionPlane(DragPos, HitDepth))
		{
			const FVector HitPoint = DragPos.WorldRay.Origin + DragPos.WorldRay.Direction * HitDepth;
			DebugData.InteractionCurrent.SetLocation(HitPoint);
			DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
		}
	}
}

void UTransformGizmo::OnClickReleaseScaleAxis(const FInputDeviceRay& InReleasePos)
{
	UpdateAllScaleAxis();

	bInInteraction = false;
}

void UTransformGizmo::OnClickReleaseScalePlanar(const FInputDeviceRay& InReleasePos)
{
	UpdateAllScaleAxis();

	bInInteraction = false;
}

void UTransformGizmo::OnClickReleaseScaleXYZ(const FInputDeviceRay& InReleasePos)
{
	UpdateAllScaleAxis();

	bInInteraction = false;
}

FVector UTransformGizmo::ComputeScaleDelta(const FVector2D& InStartPos, const FVector2D& InEndPos, FVector2D& OutScreenDelta)
{
	const FVector2D DragDir = InEndPos - InStartPos;
	const double ScaleDelta = FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir);

	const EAxisList::Type AxisList = GetInteractionAxisList();
	const FVector Scale(
		(AxisList & EAxisList::X) ? ScaleDelta : 0.0f,
		(AxisList & EAxisList::Y) ? ScaleDelta : 0.0f,
		(AxisList & EAxisList::Z) ? ScaleDelta : 0.0f);

	const FVector::FReal ScaleMax = Scale.GetMax();
	const FVector::FReal ScaleMin = Scale.GetMin();
	const FVector::FReal ScaleApplied = (ScaleMax > -ScaleMin) ? ScaleMax : ScaleMin;

	OutScreenDelta = InteractionScreenAxisDirection * ScaleApplied;

	return Scale;
}

bool UTransformGizmo::OnClickPressRotateArc(
	const FInputDeviceRay& InPressPos, const FVector& InPlaneNormal,
	const FVector& InPlaneAxis1, const FVector& InPlaneAxis2, const EAxis::Type InAxis)
{
	const FRay& Ray = InPressPos.WorldRay;

	// compute axis / view direction projection: is the rotation plane nearly perpendicular to the view plane?
	const FVector WorldOrigin = CurrentTransform.GetLocation();
	const FVector ViewDirection = GizmoViewContext->IsPerspectiveProjection() ?
		(WorldOrigin - GizmoViewContext->ViewLocation).GetSafeNormal() :
		GizmoViewContext->GetViewDirection();
	const bool bAxisPerpendicularToView = FMath::Abs(FVector::DotProduct(InPlaneNormal, ViewDirection)) < GizmoLocals::DotThreshold;
	const bool bRayPerpendicularToAxis = FMath::IsNearlyZero(FVector::DotProduct(InPlaneNormal, Ray.Direction));

	// can we project
	const bool bCanProject = bIndirectManipulation || bAxisPerpendicularToView || bRayPerpendicularToAxis;
	if (!bCanProject)
	{
		InteractionPlanarOrigin = CurrentTransform.GetLocation();
		InteractionPlanarNormal = InPlaneNormal;
		InteractionPlanarAxisX = InPlaneAxis1;
		InteractionPlanarAxisY = InPlaneAxis2;
	
		FVector::FReal HitDepth;
		if (GetRayParamIntersectionWithInteractionPlane(InPressPos, HitDepth))
		{
			const FVector HitPoint = InPressPos.WorldRay.Origin + InPressPos.WorldRay.Direction * HitDepth;
			InteractionStartAngle = GizmoMath::ComputeAngleInPlane(HitPoint,
				InteractionPlanarOrigin, InteractionPlanarNormal, InteractionPlanarAxisX, InteractionPlanarAxisY);
			InteractionCurrAngle = InteractionStartAngle;
			bInInteraction = true;

			// Update Debug
			if (GizmoLocals::bDebugDraw)
			{
				DebugData.TransformStart.SetLocation(InteractionPlanarOrigin);
				DebugData.InteractionStart.SetLocation(HitPoint);
				DebugData.InteractionAngleStart = InteractionStartAngle;
				DebugData.InteractionAngleCurrent = InteractionCurrAngle;
				DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionStart.GetLocation());
				DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
			}

			return true;
		}
	}
	return false;
}

void UTransformGizmo::OnClickPressRotateAxis(const FInputDeviceRay& InPressPos)
{
	static const TArray RotateIDs({	ETransformGizmoPartIdentifier::RotateXAxis,
									ETransformGizmoPartIdentifier::RotateYAxis,
									ETransformGizmoPartIdentifier::RotateZAxis});

	const int32 RotateID = RotateIDs.IndexOfByKey(LastHitPart);
	if (!ensure(RotateID != INDEX_NONE))
	{
		bTrySwitchingToNormalPull = false;
		bInInteraction = false;
		return;
	}

	static const TArray AxisList({EAxisList::X, EAxisList::Y, EAxisList::Z});

	DebugData.bDebugRotate = true;
	
	// initialize pull data
	InteractionScreenAxisDirection = GetScreenRotateAxisDir(InPressPos);
	SetInteractionAxisList(AxisList[RotateID]);
	InteractionScreenStartPos = InteractionScreenCurrPos = InPressPos.ScreenPosition;
	RotateMode = EAxisRotateMode::Pull;

	// initialize arc/mixed data
	const bool bRotatePull = bIndirectManipulation || DefaultRotateMode == EAxisRotateMode::Pull;
	if (!bRotatePull)
	{
		static const TArray RotateAxis({FVector::XAxisVector, FVector::YAxisVector, FVector::ZAxisVector});
		const FVector WorldPlaneNormal = GetWorldAxis(RotateAxis[RotateID]);
		const FVector WorldPlaneAxis1 = GetWorldAxis(RotateAxis[(RotateID+1) % 3]);
		const FVector WorldPlaneAxis2 = GetWorldAxis(RotateAxis[(RotateID+2) % 3]);
		
		const bool bCanRotateArc = OnClickPressRotateArc(InPressPos, WorldPlaneNormal, WorldPlaneAxis1, WorldPlaneAxis2);
		if (bCanRotateArc)
		{
			RotateMode = EAxisRotateMode::Arc;
		}
	}

	bTrySwitchingToNormalPull = bIndirectManipulation && RotateMode == EAxisRotateMode::Pull;
	
	bInInteraction = true;
	SetModeLastHitPart(EGizmoTransformMode::Rotate, LastHitPart);
}

void UTransformGizmo::OnClickPressRotateXAxis(const FInputDeviceRay& InPressPos)
{
	OnClickPressRotateAxis(InPressPos);
}

void UTransformGizmo::OnClickPressRotateYAxis(const FInputDeviceRay& InPressPos)
{
	OnClickPressRotateAxis(InPressPos);
}

void UTransformGizmo::OnClickPressRotateZAxis(const FInputDeviceRay& InPressPos)
{
	OnClickPressRotateAxis(InPressPos);
}

FVector2D UTransformGizmo::GetWorldToScreenRotateAxisDir(const FInputDeviceRay& InPressPos, const FVector& InWorldAxis)
{
	// NOTE that function is not intended to remain here indefinitely, its purpose is to debug closest point computation
	const auto PrintProjection = [this](const TCHAR* const InMessage)
	{
		if (DebugData.bDebugRotate && GizmoLocals::bDebugDraw)
		{
			UE_LOGF(LogTransformGizmo, Warning, "%ls", InMessage);
		}
	};
	
	const FRay& Ray = InPressPos.WorldRay;

	// store world origin and axis
	const FVector WorldOrigin = CurrentTransform.GetLocation();

	// compute axis / view direction projection: is the rotation plane nearly perpendicular to the view plane?
	const FVector ViewDirection = GizmoViewContext->IsPerspectiveProjection() ?
		(WorldOrigin - GizmoViewContext->ViewLocation).GetSafeNormal() :
		GizmoViewContext->GetViewDirection();
	const bool bAxisPerpendicularToView = FMath::Abs(FVector::DotProduct(InWorldAxis, ViewDirection)) < GizmoLocals::DotThreshold;
	// compute axis / ray direction projection: is the ray direction parallel to the axis?
	const bool bRayPerpendicularToAxis = FMath::IsNearlyZero(FVector::DotProduct(InWorldAxis, Ray.Direction));

	// compute closest point on the rotate handle
	const bool bUseRayForIndirect = bIndirectManipulation && !GizmoLocals::ProjectIndirect;
	const bool bUseRayOrigin = bUseRayForIndirect || bAxisPerpendicularToView || bRayPerpendicularToAxis;
	// compute the closest point from plane intersection if we can
	FVector QueryPoint = Ray.Origin;
	if (!bUseRayOrigin)
	{
		const FPlane Plane(WorldOrigin, InWorldAxis);

		// if the projection is in front of the camera then use it
		const double HitDepth = FMath::RayPlaneIntersectionParam(Ray.Origin, Ray.Direction, Plane);
		if (HitDepth >= 0.0)
		{
			PrintProjection(TEXT("front"));
			QueryPoint = Ray.Origin + Ray.Direction * HitDepth;
		}
		else
		{
			PrintProjection(TEXT("behind"));
		}
	}
	else
	{
		PrintProjection(TEXT("ray origin"));
	}

	// compute nearest point
	const float Radius = GetWorldRadius(Style.RotateAxisRadius + Style.RotateAxisOuterRadiusOffset);
	FVector ClosestPointOnCircle;
	GizmoMath::ClosetPointOnCircle(QueryPoint, WorldOrigin, InWorldAxis, Radius, ClosestPointOnCircle);

	// compute world directions
	const FVector ToClosestDirection = (ClosestPointOnCircle - WorldOrigin).GetSafeNormal();
	const FVector PullDirection = FVector::CrossProduct(ToClosestDirection, InWorldAxis);

	// compute screen projections
	const FTransform ToClosest(ClosestPointOnCircle);
	const FVector2D PullProjection = GetScreenProjectedAxis(GizmoViewContext, PullDirection, ToClosest);
	const FVector2D AxisProjection = GetScreenProjectedAxis(GizmoViewContext, InWorldAxis, ToClosest);
	const FVector2D ToClosestProjection = GetScreenProjectedAxis(GizmoViewContext, ToClosestDirection, ToClosest);

	// compute which projection to remove from drag
	const double DotAxis = FMath::Abs( FVector2D::DotProduct(PullProjection, AxisProjection) );
	const double DotClosest = FMath::Abs( FVector2D::DotProduct(PullProjection, ToClosestProjection) );
	NormalProjectionToRemove = (DotAxis < DotClosest) ? AxisProjection : ToClosestProjection;

	// Update Debug
	if (GizmoLocals::bDebugDraw)
	{
		DebugData.InteractionStart.SetLocation(QueryPoint);
		DebugData.DebugNormalRemoved = (DotAxis < DotClosest) ? InWorldAxis : ToClosestDirection;
        DebugData.DebugNormalSkip = (DotAxis < DotClosest) ? ToClosestDirection : InWorldAxis;
        DebugData.DebugDirection = PullDirection;
        DebugData.DebugClosest = ClosestPointOnCircle;
		DebugData.InteractionRadius = Radius;
	}

	return PullProjection;
}

FVector2D UTransformGizmo::GetScreenRotateAxisDir(const FInputDeviceRay& InPressPos)
{
	static const TArray RotateIDs({	ETransformGizmoPartIdentifier::RotateXAxis,
									ETransformGizmoPartIdentifier::RotateYAxis,
									ETransformGizmoPartIdentifier::RotateZAxis,
									ETransformGizmoPartIdentifier::RotateScreenSpace});
	const int32 RotateID = RotateIDs.IndexOfByKey(LastHitPart);
	if (!ensure(RotateID != INDEX_NONE))
	{
		return FVector2D::ZeroVector;
	}

	// store world origin and axis
	static const TArray RotateAxis({FVector::XAxisVector, FVector::YAxisVector, -FVector::ZAxisVector});
	const FVector WorldAxis = RotateAxis.IsValidIndex(RotateID) ? GetWorldAxis(RotateAxis[RotateID]) : GizmoViewContext->GetViewDirection();

	return GetWorldToScreenRotateAxisDir(InPressPos, WorldAxis);
}

FVector2D UTransformGizmo::GetScreenGimbalRotateAxisDir(const FInputDeviceRay& InPressPos)
{
	static const TArray RotateIDs({	ETransformGizmoPartIdentifier::RotateXGimbal,
									ETransformGizmoPartIdentifier::RotateYGimbal,
									ETransformGizmoPartIdentifier::RotateZGimbal});
	const int32 RotateID = RotateIDs.IndexOfByKey(LastHitPart);
	if (!ensure(RotateID != INDEX_NONE))
	{
		return FVector2D::ZeroVector;
	}
	
	return GetWorldToScreenRotateAxisDir(InPressPos, GetGimbalRotationAxis(RotateID));
}

void UTransformGizmo::OnClickDragRotateAxis(const FInputDeviceRay& DragPos)
{
	switch (RotateMode)
	{
	case EAxisRotateMode::Pull:
	{
		double DeltaAngle = FMath::DegreesToRadians(ComputeAxisRotateDeltaAngle(InteractionScreenCurrPos, DragPos));
		SnapRotateAngleDelta(DeltaAngle);

		const FQuat DeltaRot = ComputeAxisRotateDelta(FMath::RadiansToDegrees(DeltaAngle));

		ApplyRotateDelta(DeltaRot);

		InteractionScreenCurrPos = DragPos.ScreenPosition;

		// Update Debug
		if (GizmoLocals::bDebugDraw)
		{
			FVector::FReal HitDepth;
			GetRayParamIntersectionWithInteractionPlane(DragPos, HitDepth);

			const FVector HitPoint = DragPos.WorldRay.Origin + DragPos.WorldRay.Direction * HitDepth;
			DebugData.InteractionCurrent.SetLocation(HitPoint);

			const double HitAngle = InteractionCurrAngle + DeltaAngle;
			DebugData.InteractionAngleCurrent += FMath::FindDeltaAngleRadians(DebugData.InteractionAngleCurrent, HitAngle);
			DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());

			DebugData.InteractionPlaneNormal = DeltaRot.GetRotationAxis();
		}

		break;
	}
	case EAxisRotateMode::Arc:
	case EAxisRotateMode::ScreenArc:
	{
		FVector::FReal HitDepth;
		if (GetRayParamIntersectionWithInteractionPlane(DragPos, HitDepth))
		{
			const FVector HitPoint = DragPos.WorldRay.Origin + DragPos.WorldRay.Direction * HitDepth;
			double HitAngle = GizmoMath::ComputeAngleInPlane(HitPoint,
				InteractionPlanarOrigin, InteractionPlanarNormal, InteractionPlanarAxisX, InteractionPlanarAxisY);

			double HitAngleDelta = HitAngle - InteractionCurrAngle;
			SnapRotateAngleDelta(HitAngleDelta);

			// Reapply the (potentially) snapped delta angle
			HitAngle = InteractionCurrAngle + HitAngleDelta;

			FQuat Delta = ComputeAngularRotateDelta(InteractionCurrAngle, HitAngle);

			ApplyRotateDelta(Delta);

			InteractionCurrAngle = HitAngle;
			
			// Update Debug
			if (GizmoLocals::bDebugDraw)
			{
				DebugData.InteractionCurrent.SetLocation(HitPoint);
				DebugData.InteractionAngleCurrent += FMath::FindDeltaAngleRadians(DebugData.InteractionAngleCurrent, HitAngle);
				DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
				DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
			}
		}
		break;
	}
	default:
		ensure(false);
		break;
	}
}

double UTransformGizmo::ComputeAxisRotateDeltaAngle(const FVector2D& InStartPos, const FInputDeviceRay& InDragPos)
{
	FVector2D DragDir = InDragPos.ScreenPosition - InStartPos;

	if (bTrySwitchingToNormalPull)
	{
		const double DotTangent = FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir);
		const double DotNormal = FVector2D::DotProduct(NormalProjectionToRemove, DragDir);
		if (FMath::Abs(DotNormal) > FMath::Abs(DotTangent))
		{
			::Swap(NormalProjectionToRemove, InteractionScreenAxisDirection);
			::Swap(DebugData.DebugDirection, DebugData.DebugNormalRemoved);
			InteractionScreenAxisDirection *= FMath::Sign(DotTangent) * FMath::Sign(DotNormal);
		}
		bTrySwitchingToNormalPull = false;
	}

	const FVector2D DragDirToRemove = NormalProjectionToRemove * FVector2D::DotProduct(DragDir, NormalProjectionToRemove);
	DragDir -= DragDirToRemove;

	return FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir) * 0.25;
}

FQuat UTransformGizmo::ComputeAxisRotateDelta(const double& InDeltaAngle)
{
	const EAxisList::Type AxisList = GetInteractionAxisList();

	double DeltaAngle = InDeltaAngle;
	if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward && AxisList == EAxisList::Left)
	{
		DeltaAngle *= -1;
	}

	FRotator DeltaRot(
		AxisList == EAxisList::Y || AxisList == EAxisList::Left ? DeltaAngle : 0.0,
		AxisList == EAxisList::Z || AxisList == EAxisList::Up ? DeltaAngle : 0.0,
		AxisList == EAxisList::X || AxisList == EAxisList::Forward ? DeltaAngle : 0.0);

	auto GetCoordinateSystem = [&]()
	{
		if (TransformGizmoSource)
		{
			return TransformGizmoSource->GetGizmoCoordSystemSpace(); 
		}
		return GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem();
	};
	
	if (GetCoordinateSystem() == EToolContextCoordinateSystem::Local)
	{
		check(ActiveTarget);
		const FMatrix CurrCoordSystem = ActiveTarget->GetTransform().ToMatrixNoScale();
		DeltaRot = (CurrCoordSystem.Inverse() * FRotationMatrix(DeltaRot) * CurrCoordSystem).Rotator();
	}

	return DeltaRot.Quaternion();
}

FQuat UTransformGizmo::ComputeGimbalRotateDelta(const FVector2D& InStartPos, const FVector2D& InEndPos, const double InAngleSign)
{
	FVector2D DragDir = InEndPos - InStartPos;

	if (bTrySwitchingToNormalPull)
	{
		const double DotTangent = FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir);
		const double DotNormal = FVector2D::DotProduct(NormalProjectionToRemove, DragDir);
		if (FMath::Abs(DotNormal) > FMath::Abs(DotTangent))
		{
			::Swap(NormalProjectionToRemove, InteractionScreenAxisDirection);
			::Swap(DebugData.DebugDirection, DebugData.DebugNormalRemoved);
			InteractionScreenAxisDirection *= FMath::Sign(DotTangent) * FMath::Sign(DotNormal);
		}
		bTrySwitchingToNormalPull = false;
	}
	
	const FVector2D DragDirToRemove = NormalProjectionToRemove * FVector2D::DotProduct(DragDir, NormalProjectionToRemove);
	DragDir -= DragDirToRemove;

	const double Delta = FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir) * 0.25;
	
	const EAxisList::Type AxisList = GetInteractionAxisList();
	const FVector DeltaRot(
	 	   AxisList == EAxisList::X || AxisList == EAxisList::Forward ? Delta : 0.0,
	 	   AxisList == EAxisList::Y || AxisList == EAxisList::Left ? Delta : 0.0,
	 	   AxisList == EAxisList::Z || AxisList == EAxisList::Up ? Delta : 0.0);
	
	static constexpr bool bUseUEHandyness = true;
	return AnimationCore::QuatFromEuler(DeltaRot, GetRotationContext().RotationOrder, bUseUEHandyness);
}

void UTransformGizmo::OnClickReleaseRotateAxis(const FInputDeviceRay& InReleasePos)
{
	bInInteraction = false;
	DebugData.bDebugRotate = false;
	bTrySwitchingToNormalPull = false;
}

void UTransformGizmo::OnClickPressGimbalRotateAxis(const FInputDeviceRay& InPressPos)
{
	static const TArray RotateIDs({	ETransformGizmoPartIdentifier::RotateXGimbal,
									ETransformGizmoPartIdentifier::RotateYGimbal,
									ETransformGizmoPartIdentifier::RotateZGimbal});

	const int32 RotateID = RotateIDs.IndexOfByKey(LastHitPart);
	if (!ensure(RotateID != INDEX_NONE))
	{
		bTrySwitchingToNormalPull = false;
		bInInteraction = false;
		return;
	}

	static const TArray AxisList({EAxisList::X, EAxisList::Y, EAxisList::Z});
	
	DebugData.bDebugRotate = true;
	
	// initialize pull data
	InteractionScreenAxisDirection = GetScreenGimbalRotateAxisDir(InPressPos);
	InteractionAxisDirection = GetGimbalRotationAxis(RotateID);
	SetInteractionAxisList(AxisList[RotateID]);
	InteractionScreenStartPos = InteractionScreenCurrPos = InPressPos.ScreenPosition;
	RotateMode = EAxisRotateMode::Pull;
	
	// initialize arc/mixed data
	const bool bRotatePull = bIndirectManipulation || DefaultRotateMode == EAxisRotateMode::Pull;
	if (!bRotatePull)
	{		
		const FVector WorldPlaneNormal = GetGimbalRotationAxis(RotateID);

		FVector WorldPlaneAxis1, WorldPlaneAxis2;
		UE::Geometry::VectorUtil::MakePerpVectors(WorldPlaneNormal, WorldPlaneAxis1, WorldPlaneAxis2);
		
		const bool bCanRotateArc = OnClickPressRotateArc(InPressPos, WorldPlaneNormal, WorldPlaneAxis1, WorldPlaneAxis2);
		if (bCanRotateArc)
		{
			RotateMode = EAxisRotateMode::Arc;
		}
	}

	bTrySwitchingToNormalPull = bIndirectManipulation && RotateMode == EAxisRotateMode::Pull;

	bInInteraction = true;
	SetModeLastHitPart(EGizmoTransformMode::Rotate, LastHitPart);
}

void UTransformGizmo::OnClickDragGimbalRotateAxis(const FInputDeviceRay& InDragPos)
{
	static const TArray RotateIDs({	ETransformGizmoPartIdentifier::RotateXGimbal,
									ETransformGizmoPartIdentifier::RotateYGimbal,
									ETransformGizmoPartIdentifier::RotateZGimbal});

	const int32 RotateID = RotateIDs.IndexOfByKey(LastHitPart);
	if (!ensure(RotateID != INDEX_NONE))
	{
		return;
	}

	switch (RotateMode)
	{
	case EAxisRotateMode::Pull:
		{
			FQuat DeltaRot = ComputeGimbalRotateDelta(InteractionScreenCurrPos, InDragPos.ScreenPosition);
			SnapRotateDelta(DeltaRot);
			ApplyRotateDelta(DeltaRot);
			InteractionScreenCurrPos = InDragPos.ScreenPosition;

			// Update Debug
			if (GizmoLocals::bDebugDraw)
			{
				FVector::FReal HitDepth;
				if (GetRayParamIntersectionWithInteractionPlane(InDragPos, HitDepth))
				{
					const FVector HitPoint = InDragPos.WorldRay.Origin + InDragPos.WorldRay.Direction * HitDepth;
					DebugData.InteractionCurrent.SetLocation(HitPoint);
				}
				
				DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
				DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
			}
			
			break;
		}
	case EAxisRotateMode::Arc:
	case EAxisRotateMode::ScreenArc:
		{
			FVector::FReal HitDepth;
			if (GetRayParamIntersectionWithInteractionPlane(InDragPos, HitDepth))
			{
				const FVector HitPoint = InDragPos.WorldRay.Origin + InDragPos.WorldRay.Direction * HitDepth;
				double HitAngle = GizmoMath::ComputeAngleInPlane(HitPoint,
					InteractionPlanarOrigin, InteractionPlanarNormal, InteractionPlanarAxisX, InteractionPlanarAxisY);

				double HitAngleDelta = HitAngle - InteractionCurrAngle;
				SnapRotateAngleDelta(HitAngleDelta);

				// Reapply the (potentially) snapped delta angle
				HitAngle = InteractionCurrAngle + HitAngleDelta;

				const double DeltaAngle = FMath::RadiansToDegrees(InteractionCurrAngle - HitAngle);

				const EAxisList::Type AxisList = GetInteractionAxisList();
				const FVector DeltaRot(
					AxisList == EAxisList::X ? DeltaAngle : 0.0,
					AxisList == EAxisList::Y ? DeltaAngle : 0.0,
					AxisList == EAxisList::Z ? DeltaAngle : 0.0);
	
				static constexpr bool bUseUEHandyness = true;
				const FQuat Delta = AnimationCore::QuatFromEuler(DeltaRot, GetRotationContext().RotationOrder, bUseUEHandyness);

				ApplyRotateDelta(Delta);
				InteractionCurrAngle = HitAngle;
				
				// Update Debug
				if (GizmoLocals::bDebugDraw)
				{
					DebugData.InteractionCurrent.SetLocation(HitPoint);
					DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
					DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
				}
			}
			break;
		}
	default:
		ensure(false);
		break;
	}
}

void UTransformGizmo::OnClickPressScreenSpaceRotate(const FInputDeviceRay& InPressPos)
{
	check(GizmoViewContext);
	
	InteractionPlanarOrigin = CurrentTransform.GetLocation();
	InteractionPlanarNormal = -GizmoViewContext->GetViewDirection();
	InteractionPlanarAxisX = GizmoViewContext->GetViewUp();
	InteractionPlanarAxisY = GizmoViewContext->GetViewRight();
	SetInteractionAxisList(EAxisList::Screen);

	bTrySwitchingToNormalPull = false;
	bInInteraction = false;
	DebugData.bDebugRotate = false;

	if (bIndirectManipulation)
	{
		InteractionScreenAxisDirection = GetScreenRotateAxisDir(InPressPos);
		InteractionScreenStartPos = InteractionScreenCurrPos = InPressPos.ScreenPosition;
		bTrySwitchingToNormalPull = true;
		DebugData.bDebugRotate = true;
	}
	else
	{
		FVector::FReal HitDepth;
		if (!GetRayParamIntersectionWithInteractionPlane(InPressPos, HitDepth))
		{
			return;
		}
		const FVector HitPoint = InPressPos.WorldRay.Origin + InPressPos.WorldRay.Direction * HitDepth;
		InteractionStartAngle = GizmoMath::ComputeAngleInPlane(HitPoint,
			InteractionPlanarOrigin, InteractionPlanarNormal, InteractionPlanarAxisX, InteractionPlanarAxisY);
		InteractionCurrAngle = InteractionStartAngle;

		// Update Debug
		if (GizmoLocals::bDebugDraw)
		{
			DebugData.InteractionStart.SetLocation(HitPoint);
			DebugData.InteractionAngleStart = InteractionStartAngle;
			DebugData.InteractionAngleCurrent = InteractionCurrAngle;
			DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
			DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
		}
	}

	bInInteraction = true;
	SetModeLastHitPart(EGizmoTransformMode::Rotate, LastHitPart);
}

void UTransformGizmo::OnClickDragScreenSpaceRotate(const FInputDeviceRay& InDragPos)
{
	check(GizmoViewContext);

	if (bIndirectManipulation)
	{
		FVector2D DragDir = InDragPos.ScreenPosition - InteractionScreenCurrPos;

		if (bTrySwitchingToNormalPull)
		{
			const double DotTangent = FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir);
			const double DotNormal = FVector2D::DotProduct(NormalProjectionToRemove, DragDir);
			if (FMath::Abs(DotNormal) > FMath::Abs(DotTangent))
			{
				::Swap(NormalProjectionToRemove, InteractionScreenAxisDirection);
				::Swap(DebugData.DebugDirection, DebugData.DebugNormalRemoved);
				InteractionScreenAxisDirection *= FMath::Sign(DotTangent) * FMath::Sign(DotNormal);
			}
			bTrySwitchingToNormalPull = false;
		}
	
		const FVector2D DragDirToRemove = NormalProjectionToRemove * FVector2D::DotProduct(DragDir, NormalProjectionToRemove);
		DragDir -= DragDirToRemove;

		double DeltaAngle = FMath::DegreesToRadians(FVector2D::DotProduct(InteractionScreenAxisDirection, DragDir));
		SnapRotateAngleDelta(DeltaAngle);

		const FQuat DeltaRotAxis(InteractionPlanarNormal, DeltaAngle);

		ApplyRotateDelta(DeltaRotAxis);

		InteractionScreenCurrPos = InDragPos.ScreenPosition;

		return;
	}

	FVector::FReal HitDepth;
	if (GetRayParamIntersectionWithInteractionPlane(InDragPos, HitDepth))
	{
		const FVector HitPoint = InDragPos.WorldRay.Origin + InDragPos.WorldRay.Direction * HitDepth;
		double HitAngle = GizmoMath::ComputeAngleInPlane(HitPoint,
			InteractionPlanarOrigin, InteractionPlanarNormal, InteractionPlanarAxisX, InteractionPlanarAxisY);

		double HitAngleDelta = HitAngle - InteractionCurrAngle;
		SnapRotateAngleDelta(HitAngleDelta);

		// Reapply the (potentially) snapped delta angle
		HitAngle = InteractionCurrAngle + HitAngleDelta;

		const FQuat Delta = ComputeAngularRotateDelta(InteractionCurrAngle, HitAngle);

		ApplyRotateDelta(Delta);

		InteractionCurrAngle = HitAngle;
		
		// Update Debug
		if (GizmoLocals::bDebugDraw)
		{
			//DebugData.TransformStart.SetLocation(HitPoint);
			DebugData.InteractionCurrent.SetLocation(HitPoint);
			DebugData.InteractionAngleCurrent += FMath::FindDeltaAngleRadians(DebugData.InteractionAngleCurrent, HitAngle);
			DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
		}
	}
}

FQuat UTransformGizmo::ComputeAngularRotateDelta(FQuat::FReal InStartAngle, FQuat::FReal InEndAngle)
{
	const FQuat::FReal DeltaAngle = InEndAngle - InStartAngle;
	return FQuat(InteractionPlanarNormal, DeltaAngle);
}

void UTransformGizmo::OnClickReleaseScreenSpaceRotate(const FInputDeviceRay& InReleasePos)
{
	bInInteraction = false;
	DebugData.bDebugRotate = false;
	bTrySwitchingToNormalPull = false;
}

namespace ArcBallLocals
{	
	/* See Holroyd's implementation that mixes a sphere + and hyperbola to avoid popping
	 * Knud Henriksen, Jon Sporring, and Kasper Hornbaek, “Virtual trackballs revisited”,
	 * IEEE Transactions on Visualization and Computer Graphics, vol. 10, no. 2, pp. 206–216, 2004.
	 */
	bool GetSphereAndHyperbolicProjection(
		const FVector& SphereOrigin,
		const double SphereRadius,
		const FVector& RayOrigin,
		const FVector& RayDirection,
		const UGizmoViewContext& ViewContext,
		FVector& OutProjection)
	{
		const FVector CircleNormal = -ViewContext.GetViewDirection();
		
		// if ray is parallel to circle, no hit
		if (FMath::IsNearlyZero(FVector::DotProduct(CircleNormal, RayDirection)))
		{
			return false;
		}

		const FPlane Plane(SphereOrigin, CircleNormal);
		const double Param = FMath::RayPlaneIntersectionParam(RayOrigin, RayDirection, Plane);

		if (Param < 0)
		{
			return false;
		}

		const FVector HitPoint = RayOrigin + RayDirection * Param;
		FVector Offset = HitPoint - SphereOrigin;

		// switch to screen space
		FVector OffsetProjection = ViewContext.ViewMatrices.GetInvViewMatrix().InverseTransformVector(Offset);

		const double OffsetSquared = Offset.SizeSquared();
		const double CircleRadiusSquared = SphereRadius * SphereRadius;
		
		OffsetProjection.Z = (OffsetSquared <= CircleRadiusSquared * 0.5) ?
			-FMath::Sqrt(CircleRadiusSquared - OffsetSquared) : // spherical projection
			-CircleRadiusSquared * 0.5 / Offset.Length(); // hyperbolic projection

		// switch back to world space
		Offset = ViewContext.ViewMatrices.GetInvViewMatrix().TransformVector(OffsetProjection);
		OutProjection = SphereOrigin + Offset;
		
		return true;
	}
}

void UTransformGizmo::OnClickPressArcBallRotate(const FInputDeviceRay& PressPos)
{
	if (bGimbalRotationMode)
	{
		// arc ball rotation is disabled in gimbal mode but is still rendered to ease the visualization 
		SetModeLastHitPart(EGizmoTransformMode::Rotate, LastHitPart);
		bInInteraction = false;
		return;
	}
	
	check(GizmoViewContext);
	
	const FVector& RayOrigin = PressPos.WorldRay.Origin;
	const FVector& RayDir = PressPos.WorldRay.Direction;		
	const double SphereRadius = GetWorldRadius(Style.RotateArcballSphereRadius);

	StartRotation = CurrentRotation = CurrentTransform.GetRotation();
	InteractionPlanarOrigin = CurrentTransform.GetLocation();
	InteractionPlanarNormal = -GizmoViewContext->GetViewDirection();
	InteractionPlanarAxisX = GizmoViewContext->GetViewUp();
	InteractionPlanarAxisY = GizmoViewContext->GetViewRight();
	SetInteractionAxisList(EAxisList::XYZ);

	auto NeedsInteraction = [&]()
	{
		const bool bIntersect = IntersectionUtil::RaySphereTest(RayOrigin, RayDir, InteractionPlanarOrigin, SphereRadius);
		if (bIntersect)
		{
			return true;
		}
		
		if (bIndirectManipulation)
		{
			// change the arc ball center in indirect manipulation if we didn't hit the sphere
			FVector::FReal HitDepth;
			if (GetRayParamIntersectionWithInteractionPlane(PressPos, HitDepth))
			{
				InteractionPlanarOrigin = PressPos.WorldRay.Origin + PressPos.WorldRay.Direction * HitDepth;
				return true;
			}
		}
		
		return false;
	};

	if (NeedsInteraction())
	{
		// project on sphere
		ArcBallLocals::GetSphereAndHyperbolicProjection(
			InteractionPlanarOrigin, SphereRadius, RayOrigin, RayDir, *GizmoViewContext,
			InteractionArcBallStartPoint);

		// Update Debug
		if (GizmoLocals::bDebugDraw)
		{
			DebugData.TransformStart.SetLocation(InteractionPlanarOrigin);
			DebugData.InteractionStart.SetLocation(InteractionArcBallStartPoint);
			DebugData.InteractionCurrent.SetLocation(InteractionArcBallStartPoint);
			DebugData.InteractionPlaneNormal = InteractionPlanarNormal;
			DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
		}
	
		bInInteraction = true;
	
		SetModeLastHitPart(EGizmoTransformMode::Rotate, LastHitPart);
	}
}

void UTransformGizmo::OnClickDragArcBallRotate(const FInputDeviceRay& DragPos)
{
	const FVector& RayOrigin = DragPos.WorldRay.Origin;
	const FVector& RayDir = DragPos.WorldRay.Direction;		
	const float SphereRadius = GetWorldRadius(Style.RotateArcballSphereRadius);

	// compute projection
	ArcBallLocals::GetSphereAndHyperbolicProjection(
		InteractionPlanarOrigin, SphereRadius, RayOrigin, RayDir, *GizmoViewContext,
		InteractionArcBallCurrPoint);

	if ((InteractionArcBallCurrPoint-InteractionArcBallStartPoint).Length() <= 0.0)
	{
		return;
	}

	// compute rotation
	const FVector Axis1 = (InteractionArcBallCurrPoint - InteractionPlanarOrigin).GetSafeNormal();
	const FVector Axis0 = (InteractionArcBallStartPoint - InteractionPlanarOrigin).GetSafeNormal();
	
	const FQuat DeltaQ = FQuat::FindBetweenNormals(Axis0, Axis1);
	
	// apply rotation
	const FQuat FinalRotation = DeltaQ * StartRotation;
	const FQuat InvCurrentRot = CurrentRotation.Inverse();
	
	FQuat DeltaRot = (FinalRotation * InvCurrentRot).GetNormalized();
	if (!DeltaRot.IsIdentity())
	{
		SnapRotateDelta(DeltaRot);
		ApplyRotateDelta(DeltaRot);
		CurrentRotation = DeltaRot * CurrentRotation; 
	}

	// Update Debug
	if (GizmoLocals::bDebugDraw)
	{
		DebugData.InteractionCurrent.SetLocation(InteractionArcBallCurrPoint);
		DebugData.InteractionRadius = FVector::Distance(DebugData.TransformStart.GetLocation(), DebugData.InteractionCurrent.GetLocation());
		DebugData.InteractionPlaneNormal = FVector::CrossProduct(Axis1, Axis0);
	}
}

void UTransformGizmo::OnClickReleaseArcBallRotate(const FInputDeviceRay& ReleasePos)
{
	bInInteraction = false;
}

float UTransformGizmo::GetWorldRadius(const float InRadius) const
{
	const float PixelToWorldScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(GizmoViewContext, CurrentTransform.GetLocation(), true);
	const float GizmoScale = TransformGizmoSource ? TransformGizmoSource->GetGizmoScale() : 1.0f;
	return InRadius * GetSizeCoefficient() * PixelToWorldScale * GizmoScale;
}

float UTransformGizmo::GetSizeCoefficient() const
{
	return CustomizationFunction ? CustomizationFunction().SizeCoefficient : 1.5f;
}

FVector2D UTransformGizmo::GetScreenProjectedAxis(const UGizmoViewContext* View, const FVector& InLocalAxis, const FTransform& InLocalToWorld)
{
	FVector2D Origin;
	FVector2D AxisEnd;
	const FVector WorldOrigin = InLocalToWorld.GetTranslation();
	const FVector WorldAxisEnd = WorldOrigin + InLocalToWorld.TransformVectorNoScale(InLocalAxis * 64.0);

	if (View->ScreenToPixel(View->WorldToScreen(WorldOrigin), Origin) &&
		View->ScreenToPixel(View->WorldToScreen(WorldAxisEnd), AxisEnd))
	{
		// If both the origin and the axis endpoint are in front of the camera, trivially calculate the viewport space axis direction
		return (AxisEnd - Origin).GetSafeNormal();
	}
	
	// If either the origin or axis endpoint are behind the camera, translate the entire widget in front of the camera in the view direction before performing the
	// viewport space calculation
	const FMatrix InvViewMatrix = View->ViewMatrices.GetInvViewMatrix();
	const FVector ViewLocation = InvViewMatrix.GetOrigin();
	const FVector ViewDirection = InvViewMatrix.GetUnitAxis(EAxis::Z);
	const FVector Offset = ViewDirection * (FVector::DotProduct(ViewLocation - WorldOrigin, ViewDirection) + 100.0f);
	const FVector AdjustedWidgetOrigin = WorldOrigin + Offset;
	const FVector AdjustedWidgetAxisEnd = WorldAxisEnd + Offset;

	View->ScreenToPixel(View->WorldToScreen(AdjustedWidgetOrigin), Origin);
	View->ScreenToPixel(View->WorldToScreen(AdjustedWidgetAxisEnd), AxisEnd);
	return -(AxisEnd - Origin).GetSafeNormal();	
}

void UTransformGizmo::ApplyTranslateDelta(const FVector& InTranslateDelta)
{
	CurrentTransform.AddToTranslation(InTranslateDelta);
	ActiveTarget->SetTransform(CurrentTransform);
}

void UTransformGizmo::ApplyRotateDelta(const FQuat& InRotateDelta)
{
	// Applies rot delta after the current rotation
	const FQuat NewRotation = InRotateDelta * CurrentTransform.GetRotation();
	CurrentTransform.SetRotation(NewRotation);
	ActiveTarget->SetTransform(CurrentTransform);
}

void UTransformGizmo::ApplyScaleDelta(const FVector& InScaleDelta)
{
	const FVector StartScale = CurrentTransform.GetScale3D();
	const FVector NewScale = StartScale + InScaleDelta;
	CurrentTransform.SetScale3D(NewScale);
	ActiveTarget->SetTransform(CurrentTransform);
}

FOnGizmoInteraction& UTransformGizmo::OnPreCanInteract()
{
	return OnPreCanInteractDelegate;
}
	
FOnGizmoInteraction& UTransformGizmo::OnPostInteraction()
{
	return OnPostInteractionDelegate;
}

#undef LOCTEXT_NAMESPACE
