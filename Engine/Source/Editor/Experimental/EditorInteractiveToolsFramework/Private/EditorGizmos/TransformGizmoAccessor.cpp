// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorGizmos/TransformGizmoAccessor.h"

#include "BaseGizmos/GizmoElementGroup.h"

namespace UE::Editor::InteractiveToolsFramework
{
	ETransformGizmoPartIdentifier FTransformGizmoAccessor::GetLastHitPart(const UTransformGizmo& InGizmo) const
	{
		return InGizmo.LastHitPart;
	}

	bool FTransformGizmoAccessor::IsPartSelected(const UTransformGizmo& InGizmo, const ETransformGizmoPartIdentifier InPartId) const
	{
		if (InGizmo.GizmoElementRoot)
		{
			return InGizmo.GizmoElementRoot->GetPartInteractionState(static_cast<uint32>(InPartId)) == EGizmoElementInteractionState::Selected;
		}

		return false;
	}

	FTransform FTransformGizmoAccessor::GetGizmoTransform(const UTransformGizmo& InGizmo) const
	{
		return InGizmo.GetGizmoTransform();
	}

	UGizmoElementHitMultiTarget* FTransformGizmoAccessor::GetHitTarget(const UTransformGizmo& InGizmo) const
	{
		return InGizmo.HitTarget;
	}

	EGizmoTransformMode FTransformGizmoAccessor::GetCurrentMode(const UTransformGizmo& InGizmo) const
	{
		return InGizmo.CurrentMode;
	}

	FVector2D FTransformGizmoAccessor::GetScreenProjectedDirection(const UTransformGizmo& InGizmo, const FVector& InLocalAxis) const
	{
		if (!ensure(InGizmo.GizmoViewContext))
		{
			return FVector2D::ZeroVector;
		}

		return InGizmo.GetScreenProjectedAxis(InGizmo.GizmoViewContext, InLocalAxis, InGizmo.CurrentTransform);
	}
}
