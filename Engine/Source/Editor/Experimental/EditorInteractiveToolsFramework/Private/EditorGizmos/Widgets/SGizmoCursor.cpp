// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGizmoCursor.h"

#include "Framework/Application/SlateApplication.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

SLATE_IMPLEMENT_WIDGET(SGizmoCursor)
void SGizmoCursor::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, Size,  EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, Rotation, EInvalidateWidgetReason::Layout);
}

SGizmoCursor::SGizmoCursor()
	: Size(*this, FSlateApplication::Get().GetCursorSize())
	, Rotation(*this, 0.0f)
{
}

void SGizmoCursor::Construct(const FArguments& InArgs)
{
	Size.Assign(*this, InArgs._Size);
	Rotation.Assign(*this, InArgs._Rotation);

	ChildSlot
	[
		SNew(SImage)
		.ColorAndOpacity(InArgs._ColorAndOpacity.IsSet() ? InArgs._ColorAndOpacity : FLinearColor::White)
		.DesiredSizeOverride(this, &SGizmoCursor::GetSizeOverride)
		.RenderTransformPivot(FVector2D(0.5f, 0.5f))
		.RenderTransform(this, &SGizmoCursor::GetImageTransform)
		.Image(InArgs._Image.IsSet() ? InArgs._Image : FAppStyle::GetBrush("EditorViewport.Cursor"))
	];
}

void SGizmoCursor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(BrushMaterial);
}

FString SGizmoCursor::GetReferencerName() const
{
	return TEXT("SGizmoCursor");
}

void SGizmoCursor::SetSize(const FVector2f& InSize)
{
	Size.Set(*this, InSize);
}

TOptional<FVector2D> SGizmoCursor::GetSizeOverride() const
{
	return FVector2D(Size.Get());
}

TOptional<FSlateRenderTransform> SGizmoCursor::GetImageTransform() const
{
	return FSlateRenderTransform(FQuat2D(Rotation.Get()));
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
