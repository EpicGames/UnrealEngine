// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/SoftObjectPath.h"

#include "Widgets/SCompoundWidget.h"

class UMaterialInstanceDynamic;

/**
 * Slate widget that renders a rotatable, resizable cursor image for gizmo interactions.
 * Uses a dynamic material instance to draw the cursor brush, and implements FGCObject to prevent GC of the material.
 */
class SGizmoCursor
	: public SCompoundWidget
	, public FGCObject
{
	SLATE_DECLARE_WIDGET(SGizmoCursor, SCompoundWidget)
	
public:
	SLATE_BEGIN_ARGS(SGizmoCursor)
	{}
		/** Image resource */
		SLATE_ATTRIBUTE(const FSlateBrush*, Image)

		/** Color and opacity */
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)

		/** Size of the cursor image, in pixels. */
		SLATE_ATTRIBUTE(FVector2f, Size)

		/** Rotation of the cursor image, in radians. */
		SLATE_ATTRIBUTE(float, Rotation)
	SLATE_END_ARGS()

	SGizmoCursor();

	/** Constructs the cursor widget from the provided Slate arguments. */
	void Construct(const FArguments& InArgs);

	/** Reports various uobjects to the garbage collector so it is not collected while this widget is alive. */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	/** Returns the name of this referencer for GC debugging and logging. */
	virtual FString GetReferencerName() const override;

	/** Sets the size of the cursor widget. This will overwrite any attribute binding present. */
	void SetSize(const FVector2f& InSize);

private:
	/** Returns the desired size override for the cursor widget, or empty to use default sizing. */
	TOptional<FVector2D> GetSizeOverride() const;

	/** Returns the render transform (rotation) to apply to the cursor image, or empty for no transform. */
	TOptional<FSlateRenderTransform> GetImageTransform() const;

private:
	/** Current size of the cursor image in pixels. */
	TSlateAttribute<FVector2f> Size;

	/** Current rotation of the cursor image in radians. */
	TSlateAttribute<float> Rotation;

	/** Brush is used to draw the cursor on SImage */
	TSharedPtr<FSlateBrush> Brush;

	/** Asset path to the material used for the cursor brush. */
	FSoftObjectPath BrushMaterialPath;

	/** Material used to draw the brush texture */
	TObjectPtr<UMaterialInstanceDynamic> BrushMaterial = nullptr;
};
