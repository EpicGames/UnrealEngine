// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGizmoElementValue.h"

#include "Brushes/SlateRoundedBoxBrush.h"
#include "SlateOptMacros.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

namespace UE::Editor::InteractiveToolsFramework
{
	void SGizmoElementValue::Construct(const FArguments& InArgs)
	{
		// Switches between (readonly) label, and an edit box, depending on focus state.

		constexpr float Height = 28.0f;
		constexpr float MinWidth = 80.0f;

		const FSlateColor DefaultBackgroundColor = FSlateColor(FLinearColor::Black.CopyWithNewOpacity(0.65f));

		TAttribute<FSlateColor> BackgroundColorAndOpacity =
			InArgs._BackgroundColorAndOpacity.IsSet()
			? InArgs._BackgroundColorAndOpacity
			: DefaultBackgroundColor;

		RoundedBoxBrush = MakeShared<FSlateRoundedBoxBrush>(FStyleColors::White, 4.0f);

		ChildSlot
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(0) // Default to label

			+ SWidgetSwitcher::Slot()
			[
				SNew(SBox)
				.MinDesiredWidth(MinWidth)
				.HeightOverride(Height)
				[
					SNew(SBorder)
					.BorderImage(RoundedBoxBrush.Get())
					.BorderBackgroundColor(BackgroundColorAndOpacity)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(InArgs._Text)
						.ColorAndOpacity(FSlateColor(FLinearColor::White))
					]
				]
			]
		];
	}
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
