// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "CoreTypes.h"

#include "Widgets/SCompoundWidget.h"

struct FSlateRoundedBoxBrush;

namespace UE::Editor::InteractiveToolsFramework
{
	/** Slate widget that displays a formatted text value with a colored background, used for gizmo delta labels (distance, angle, scale). */
	class SGizmoElementValue : public SCompoundWidget
	{
		public:
			SLATE_BEGIN_ARGS(SGizmoElementValue)
			{ }
			SLATE_ATTRIBUTE(FText, Text)
			SLATE_ATTRIBUTE(FSlateColor, BackgroundColorAndOpacity)
		SLATE_END_ARGS()

		/** Constructs this widget with InArgs */
		void Construct(const FArguments& InArgs);

	private:
		TSharedPtr<FSlateRoundedBoxBrush> RoundedBoxBrush;
	};
}
