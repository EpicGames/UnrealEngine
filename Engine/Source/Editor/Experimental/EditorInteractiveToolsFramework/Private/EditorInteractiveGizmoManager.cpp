// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditorInteractiveGizmoManager.h"

#include "ContextObjectStore.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorInteractiveGizmoConditionalBuilder.h"
#include "EditorInteractiveGizmoSelectionBuilder.h"
#include "EditorInteractiveGizmoSubsystem.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "InputRouter.h"
#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "Misc/AssertionMacros.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "Snapping/EditorSnappingManager.h"
#include "Templates/Casts.h"
#include "ToolContextInterfaces.h"
#include "TransformGizmoEditorSettings.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorGizmos/TransformGizmo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorInteractiveGizmoManager)

class FCanvas;

#define LOCTEXT_NAMESPACE "UEditorInteractiveGizmoManager"

DEFINE_LOG_CATEGORY_STATIC(LogEditorInteractiveGizmoManager, Log, All);

namespace GizmoManagerLocals
{
	static TOptional<FGizmosParameters> OptDefaultParameters;
	static UEditorInteractiveGizmoManager::FOnUsesNewTRSGizmosChanged OnUsesNewTRSGizmosChanged;
	static UEditorInteractiveGizmoManager::FOnGizmosParametersChanged OnGizmosParametersChanged;
}

bool UEditorInteractiveGizmoManager::UsesNewTRSGizmos()
{
	return GetTransformGizmoVersion() >= 1;
}

void UEditorInteractiveGizmoManager::SetUsesNewTRSGizmos(const bool bUseNewTRSGizmos)
{
	constexpr int32 LegacyGizmoVersion = 0;
	constexpr int32 NewGizmoVersion = 2;
	SetTransformGizmoVersion(bUseNewTRSGizmos ? NewGizmoVersion : LegacyGizmoVersion);
}

int32 UEditorInteractiveGizmoManager::GetTransformGizmoVersion()
{
	if (const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>())
	{
		if (Settings->UsesNewTRSGizmo())
		{
			return 2; // New/5.8
		}

		if (Settings->UsesExperimentalGizmo())
		{
			return 1; // New/Experimental
		}
	}

	return 0; // Legacy
}

void UEditorInteractiveGizmoManager::SetTransformGizmoVersion(const int32 InVersion)
{
	if (GetTransformGizmoVersion() == InVersion)
	{
		return;
	}

	if (!FMath::IsWithinInclusive(InVersion, 0, 2))
	{
		UE_LOGF(LogEditorInteractiveGizmoManager, Error, "Invalid GizmoVersion %d, expected value between 0 and 2 inclusive. No changes applied.", InVersion);
		return;
	}

	if (UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>())
	{
		const bool bUseExperimentalGizmo = InVersion == 1;
		const bool bUseNew58Gizmo = InVersion == 2;

		// Ordering is important here, so we consider what transitions to what

		const bool bFromExperimentalToNew = GetTransformGizmoVersion() == 1 && bUseNew58Gizmo;

		if (bFromExperimentalToNew)
		{
			Settings->SetUseExperimentalGizmo(bUseExperimentalGizmo);
			Settings->SetUseNewTRSGizmo(bUseNew58Gizmo);	
		}
		else // bFromNewToExperimental or Legacy
		{
			Settings->SetUseNewTRSGizmo(bUseNew58Gizmo);
			Settings->SetUseExperimentalGizmo(bUseExperimentalGizmo);
		}
	}

	// If legacy is specified, the two above will have been disabled, leaving us with legacy enabled
}

UEditorInteractiveGizmoManager::FOnUsesNewTRSGizmosChanged& UEditorInteractiveGizmoManager::OnUsesNewTRSGizmosChangedDelegate()
{
	return GizmoManagerLocals::OnUsesNewTRSGizmosChanged;
}

void UEditorInteractiveGizmoManager::SetGizmosParameters(const FGizmosParameters& InParameters)
{
	if (UTransformGizmoEditorSettings* const TransformGizmoEditorSettings = GetMutableDefault<UTransformGizmoEditorSettings>())
	{
		TransformGizmoEditorSettings->SetGizmosParameters(InParameters);
	}
}

UEditorInteractiveGizmoManager::FOnGizmosParametersChanged& UEditorInteractiveGizmoManager::OnGizmosParametersChangedDelegate()
{
	return GizmoManagerLocals::OnGizmosParametersChanged;
}

const TOptional<FGizmosParameters>& UEditorInteractiveGizmoManager::GetDefaultGizmosParameters()
{
	if (const UTransformGizmoEditorSettings* TransformGizmoEditorSettings = GetDefault<UTransformGizmoEditorSettings>())
	{
		GizmoManagerLocals::OptDefaultParameters = TransformGizmoEditorSettings->GizmosParameters;
	}

	static TOptional<FGizmosParameters> Invalid;
	return GizmoManagerLocals::OptDefaultParameters.IsSet() ? GizmoManagerLocals::OptDefaultParameters : Invalid;
}

bool UEditorInteractiveGizmoManager::IsExplicitModeEnabled()
{
	return GetDefaultGizmosParameters().IsSet() ? GetDefaultGizmosParameters()->bEnableExplicit : false;
}

UEditorInteractiveGizmoManager::UEditorInteractiveGizmoManager()
{
	Registry = NewObject<UEditorInteractiveGizmoRegistry>();
	bShowEditorGizmos = UsesNewTRSGizmos();
}


void UEditorInteractiveGizmoManager::InitializeWithEditorModeManager(IToolsContextQueriesAPI* QueriesAPIIn, IToolsContextTransactionsAPI* TransactionsAPIIn, UInputRouter* InputRouterIn, FEditorModeTools* InEditorModeManager)
{
	Super::Initialize(QueriesAPIIn, TransactionsAPIIn, InputRouterIn);
	EditorModeManager = InEditorModeManager;

	if (UModeManagerInteractiveToolsContext* InteractiveToolsContext = EditorModeManager->GetInteractiveToolsContext())
	{
		// @note: this doesn't re-register if a snapping manager already exists
		UE::Editor::Gizmos::RegisterSceneSnappingManager(InteractiveToolsContext);
	}
}

void UEditorInteractiveGizmoManager::Shutdown()
{
	// Only the ModeManager-level GizmoManager should deregister the snapping manager.
	// Per-mode GizmoManagers must not touch it - their Shutdown() runs deferred (after the
	// next mode has already entered), which would remove the snapping manager out from under it.
	if (EditorModeManager)
	{
		if (UModeManagerInteractiveToolsContext* InteractiveToolsContext = EditorModeManager->GetInteractiveToolsContext())
		{
			if (InteractiveToolsContext->GizmoManager == this)
			{
				UE::Editor::Gizmos::DeregisterSceneSnappingManager(InteractiveToolsContext);
			}
		}
	}

	Registry->Shutdown();
	Super::Shutdown();
}

void UEditorInteractiveGizmoManager::RegisterEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder)
{
	check(Registry);
	Registry->RegisterEditorGizmoType(InGizmoCategory, InGizmoBuilder);
}

void UEditorInteractiveGizmoManager::DeregisterEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder)
{
	check(Registry);
	Registry->DeregisterEditorGizmoType(InGizmoCategory, InGizmoBuilder);
}

void UEditorInteractiveGizmoManager::GetQualifiedEditorGizmoBuilders(EEditorGizmoCategory InGizmoCategory, const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& InFoundBuilders)
{
	check(Registry);
	Registry->GetQualifiedEditorGizmoBuilders(InGizmoCategory, InToolBuilderState, InFoundBuilders);

	FEditorGizmoTypePriority FoundPriority = 0;

	if (!bSearchLocalBuildersOnly)
	{
		UEditorInteractiveGizmoSubsystem* GizmoSubsystem = GEditor->GetEditorSubsystem<UEditorInteractiveGizmoSubsystem>();
		if (ensure(GizmoSubsystem))
		{
			GizmoSubsystem->GetQualifiedGlobalEditorGizmoBuilders(InGizmoCategory, InToolBuilderState, InFoundBuilders);
		}
	}
}

UTransformGizmo* UEditorInteractiveGizmoManager::FindDefaultTransformGizmo() const
{
	return Cast<UTransformGizmo>( FindGizmoByInstanceIdentifier(TransformInstanceIdentifier()) );
}

bool UEditorInteractiveGizmoManager::DestroyEditorGizmo(UInteractiveGizmo* Gizmo)
{
	auto Pred = [Gizmo](const FActiveEditorGizmo& ActiveEditorGizmo) {return ActiveEditorGizmo.Gizmo == Gizmo; };
	if (!ensure(ActiveEditorGizmos.FindByPredicate(Pred)))
	{
		return false;
	}

	OnGizmosParametersChangedDelegate().RemoveAll(Gizmo);
	
	InputRouter->DeregisterSource(Gizmo);

	Gizmo->Shutdown();

	ActiveEditorGizmos.RemoveAll(Pred);

	PostInvalidation();

	return true;
}

void UEditorInteractiveGizmoManager::DestroyAllEditorGizmos()
{
	for (int i = 0; i < ActiveEditorGizmos.Num(); i++)
	{
		UInteractiveGizmo* Gizmo = ActiveEditorGizmos[i].Gizmo;
		if (ensure(Gizmo))
		{
			DestroyEditorGizmo(Gizmo);
		}
	}

	ActiveEditorGizmos.Reset();

	PostInvalidation();
}

UInteractiveGizmo* UEditorInteractiveGizmoManager::CreateGizmo(const FString& BuilderIdentifier, const FString& InstanceIdentifier, void* Owner)
{
	if (BuilderIdentifier == TransformBuilderIdentifier() && InstanceIdentifier == TransformInstanceIdentifier())
	{
		// return the default transform gizmo if it already exists.
		if (UTransformGizmo* ExistingGizmo = FindDefaultTransformGizmo())
		{
			return ExistingGizmo;
		}

		// create a new one
		UInteractiveGizmo* NewGizmo = Super::CreateGizmo(BuilderIdentifier, InstanceIdentifier, Owner);
		if (!NewGizmo)
		{
			return nullptr;
		}
		
		if (IEditorInteractiveGizmoSelectionBuilder* SelectionBuilder = Cast<IEditorInteractiveGizmoSelectionBuilder>(GizmoBuilders[BuilderIdentifier]))
		{
			FToolBuilderState CurrentSceneState;
			QueriesAPI->GetCurrentSelectionState(CurrentSceneState);
			
			SelectionBuilder->UpdateGizmoForSelection(NewGizmo, CurrentSceneState);
		}

		return NewGizmo;
	}
	
	return Super::CreateGizmo(BuilderIdentifier, InstanceIdentifier, Owner);
}

bool UEditorInteractiveGizmoManager::DestroyGizmo(UInteractiveGizmo* InGizmo)
{
	const bool bHasGizmo = ActiveGizmos.ContainsByPredicate([InGizmo](const FActiveGizmo& ActiveGizmo)
	{
		return ActiveGizmo.Gizmo == InGizmo;
	});
	if (bHasGizmo)
	{
		OnGizmosParametersChangedDelegate().RemoveAll(InGizmo);
	}
	
	return Super::DestroyGizmo(InGizmo);
}

// @todo move this to a gizmo context object
bool UEditorInteractiveGizmoManager::GetShowEditorGizmos()
{
	return bShowEditorGizmos;
}

bool UEditorInteractiveGizmoManager::GetShowEditorGizmosForView(IToolsContextRenderAPI* RenderAPI)
{
	const bool bEngineShowFlagsModeWidget = (RenderAPI && RenderAPI->GetSceneView() && 
											 RenderAPI->GetSceneView()->Family &&
											 RenderAPI->GetSceneView()->Family->EngineShowFlags.ModeWidgets);
	return bShowEditorGizmos && bEngineShowFlagsModeWidget;
}

bool UEditorInteractiveGizmoManager::AreEditorGizmosAllowed() const
{
	return bAllowEditorGizmos && UsesNewTRSGizmos();
}

void UEditorInteractiveGizmoManager::SetAllowEditorGizmos(bool bInAllowed)
{
	if (bAllowEditorGizmos != bInAllowed)
	{
		bAllowEditorGizmos = bInAllowed;
		
		if (UContextObjectStore* Store = GetContextObjectStore())
		{
			if (UEditorTransformGizmoContextObject* GizmoContext = Store->FindContext<UEditorTransformGizmoContextObject>())
			{
				GizmoContext->SetGizmosEnabled(AreEditorGizmosAllowed());
			}
		}
		
		UpdateActiveEditorGizmos();
	}
}

bool UEditorInteractiveGizmoManager::AreEditorGizmosAllowed(const FEditorModeTools* ModeTools)
{
	if (ModeTools && ModeTools->SupportsViewportITF())
	{
		if (const UModeManagerInteractiveToolsContext* Context = ModeTools->GetInteractiveToolsContext())
		{
			if (const UEditorInteractiveGizmoManager* GizmoManager = Cast<UEditorInteractiveGizmoManager>(Context->GizmoManager))
			{
				return GizmoManager->AreEditorGizmosAllowed();
			}
		}
	}

	return false;
}

void UEditorInteractiveGizmoManager::UpdateActiveEditorGizmos()
{
	if (!AreEditorGizmosAllowed())
	{
		if (UTransformGizmo* Gizmo = FindDefaultTransformGizmo())
		{
			DestroyGizmo(Gizmo);
		}
		
		if (bShowEditorGizmos)
		{
			DestroyAllEditorGizmos();
			bShowEditorGizmos = false;
		}
		
		return;
	}
	
	const bool bGizmosShouldBeVisible = EditorModeManager ? EditorModeManager->GetShowWidget() : true;

	if (UTransformGizmo* Gizmo = FindDefaultTransformGizmo())
	{
		if (Gizmo->bVisible != bGizmosShouldBeVisible)
		{
			Gizmo->SetVisibility(bGizmosShouldBeVisible);
		}
	}

	if (bShowEditorGizmos != bGizmosShouldBeVisible)
	{
		bShowEditorGizmos = bGizmosShouldBeVisible;
		
		if (!bShowEditorGizmos)
		{
			DestroyAllEditorGizmos();
		}
	}
}

void UEditorInteractiveGizmoManager::Tick(float DeltaTime)
{
	UpdateActiveEditorGizmos();
	
	Super::Tick(DeltaTime);

	for (FActiveEditorGizmo& ActiveEditorGizmo : ActiveEditorGizmos)
	{
		ActiveEditorGizmo.Gizmo->Tick(DeltaTime);
	}
}


void UEditorInteractiveGizmoManager::Render(IToolsContextRenderAPI* RenderAPI)
{
	Super::Render(RenderAPI);

	if (GetShowEditorGizmosForView(RenderAPI))
	{
		for (FActiveEditorGizmo& ActiveEditorGizmo : ActiveEditorGizmos)
		{
			ActiveEditorGizmo.Gizmo->Render(RenderAPI);
		}
	}
}

void UEditorInteractiveGizmoManager::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	Super::DrawHUD(Canvas, RenderAPI);

	if (GetShowEditorGizmosForView(RenderAPI))
	{
		for (FActiveEditorGizmo& ActiveEditorGizmo : ActiveEditorGizmos)
		{
			ActiveEditorGizmo.Gizmo->DrawHUD(Canvas, RenderAPI);
		}
	}
}

const FString& UEditorInteractiveGizmoManager::TransformInstanceIdentifier()
{
	static const FString Identifier(TEXT("EditorTransformGizmoInstance"));
	return Identifier;	
}

const FString& UEditorInteractiveGizmoManager::TransformBuilderIdentifier()
{
	static const FString Identifier(TEXT("EditorTransformGizmoBuilder"));
	return Identifier;
}

#undef LOCTEXT_NAMESPACE
