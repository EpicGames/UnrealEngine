// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorInteractiveGizmoRegistry.h"

#include "EditorInteractiveGizmoConditionalBuilder.h"
#include "EditorInteractiveGizmoSelectionBuilder.h"
#include "HAL/PlatformCrt.h"
#include "InteractiveGizmoBuilder.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorInteractiveGizmoRegistry)


#define LOCTEXT_NAMESPACE "UEditorInteractiveGizmoRegistry"

DEFINE_LOG_CATEGORY_STATIC(LogEditorInteractiveGizmoRegistry, Log, All);

void UEditorGizmoRegistryCategoryEntry::RegisterGizmoType(UInteractiveGizmoBuilder* InGizmoBuilder)
{
	check(InGizmoBuilder);

	if (GizmoTypes.Contains(InGizmoBuilder))
	{
		UE_LOGF(LogEditorInteractiveGizmoRegistry, Warning,
			"gizmo builder type %ls has already been registered!", *InGizmoBuilder->GetPathName());
		return;
	}

	GizmoTypes.Add(InGizmoBuilder);
}

void UEditorGizmoRegistryCategoryEntry::DeregisterGizmoType(UInteractiveGizmoBuilder* InGizmoBuilder)
{
	check(InGizmoBuilder);

	if (GizmoTypes.Contains(InGizmoBuilder))
	{
		GizmoTypes.Remove(InGizmoBuilder);
	}
	else
	{
		UE_LOGF(LogEditorInteractiveGizmoRegistry, Warning,
			"UInteractiveGizmoRegistry::DeregisterEditorGizmoType: gizmo type not found in registry %ls for category %ls", *InGizmoBuilder->GetName(), *CategoryName);
	}
}

void UEditorGizmoRegistryCategoryEntry::ClearGizmoTypes()
{
	GizmoTypes.Reset();
}

void UEditorGizmoRegistryCategoryEntry_ConditionalSelection::RegisterGizmoType(UInteractiveGizmoBuilder* InGizmoBuilder)
{
	if (!InGizmoBuilder->Implements<UEditorInteractiveGizmoConditionalBuilder>())
	{
		UE_LOGF(LogEditorInteractiveGizmoRegistry, Warning,
			"%ls gizmo builder '%ls' of type '%ls' does not implement the IEditorInteractiveGizmoConditionalBuilder interface! Skipping registration.", *CategoryName, *InGizmoBuilder->GetPathName(), *InGizmoBuilder->GetClass()->GetName());
		return;
	}

	if (!InGizmoBuilder->Implements<UEditorInteractiveGizmoSelectionBuilder>())
	{
		UE_LOGF(LogEditorInteractiveGizmoRegistry, Warning,
			"%ls gizmo builder '%ls' of type '%ls' does not implement the IEditorInteractiveGizmoSelectionBuilder interface! Skipping registration.", *CategoryName, *InGizmoBuilder->GetPathName(), *InGizmoBuilder->GetClass()->GetName());
		return;
	}

	Super::RegisterGizmoType(InGizmoBuilder);

	GizmoTypes.StableSort(
		[](UInteractiveGizmoBuilder& A, UInteractiveGizmoBuilder& B) {
			IEditorInteractiveGizmoConditionalBuilder* AA = Cast<IEditorInteractiveGizmoConditionalBuilder>(&A);
			IEditorInteractiveGizmoConditionalBuilder* BB = Cast<IEditorInteractiveGizmoConditionalBuilder>(&B);
			return (AA)->GetPriority() > (BB)->GetPriority();
		});
}

UEditorGizmoRegistryCategoryEntry_Primary::UEditorGizmoRegistryCategoryEntry_Primary()
{
	CategoryName = TEXT("Primary");
}

void UEditorGizmoRegistryCategoryEntry_Primary::GetQualifiedGizmoBuilders(const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& FoundBuilders)
{
	FEditorGizmoTypePriority FoundPriority = 0;

	if (FoundBuilders.Num() > 0)
	{
		if (IEditorInteractiveGizmoConditionalBuilder* Builder = Cast<IEditorInteractiveGizmoConditionalBuilder>(FoundBuilders[0]))
		{
			FoundPriority = Builder->GetPriority();
		}
	}

	for (const TObjectPtr<UInteractiveGizmoBuilder>& GizmoBuilder : GizmoTypes)
	{
		if (IEditorInteractiveGizmoConditionalBuilder* Builder = Cast<IEditorInteractiveGizmoConditionalBuilder>(GizmoBuilder))
		{
			if (Builder->GetPriority() < FoundPriority)
			{
				break;
			}

			if (Builder->SatisfiesCondition(InToolBuilderState))
			{
				// Reset found builders since only one primary builder should be buildable.
				FoundBuilders.Reset();
				FoundBuilders.Add(GizmoBuilder);
				break;
			}
		}
	}
}

UEditorGizmoRegistryCategoryEntry_Accessory::UEditorGizmoRegistryCategoryEntry_Accessory()
{
	CategoryName = TEXT("Accessory");
}

void UEditorGizmoRegistryCategoryEntry_Accessory::GetQualifiedGizmoBuilders(const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& FoundBuilders)
{
	FEditorGizmoTypePriority FoundPriority = 0;

	if (FoundBuilders.Num() > 0)
	{
		if (IEditorInteractiveGizmoConditionalBuilder* Builder = Cast<IEditorInteractiveGizmoConditionalBuilder>(FoundBuilders[0]))
		{
			FoundPriority = Builder->GetPriority();
		}
	}

	for (const TObjectPtr<UInteractiveGizmoBuilder>& GizmoBuilder : GizmoTypes)
	{
		if (IEditorInteractiveGizmoConditionalBuilder* Builder = Cast<IEditorInteractiveGizmoConditionalBuilder>(GizmoBuilder))
		{
			if (Builder->GetPriority() < FoundPriority)
			{
				break;
			}

			if (Builder->SatisfiesCondition(InToolBuilderState))
			{
				// If priority is greater than previous found priority, reset found builders. If it is the same simply append 
				// to the builders since more than one accessory builder can be built at a time.
				if (Builder->GetPriority() > FoundPriority)
				{
					FoundBuilders.Reset();
				}
				FoundBuilders.Add(GizmoBuilder);
			}
		}
	}
}

UEditorInteractiveGizmoRegistry::UEditorInteractiveGizmoRegistry()
{
	GizmoCategoryMap.Add(EEditorGizmoCategory::Primary, NewObject<UEditorGizmoRegistryCategoryEntry_Primary>());
	GizmoCategoryMap.Add(EEditorGizmoCategory::Accessory, NewObject<UEditorGizmoRegistryCategoryEntry_Accessory>());
}

void UEditorInteractiveGizmoRegistry::Shutdown()
{
	ClearEditorGizmoTypes();
}

void UEditorInteractiveGizmoRegistry::RegisterEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder)
{
	if (ensure(InGizmoBuilder))
	{
		if (TObjectPtr<UEditorGizmoRegistryCategoryEntry>* RegEntry = GizmoCategoryMap.Find(InGizmoCategory))
		{
			if (*RegEntry)
			{
				(*RegEntry)->RegisterGizmoType(InGizmoBuilder);
			}
		}
	}
}

void UEditorInteractiveGizmoRegistry::GetQualifiedEditorGizmoBuilders(EEditorGizmoCategory InGizmoCategory, const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& InFoundBuilders)
{
	if (TObjectPtr<UEditorGizmoRegistryCategoryEntry>* RegEntry = GizmoCategoryMap.Find(InGizmoCategory))
	{
		if (*RegEntry)
		{
			(*RegEntry)->GetQualifiedGizmoBuilders(InToolBuilderState, InFoundBuilders);
		}
	}
}

void UEditorInteractiveGizmoRegistry::DeregisterEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder)
{
	if (ensure(InGizmoBuilder))
	{
		if (TObjectPtr<UEditorGizmoRegistryCategoryEntry>* RegEntry = GizmoCategoryMap.Find(InGizmoCategory))
		{
			if (*RegEntry)
			{
				(*RegEntry)->DeregisterGizmoType(InGizmoBuilder);
			}
		}
	}

}

void UEditorInteractiveGizmoRegistry::ClearEditorGizmoTypes()
{
	for (auto& Elem : GizmoCategoryMap)
	{
		if (Elem.Value)
		{
			Elem.Value->ClearGizmoTypes();
		}
	}
}

#undef LOCTEXT_NAMESPACE

