// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorInteractiveGizmoSelectionBuilder.h"

#include "BaseGizmos/TransformProxy.h"
#include "Containers/Array.h"
#include "ContextObjectStore.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "GameFramework/Actor.h"
#include "InteractiveToolManager.h"
#include "Templates/SharedPointer.h"
#include "ToolContextInterfaces.h"
#include "Tools/AssetEditorContextInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorInteractiveGizmoSelectionBuilder)

class USceneComponent;

UTransformProxy* FEditorGizmoSelectionBuilderHelper::CreateTransformProxyForSelection(const FToolBuilderState& SceneState)
{
	// @todo - once UTransformProxy supports typed elements, update this to use the normalized typed
	// element selection set.
	if (IAssetEditorContextInterface* AssetEditorContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>())
	{
		if (const UTypedElementSelectionSet* SelectionSet = AssetEditorContext->GetSelectionSet())
		{
			if (SelectionSet->GetNumSelectedElements() > 0)
			{
				bool bHasSelectedElements = false;
				UTransformProxy* TransformProxy = NewObject<UTransformProxy>();

				SelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>([TransformProxy, SelectionSet, &bHasSelectedElements](const TTypedElement<ITypedElementWorldInterface>& InWorldElement)
					{
						if (InWorldElement.CanMoveElement(ETypedElementWorldType::Editor))
						{
							if (TTypedElement<ITypedElementObjectInterface> ObjectTypedElement = SelectionSet->GetElementList()->GetElement<ITypedElementObjectInterface>(InWorldElement))
							{
								if (AActor* Actor = ObjectTypedElement.GetObjectAs<AActor>())
								{
									USceneComponent* SceneComponent = Actor->GetRootComponent();
									TransformProxy->AddComponent(SceneComponent);
									bHasSelectedElements = true;
								}
							}
						}
						return true;
					});

				if (bHasSelectedElements)
				{
					return TransformProxy;
				}
			}
		}
	}

	return nullptr;
}
	
