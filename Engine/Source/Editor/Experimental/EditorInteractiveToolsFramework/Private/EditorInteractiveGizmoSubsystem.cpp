// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorInteractiveGizmoSubsystem.h"

#include "EditorGizmos/EditorTRSGizmoBuilder.h"
#include "Engine/Engine.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorInteractiveGizmoSubsystem)

class UInteractiveGizmoBuilder;
struct FToolBuilderState;

#define LOCTEXT_NAMESPACE "UEditorInteractiveGizmoSubsystem"

DEFINE_LOG_CATEGORY_STATIC(LogEditorInteractiveGizmoSubsystem, Log, All);


UEditorInteractiveGizmoSubsystem::UEditorInteractiveGizmoSubsystem()
	: UEditorSubsystem()
{
	Registry = NewObject<UEditorInteractiveGizmoRegistry>();
}

void UEditorInteractiveGizmoSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	if (GEngine && GEngine->IsInitialized())
	{
		RegisterBuiltinEditorGizmoTypes();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &UEditorInteractiveGizmoSubsystem::RegisterBuiltinEditorGizmoTypes);
	}
}

void UEditorInteractiveGizmoSubsystem::Deinitialize()
{
	check(Registry);
	DeregisterBuiltinEditorGizmoTypes();
	Registry->Shutdown();
}

void UEditorInteractiveGizmoSubsystem::RegisterBuiltinEditorGizmoTypes()
{
	// Setup gizmo transform builder (UEditorTRSGizmoBuilder extends UEditorTransformGizmoBuilder
	// and falls back to it when bUseEditorTRSGizmo is false)
	RegisterTransformGizmoBuilder(NewObject<UEditorTRSGizmoBuilder>());

	// Register built-in gizmo types here

	RegisterGlobalEditorGizmoTypesDelegate.Broadcast();
}

void UEditorInteractiveGizmoSubsystem::DeregisterBuiltinEditorGizmoTypes()
{
	DeregisterGlobalEditorGizmoTypesDelegate.Broadcast();

	TransformGizmoBuilder = nullptr;
}

void UEditorInteractiveGizmoSubsystem::RegisterGlobalEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder)
{
	check(Registry);
	Registry->RegisterEditorGizmoType(InGizmoCategory, InGizmoBuilder);
}

void UEditorInteractiveGizmoSubsystem::GetQualifiedGlobalEditorGizmoBuilders(EEditorGizmoCategory InGizmoCategory, const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& InFoundBuilders)
{
	check(Registry);
	Registry->GetQualifiedEditorGizmoBuilders(InGizmoCategory, InToolBuilderState, InFoundBuilders);
}

void UEditorInteractiveGizmoSubsystem::RegisterTransformGizmoBuilder(UInteractiveGizmoBuilder* InTransformGizmoBuilder)
{
	TransformGizmoBuilder = InTransformGizmoBuilder;
}

void UEditorInteractiveGizmoSubsystem::DeregisterGlobalEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder)
{
	check(Registry);
	Registry->DeregisterEditorGizmoType(InGizmoCategory, InGizmoBuilder);
}

#undef LOCTEXT_NAMESPACE

