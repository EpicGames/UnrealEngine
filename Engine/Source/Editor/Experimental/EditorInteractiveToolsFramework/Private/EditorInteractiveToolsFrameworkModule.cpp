// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorInteractiveToolsFrameworkModule.h"

#include "ComponentSourceInterfaces.h"
#include "Customizations/GizmoPerStateValueCustomization.h"
#include "Customizations/TransformGizmoEditorSettingsCustomization.h"
#include "EditorGizmos/GizmoRotationUtil.h"
#include "EditorInteractiveGizmoManager.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "PropertyEditorModule.h"
#include "Styling/AppStyle.h"
#include "Templates/UniquePtr.h"
#include "Tools/EditorComponentSourceFactory.h"
#include "TransformGizmoEditorSettings.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SGizmoSettings.h"
#include "Widgets/SGizmoTree.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Settings/LevelEditorViewportSettings.h"

#define LOCTEXT_NAMESPACE "FEditorInteractiveToolsFrameworkModule"

DEFINE_LOG_CATEGORY_STATIC(LogEditorInteractiveToolsFrameworkModule, Log, All);

namespace UE::Editor::InteractiveToolsFramework::Private
{
	constexpr const TCHAR* GizmoVersionCVarName = TEXT("Gizmos.Version");
}

int32 FEditorInteractiveToolsFrameworkGlobals::RegisteredStaticMeshTargetFactoryKey = -1;

const FLazyName FEditorInteractiveToolsFrameworkModule::GizmoSettingsTabId = TEXT("GizmoSettings");
const FLazyName FEditorInteractiveToolsFrameworkModule::GizmoTreeTabId = TEXT("GizmoTree");

// ---------------------------------------------------------------------------
// TRS Gizmo CVar management (merged from EditorTRSGizmoModule)
// ---------------------------------------------------------------------------

void FEditorInteractiveToolsFrameworkModule::OnUsesNewTRSGizmosChanged(bool bInUsesNewTRSGizmos)
{
	const bool bWillEditorTRSGizmo = bInUsesNewTRSGizmos && GetDefault<UTransformGizmoEditorSettings>()->bUseEditorTRSGizmo;

	static bool bUsesEditorTRSGizmo = false;
	if (bUsesEditorTRSGizmo == bWillEditorTRSGizmo)
	{
		return;
	}

	bUsesEditorTRSGizmo = bWillEditorTRSGizmo;

	constexpr const TCHAR* ITFGizmoEnabledCVars[] = {
		TEXT("Editor.EnableITFCursorOverrideSupport"),
		TEXT("DragTools.EnableITFTools"),
		TEXT("ViewportInteractions.EnableITFInteractions"),
		TEXT("Editor.UnboundedCursorDrag")
	};

	constexpr const TCHAR* ITFGizmoDisabledCVars[] = {
		TEXT("Editor.AlwaysSnapPercentageBasedScale")
	};

	auto ToggleCVars = [](TConstArrayView<const TCHAR*> CVars, bool bEnable)
	{
		for (const TCHAR* CVarToToggle : CVars)
		{
			if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarToToggle))
			{
				if (CVar->GetBool() != bEnable)
				{
					CVar->Set(bEnable);
				}
			}
		}
	};

	ToggleCVars(ITFGizmoEnabledCVars, bWillEditorTRSGizmo);
	ToggleCVars(ITFGizmoDisabledCVars, !bWillEditorTRSGizmo);

	if (bWillEditorTRSGizmo)
	{
		// The combined translate/rotate widget is not supported with the new system
		ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
		ViewportSettings->bAllowTranslateRotateZWidget = false;
		
		UTransformGizmoEditorSettings* GizmoSettings = GetMutableDefault<UTransformGizmoEditorSettings>();
		GizmoSettings->bEnableCombinedTranslateRotate = false; 
	}
}

void FEditorInteractiveToolsFrameworkModule::OnPostEngineInit()
{
	// Sync CVars to initial state and subscribe to future changes.
	// Builder registration is handled by UEditorInteractiveGizmoSubsystem::RegisterBuiltinEditorGizmoTypes.
	OnUsesNewTRSGizmosChanged(UEditorInteractiveGizmoManager::UsesNewTRSGizmos());
	TRSGizmoChangeDelegate = UEditorInteractiveGizmoManager::OnUsesNewTRSGizmosChangedDelegate().AddStatic(
		&FEditorInteractiveToolsFrameworkModule::OnUsesNewTRSGizmosChanged
	);
}

// ---------------------------------------------------------------------------
// Settings UI (merged from EditorTRSGizmoSettingsModule)
// ---------------------------------------------------------------------------

void FEditorInteractiveToolsFrameworkModule::RegisterConsoleCommands()
{
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("Gizmos.ShowSettings"),
		TEXT("Displays a window listing hidden settings for Gizmos"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FEditorInteractiveToolsFrameworkModule::ExecuteShowSettings),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("Gizmos.ShowTree"),
		TEXT("Displays a window with various debug options for Gizmos"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FEditorInteractiveToolsFrameworkModule::ExecuteShowGizmoTree),
		ECVF_Default
	));

	using namespace UE::Editor::InteractiveToolsFramework::Private;
	constexpr int32 DefaultGizmoVersion = 2; // 2 = New TRS Gizmo
	ConsoleCommands.Add(IConsoleManager::Get().RegisterDelegatedConsoleVariable(
		GizmoVersionCVarName,
		DefaultGizmoVersion,
		[&]() { return UEditorInteractiveGizmoManager::GetTransformGizmoVersion(); },
		[&](int32 Version) { UEditorInteractiveGizmoManager::SetTransformGizmoVersion(Version); },
		TEXT("0 = Legacy\n1 = Experimental (<5.8)\n2 = New (5.8+)"),
		ECVF_Default
	));
}

void FEditorInteractiveToolsFrameworkModule::UnregisterConsoleCommands()
{
	for (IConsoleObject* Cmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}

	ConsoleCommands.Empty();
}

void FEditorInteractiveToolsFrameworkModule::RegisterCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomClassLayout(
		UTransformGizmoEditorSettings::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FTransformGizmoEditorSettingsCustomization::MakeInstance));

	// Notify PropertyEditor to pick up the customization if any detail panels were already created
	PropertyEditorModule.NotifyCustomizationModuleChanged();

	// Sections
	{
		// Shared
		{
			TSharedRef<FPropertySection> SharedSection = PropertyEditorModule.FindOrCreateSection(
			UTransformGizmoEditorSettings::StaticClass()->GetFName(),
			TEXT("Shared"),
			FText::FromString(TEXT("Shared")),
			-5);

			SharedSection->AddCategory("Shared");
		}

		// Translate
		{
			TSharedRef<FPropertySection> TranslateSection = PropertyEditorModule.FindOrCreateSection(
			UTransformGizmoEditorSettings::StaticClass()->GetFName(),
				TEXT("Translate"),
				FText::FromString(TEXT("Translate")),
				-1);

			TranslateSection->AddCategory("Translate");
		}

		// Rotate
		{
			TSharedRef<FPropertySection> RotateSection = PropertyEditorModule.FindOrCreateSection(
			UTransformGizmoEditorSettings::StaticClass()->GetFName(),
				TEXT("Rotate"),
				FText::FromString(TEXT("Rotate")));

			RotateSection->AddCategory("Rotate");
		}

		// Scale
		{
			TSharedRef<FPropertySection> ScaleSection = PropertyEditorModule.FindOrCreateSection(
				UTransformGizmoEditorSettings::StaticClass()->GetFName(),
				TEXT("Scale"),
				FText::FromString(TEXT("Scale")),
				1);

			ScaleSection->AddCategory("Scale");
		}
	}
}

void FEditorInteractiveToolsFrameworkModule::UnregisterCustomizations()
{
	const FName PropertyEditorModuleName("PropertyEditor");
	if (FModuleManager::Get().IsModuleLoaded(PropertyEditorModuleName))
	{
		if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>(PropertyEditorModuleName))
		{
			const FName TransformGizmoEditorSettingsClassName = UTransformGizmoEditorSettings::StaticClass()->GetFName();
			PropertyEditorModule->UnregisterCustomClassLayout(TransformGizmoEditorSettingsClassName);

			PropertyEditorModule->NotifyCustomizationModuleChanged();
		}
	}
}

void FEditorInteractiveToolsFrameworkModule::RegisterTabs()
{
	TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();

	GlobalTabManager->RegisterNomadTabSpawner(
		GizmoSettingsTabId,
		FOnSpawnTab::CreateLambda(
			[this](const FSpawnTabArgs&)
			{
				TSharedRef<SDockTab> DockTab =
					SNew(SDockTab)
					.TabRole(ETabRole::NomadTab)
					.Content()
					[
						SNew(SGizmoSettings)
					];

				return DockTab;
			}
		)
	)
	.SetDisplayName(LOCTEXT("GizmoSettingsTabName", "Gizmo Settings"))
	.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory())
	.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"))
	.SetAutoGenerateMenuEntry(false);

	GlobalTabManager->RegisterNomadTabSpawner(
		GizmoTreeTabId,
		FOnSpawnTab::CreateLambda(
			[this](const FSpawnTabArgs&)
			{
				TSharedRef<SDockTab> DockTab =
					SNew(SDockTab)
					.TabRole(ETabRole::NomadTab)
					.Content()
					[
						SNew(SGizmoTree)
					];

				return DockTab;
			}
		)
	)
	.SetDisplayName(LOCTEXT("GizmoTreeTabName", "Gizmo Tree"))
	.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory())
	.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"))
	.SetAutoGenerateMenuEntry(false);
}

void FEditorInteractiveToolsFrameworkModule::UnregisterTabs()
{
	if (FSlateApplication::IsInitialized())
	{
		const TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();

		GlobalTabManager->UnregisterNomadTabSpawner(GizmoSettingsTabId);
		GlobalTabManager->UnregisterNomadTabSpawner(GizmoTreeTabId);

		WeakGizmoSettingsTab.Reset();
		WeakGizmoTreeTab.Reset();
	}
}

void FEditorInteractiveToolsFrameworkModule::ExecuteShowSettings(const TArray<FString>& InArgs)
{
	if (!WeakGizmoSettingsTab.IsValid())
	{
		WeakGizmoSettingsTab = FGlobalTabmanager::Get()->TryInvokeTab(GizmoSettingsTabId.Resolve());
	}
}

void FEditorInteractiveToolsFrameworkModule::ExecuteShowGizmoTree(const TArray<FString>& InArgs)
{
	if (!WeakGizmoTreeTab.IsValid())
	{
		WeakGizmoTreeTab = FGlobalTabmanager::Get()->TryInvokeTab(GizmoTreeTabId.Resolve());
	}
}

// ---------------------------------------------------------------------------
// Module startup / shutdown
// ---------------------------------------------------------------------------

void FEditorInteractiveToolsFrameworkModule::StartupModule()
{
	FEditorInteractiveToolsFrameworkGlobals::RegisteredStaticMeshTargetFactoryKey
		= AddComponentTargetFactory( TUniquePtr<FComponentTargetFactory>{new FStaticMeshComponentTargetFactory{} } );

	// The Interactive Tools Framework needs to be explicitly loaded, at the very least to make sure that
	// the InteractiveToolsSelectionStoreSubsystem gets initialized.
	FModuleManager::Get().LoadModule(TEXT("InteractiveToolsFramework"));

	// Register relative transform interface
	using namespace UE::GizmoRotationUtil;
	FRelativeTransformInterfaceRegistry& Registry = FRelativeTransformInterfaceRegistry::Get();
	Registry.RegisterDefaultInterfaces();

	if (!IsRunningCommandlet())
	{
		// TRS gizmo builder registration and CVar sync (from EditorTRSGizmoModule)
		if (GEngine && GEngine->IsInitialized())
		{
			OnPostEngineInit();
		}
		else
		{
			FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FEditorInteractiveToolsFrameworkModule::OnPostEngineInit);
		}

		// Settings UI (from EditorTRSGizmoSettingsModule)
		RegisterConsoleCommands();
		RegisterCustomizations();
		RegisterTabs();
	}
}

void FEditorInteractiveToolsFrameworkModule::ShutdownModule()
{
	if (!IsRunningCommandlet())
	{
		FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

		UnregisterTabs();
		UnregisterCustomizations();
		UnregisterConsoleCommands();
	}

	if (TRSGizmoChangeDelegate.IsValid())
	{
		UEditorInteractiveGizmoManager::OnUsesNewTRSGizmosChangedDelegate().Remove(TRSGizmoChangeDelegate);
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FEditorInteractiveToolsFrameworkModule, EditorInteractiveToolsFramework)
