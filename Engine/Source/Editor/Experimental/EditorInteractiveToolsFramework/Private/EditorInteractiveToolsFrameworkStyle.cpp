// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorInteractiveToolsFrameworkStyle.h"

#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"

namespace UE::Editor::InteractiveToolsFramework::Private
{
	FName FEditorInteractiveToolsFrameworkStyle::StyleName("EditorInteractiveToolsFramework");

	FEditorInteractiveToolsFrameworkStyle& FEditorInteractiveToolsFrameworkStyle::Get()
	{
		static FEditorInteractiveToolsFrameworkStyle Instance;
		return Instance;
	}

	const FName& FEditorInteractiveToolsFrameworkStyle::GetStyleSetName() const
	{
		return StyleName;
	}

	FEditorInteractiveToolsFrameworkStyle::FEditorInteractiveToolsFrameworkStyle()
		: FSlateStyleSet(StyleName)
	{
		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		// Cursors
		Set("Cursor.ArrowLeftRight", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-leftright", CoreStyleConstants::Icon16x16));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FEditorInteractiveToolsFrameworkStyle::~FEditorInteractiveToolsFrameworkStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
}
