// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateColor.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"

namespace UE::Editor::InteractiveToolsFramework::Private
{
	/** Slate style set providing brushes, fonts, and colors for the Editor Interactive Tools Framework widgets (gizmo cursors, value labels, etc.). */
	class FEditorInteractiveToolsFrameworkStyle
		: public FSlateStyleSet
	{
	public:
		/** Returns the singleton instance, creating and registering it on first access. */
		static FEditorInteractiveToolsFrameworkStyle& Get();

		/** Returns the registered name used to identify this style set. */
		virtual const FName& GetStyleSetName() const override;

	private:
		FEditorInteractiveToolsFrameworkStyle();
		virtual ~FEditorInteractiveToolsFrameworkStyle() override;

	private:
		/** The registered name of this style set. */
		static FName StyleName;
	};
}
