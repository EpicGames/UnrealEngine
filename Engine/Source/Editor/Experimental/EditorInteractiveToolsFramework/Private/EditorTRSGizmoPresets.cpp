// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTRSGizmoPresets.h"

#include "CoreGlobals.h"
#include "Editor.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorInteractiveGizmoSubsystem.h"
#include "ScopedTransaction.h"
#include "TransformGizmoEditorSettings.h"

#define LOCTEXT_NAMESPACE "EditorTRSGizmoPresets"

class UTransformGizmoEditorSettings;

const TArray<FTransformGizmoPreset>& FTransformGizmoPreset::GetBuiltInPresets()
{
	static TArray<FTransformGizmoPreset> BuiltInPresets;
	if (BuiltInPresets.IsEmpty())
	{
		FTransformGizmoPreset DefaultPreset(
			TEXT("Default"),
			LOCTEXT("PresetDefaultDisplayName", "Default"),
			LOCTEXT("PresetDefaultToolTip", "The default settings for the transform gizmo."));
		{
			DefaultPreset.bDragDuplicateOnRotation = false;
			DefaultPreset.bIncludeExplicitRotation = false;
			DefaultPreset.bUniformIndirectScale = false;
			DefaultPreset.bPersistHandleSelection = false;
			DefaultPreset.bSequentialIndirectAxesButtons = false;
			DefaultPreset.bAdditiveIndirectAxes = false;
			DefaultPreset.bScreenSpaceIndirectOrthographicManipulation = true;
			DefaultPreset.RotateMode = EAxisRotateMode::ScreenArc;
			DefaultPreset.ScaleType = EGizmoTransformScaleType::PercentageBased;
		}

		FTransformGizmoPreset ClassicPreset(
			TEXT("Classic"),
			LOCTEXT("PresetUELegacyDisplayName", "UE Classic"),
			LOCTEXT("PresetUELegacyToolTip", "Unreal Classic settings for the transform gizmo, similar to the original UE editor behavior."));
		{
			ClassicPreset.bDragDuplicateOnRotation = true;
			ClassicPreset.bIncludeExplicitRotation = false;
			ClassicPreset.bUniformIndirectScale = true;
			ClassicPreset.bPersistHandleSelection = false;
			ClassicPreset.bSequentialIndirectAxesButtons = false;
			ClassicPreset.bAdditiveIndirectAxes = false;
			ClassicPreset.bScreenSpaceIndirectOrthographicManipulation = true;
			ClassicPreset.RotateMode = EAxisRotateMode::Pull;
			ClassicPreset.ScaleType = EGizmoTransformScaleType::PercentageBased;
		}

		FTransformGizmoPreset MayaPreset(
			TEXT("Maya"),
			LOCTEXT("PresetMayaDisplayName", "Maya-Style Interaction"),
			LOCTEXT("PresetMayaToolTip", "Settings for the transform gizmo that mimic the behavior of Maya."));
		{
			MayaPreset.bDragDuplicateOnRotation = false;
			MayaPreset.bIncludeExplicitRotation = true;
			MayaPreset.bUniformIndirectScale = false;
			MayaPreset.bPersistHandleSelection = true;
			MayaPreset.bSequentialIndirectAxesButtons = true;
			MayaPreset.bAdditiveIndirectAxes = false;
			MayaPreset.bScreenSpaceIndirectOrthographicManipulation = false;
			MayaPreset.RotateMode = EAxisRotateMode::ScreenArc;
			MayaPreset.ScaleType = EGizmoTransformScaleType::PercentageBased;
		}

		BuiltInPresets.Emplace(DefaultPreset);
		BuiltInPresets.Emplace(ClassicPreset);
		BuiltInPresets.Emplace(MayaPreset);
	}

	return BuiltInPresets;
}

const FTransformGizmoPreset* FTransformGizmoPreset::FindPresetByName(FName InName)
{
	const TArray<FTransformGizmoPreset>& Presets = GetBuiltInPresets();
	return Presets.FindByPredicate([InName](const FTransformGizmoPreset& P) { return P.Name == InName; });
}

namespace UE::Editor::InteractiveToolsFramework
{
	bool FTransformGizmoPresetAccessor::CurrentSettingsMatchPreset(const FTransformGizmoPreset& InPreset) const
	{
		return GetDragDuplicateOnRotation() == InPreset.bDragDuplicateOnRotation
			&& GetIncludeExplicitRotation() == InPreset.bIncludeExplicitRotation
			&& GetUniformIndirectScale() == InPreset.bUniformIndirectScale
			&& GetPersistHandleSelection() == InPreset.bPersistHandleSelection
			&& GetSequentialIndirectAxesButtons() == InPreset.bSequentialIndirectAxesButtons
			&& GetAdditiveIndirectAxes() == InPreset.bAdditiveIndirectAxes
			&& GetScreenSpaceIndirectOrthographicManipulation() == InPreset.bScreenSpaceIndirectOrthographicManipulation
			&& GetRotateMode() == InPreset.RotateMode
			&& GetScaleType() == InPreset.ScaleType;
	}

	TArray<FPresetSettingComparison> FTransformGizmoPresetAccessor::BuildComparisonWithPreset(const FTransformGizmoPreset& InPreset) const
	{
		TArray<FPresetSettingComparison> Comparisons;
		Comparisons.Reserve(8);

		auto AddBoolComparison = [&](const FText& InSettingName, const bool bInCurrentValue, const bool bInPresetValue)
		{
			FPresetSettingComparison Comparison;
			Comparison.SettingName = InSettingName;
			Comparison.bCurrentValueDiffersToPresetValue = bInCurrentValue != bInPresetValue;
			Comparison.bIsBool = true;
			Comparison.bCurrentValue = bInCurrentValue;
			Comparison.bPresetValue = bInPresetValue;
			Comparisons.Add(MoveTemp(Comparison));
		};

		AddBoolComparison(LOCTEXT("CompareRotateDuplicate", "Drag Duplicate on Rotation"), GetDragDuplicateOnRotation(), InPreset.bDragDuplicateOnRotation);
		AddBoolComparison(LOCTEXT("CompareExplicitRotation", "Include Explicit Rotation"), GetIncludeExplicitRotation(), InPreset.bIncludeExplicitRotation);
		AddBoolComparison(LOCTEXT("CompareUniformScale", "Uniform Indirect Scale"), GetUniformIndirectScale(), InPreset.bUniformIndirectScale);
		AddBoolComparison(LOCTEXT("ComparePersistHandle", "Persist Handle Selection"), GetPersistHandleSelection(), InPreset.bPersistHandleSelection);
		AddBoolComparison(LOCTEXT("CompareSequentialAxes", "Sequential Indirect Axes"), GetSequentialIndirectAxesButtons(), InPreset.bSequentialIndirectAxesButtons);
		AddBoolComparison(LOCTEXT("CompareAdditiveAxes", "Additive Indirect Axes"), GetAdditiveIndirectAxes(), InPreset.bAdditiveIndirectAxes);
		AddBoolComparison(LOCTEXT("CompareScreenSpaceOrtho", "Screen Space Indirect Orthographic Manipulation"), GetScreenSpaceIndirectOrthographicManipulation(), InPreset.bScreenSpaceIndirectOrthographicManipulation);

		{
			auto RotateModeToText = [](const EAxisRotateMode::Type InMode) -> FText
			{
				switch (InMode)
				{
				case EAxisRotateMode::Pull: 
					return LOCTEXT("CompareRotateModePull", "Pull");

				case EAxisRotateMode::Arc: 
					return LOCTEXT("CompareRotateModeArc", "Arc");

				case EAxisRotateMode::ScreenArc:
				default:
					return LOCTEXT("CompareRotateModeScreenArc", "Screen Arc");
				}
			};
			
			const EAxisRotateMode::Type CurrentRotateMode = GetRotateMode();
			Comparisons.Add({
				LOCTEXT("CompareRotateMode", "Rotate Mode"),
				RotateModeToText(CurrentRotateMode),
				RotateModeToText(InPreset.RotateMode),
				CurrentRotateMode != InPreset.RotateMode
			});
		}

		{
			auto ScaleTypeToText = [](const EGizmoTransformScaleType InType) -> FText
			{
				switch (InType)
				{
				case EGizmoTransformScaleType::Default: 
					return LOCTEXT("CompareScaleTypeDefault", "Default");

				case EGizmoTransformScaleType::Legacy: 
					return LOCTEXT("CompareScaleTypeLegacy", "Legacy");

				case EGizmoTransformScaleType::PercentageBased:
				default:
					return LOCTEXT("CompareScaleTypePercentage", "Percentage Based");
				}
			};

			const EGizmoTransformScaleType CurrentScaleType = GetScaleType();
			Comparisons.Add({
				LOCTEXT("CompareScaleType", "Scale Type"),
				ScaleTypeToText(CurrentScaleType),
				ScaleTypeToText(InPreset.ScaleType),
				CurrentScaleType != InPreset.ScaleType
			});
		}

		return Comparisons;
	}

	void FTransformGizmoPresetAccessor::ApplyPreset(const FTransformGizmoPreset& InPreset) const
	{
		FScopedTransaction Transaction(LOCTEXT("ApplyGizmoPreset", "Apply Transform Gizmo Preset"));

		SetDragDuplicateOnRotation(InPreset.bDragDuplicateOnRotation);
		SetIncludeExplicitRotation(InPreset.bIncludeExplicitRotation);
		SetUniformIndirectScale(InPreset.bUniformIndirectScale);
		SetPersistHandleSelection(InPreset.bPersistHandleSelection);
		SetSequentialIndirectAxesButtons(InPreset.bSequentialIndirectAxesButtons);
		SetAdditiveIndirectAxes(InPreset.bAdditiveIndirectAxes);
		SetScreenSpaceIndirectOrthographicManipulation(InPreset.bScreenSpaceIndirectOrthographicManipulation);
		SetRotateMode(InPreset.RotateMode);
		SetScaleType(InPreset.ScaleType);
	}

	bool FTransformGizmoPresetAccessor::GetDragDuplicateOnRotation() const
	{
		const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
		return Settings->GizmosParameters.bDragDuplicateOnRotation;
	}

	void FTransformGizmoPresetAccessor::SetDragDuplicateOnRotation(bool bInValue) const
	{
		UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
		Settings->GizmosParameters.bDragDuplicateOnRotation = bInValue;

		BroadcastPropertyChanged<UTransformGizmoEditorSettings>(Settings, "GizmosParameters");
	}

	bool FTransformGizmoPresetAccessor::GetIncludeExplicitRotation() const
	{
		const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
		return Settings->GizmosParameters.bEnableExplicit;
	}

	void FTransformGizmoPresetAccessor::SetIncludeExplicitRotation(bool bInValue) const
	{
		UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
		Settings->GizmosParameters.bEnableExplicit = bInValue;

		BroadcastPropertyChanged<UTransformGizmoEditorSettings>(Settings, "GizmosParameters");
	}

	bool FTransformGizmoPresetAccessor::GetUniformIndirectScale() const
	{
		const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
		return Settings->GizmosParameters.bUniformIndirectScale;
	}

	void FTransformGizmoPresetAccessor::SetUniformIndirectScale(bool bInValue) const
	{
		UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
		Settings->GizmosParameters.bUniformIndirectScale = bInValue;

		BroadcastPropertyChanged<UTransformGizmoEditorSettings>(Settings, "GizmosParameters");
	}

	bool FTransformGizmoPresetAccessor::GetAdditiveIndirectAxes() const
	{
		const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
		return Settings->GizmosParameters.bAdditiveIndirectAxes;
	}

	void FTransformGizmoPresetAccessor::SetAdditiveIndirectAxes(bool bInValue) const
	{
		UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
		Settings->GizmosParameters.bAdditiveIndirectAxes = bInValue;

		BroadcastPropertyChanged<UTransformGizmoEditorSettings>(Settings, "GizmosParameters");
	}

	bool FTransformGizmoPresetAccessor::GetScreenSpaceIndirectOrthographicManipulation() const
	{
		const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
		return Settings->GizmosParameters.bScreenSpaceIndirectOrthographicManipulation;
	}

	void FTransformGizmoPresetAccessor::SetScreenSpaceIndirectOrthographicManipulation(bool bInValue) const
	{
		UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
		Settings->GizmosParameters.bScreenSpaceIndirectOrthographicManipulation = bInValue;

		BroadcastPropertyChanged<UTransformGizmoEditorSettings>(Settings, "GizmosParameters");
	}

	bool FTransformGizmoPresetAccessor::GetPersistHandleSelection() const
	{
		const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
		return Settings->GizmosParameters.bPersistHandleSelection;
	}

	void FTransformGizmoPresetAccessor::SetPersistHandleSelection(bool bInValue) const
	{
		UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
		Settings->GizmosParameters.bPersistHandleSelection = bInValue;
		
		BroadcastPropertyChanged<UTransformGizmoEditorSettings>(Settings, "GizmosParameters");
	}

	bool FTransformGizmoPresetAccessor::GetSequentialIndirectAxesButtons() const
	{
		const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
		return Settings->GizmosParameters.bSequentialIndirectAxesButtons;
	}

	void FTransformGizmoPresetAccessor::SetSequentialIndirectAxesButtons(bool bInValue) const
	{
		UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
		Settings->GizmosParameters.bSequentialIndirectAxesButtons = bInValue;
		
		BroadcastPropertyChanged<UTransformGizmoEditorSettings>(Settings, "GizmosParameters");
	}

	EAxisRotateMode::Type FTransformGizmoPresetAccessor::GetRotateMode() const
	{
		const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
		return Settings->GizmosParameters.RotateMode;
	}

	void FTransformGizmoPresetAccessor::SetRotateMode(EAxisRotateMode::Type InRotateMode) const
	{
		UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
		Settings->GizmosParameters.RotateMode = InRotateMode;

		BroadcastPropertyChanged<UTransformGizmoEditorSettings>(Settings, "GizmosParameters");
	}

	EGizmoTransformScaleType FTransformGizmoPresetAccessor::GetScaleType() const
	{
		const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
		return Settings->GizmosParameters.ScaleType;
	}

	void FTransformGizmoPresetAccessor::SetScaleType(EGizmoTransformScaleType InScaleType) const
	{
		UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
		Settings->GizmosParameters.ScaleType = InScaleType;

		BroadcastPropertyChanged<UTransformGizmoEditorSettings>(Settings, "GizmosParameters");
	}
}

#undef LOCTEXT_NAMESPACE
