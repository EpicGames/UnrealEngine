// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorGizmos/TransformGizmo.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"
#include "Engine/DeveloperSettings.h"

#include "EditorTRSGizmoPresets.generated.h"

/** Represents a predefined gizmo configuration preset. */
USTRUCT(MinimalAPI)
struct FTransformGizmoPreset
{
	GENERATED_BODY()

	FTransformGizmoPreset() = default;

	FTransformGizmoPreset(
		const FName InName,
		const FText& InDisplayName,
		const FText& InToolTipText)
		: Name(InName)
		, DisplayName(InDisplayName)
		, ToolTipText(InToolTipText)
	{}

	/** The internal name of the preset. */
	UPROPERTY()
	FName Name;

	/** The display name of the preset. */
	UPROPERTY()
	FText DisplayName;

	/** Provides text for a tooltip display. */
	UPROPERTY()
	FText ToolTipText;

	/**
	 * Indicates whether a drag duplication action should be triggered when rotating an object.
	 */
	UPROPERTY()
	bool bDragDuplicateOnRotation = true;

	/**
	 * Indicates whether explicit rotation operations should be included.
	 */
	UPROPERTY()
	bool bIncludeExplicitRotation = false;

	/** Indicates whether the indirect scale should be applied uniformly across all dimensions. */
	UPROPERTY()
	bool bUniformIndirectScale = false;

	/** Indicates whether the selection of handles should be maintained across different operations. */
	UPROPERTY()
	bool bPersistHandleSelection = false;

	/** Indicates whether the indirect axes buttons are activated in a sequential manner. */
	UPROPERTY()
	bool bSequentialIndirectAxesButtons = false;

	/** Determines whether indirect axes should be combined additively. */
	UPROPERTY()
	bool bAdditiveIndirectAxes = false;

	/** Indicates whether screen-space indirect manipulation is enabled in orthographic views. */
	UPROPERTY()
	bool bScreenSpaceIndirectOrthographicManipulation = true;

	/** The mode to use for rotation (e.g. ScreenArc, Pull). */
	UPROPERTY()
	TEnumAsByte<EAxisRotateMode::Type> RotateMode = EAxisRotateMode::ScreenArc;

	/** The type of scaling to use (e.g. PercentageBased). */
	UPROPERTY()
	EGizmoTransformScaleType ScaleType = EGizmoTransformScaleType::PercentageBased;

	/** Returns the array of built-in presets. */
	static const TArray<FTransformGizmoPreset>& GetBuiltInPresets();

	/** Finds a preset by name. Returns nullptr if not found. */
	 static const FTransformGizmoPreset* FindPresetByName(FName InName);
};

namespace UE::Editor::InteractiveToolsFramework
{
	/**
	 * A single row of comparison data between current settings and a preset.
	 */
	struct FPresetSettingComparison
	{
		/** The display name of the setting (e.g. "Rotate Mode"). */
		FText SettingName;

		/** The current value of this setting, formatted for display (used for non-bool settings). */
		FText CurrentValue;

		/** The value this setting will be changed to if the preset is applied (used for non-bool settings). */
		FText PresetValue;

		/** True if the current value differs from the preset value. */
		bool bCurrentValueDiffersToPresetValue = false;

		/** True if this setting is a boolean and should be rendered as checkboxes. */
		bool bIsBool = false;

		/** The current boolean state (only valid when bIsBool is true). */
		bool bCurrentValue = false;

		/** The preset boolean state (only valid when bIsBool is true). */
		bool bPresetValue = false;
	};

	/** Provides getters/setters to get current settings, and apply them. */
	struct FTransformGizmoPresetAccessor
	{
		/** Returns true if the current settings match the given preset. */
		bool CurrentSettingsMatchPreset(const FTransformGizmoPreset& InPreset) const;

		/** Applies the given preset to the current settings. */
		void ApplyPreset(const FTransformGizmoPreset& InPreset) const;

		/** Builds a per-setting comparison between the current settings and the given preset. */
		TArray<FPresetSettingComparison> BuildComparisonWithPreset(const FTransformGizmoPreset& InPreset) const;

		/** Returns the current value of bDragDuplicateOnRotation. */
		bool GetDragDuplicateOnRotation() const;

		/** Sets the value of bDragDuplicateOnRotation. */
		void SetDragDuplicateOnRotation(bool bInValue) const;

		/** Returns the current value of bIncludeExplicitRotation. */
		bool GetIncludeExplicitRotation() const;

		/** Sets the value of bIncludeExplicitRotation. */
		void SetIncludeExplicitRotation(bool bInValue) const;

		/** Returns the current value of bUniformIndirectScale. */
		bool GetUniformIndirectScale() const;

		/** Sets the value of bUniformIndirectScale. */
		void SetUniformIndirectScale(bool bInValue) const;

		/** Returns the current value of bPersistHandleSelection. */
		bool GetPersistHandleSelection() const;

		/** Sets the value of bPersistHandleSelection. */
		void SetPersistHandleSelection(bool bInValue) const;

		/** Returns the current value of bSequentialIndirectAxesButtons. */
		bool GetSequentialIndirectAxesButtons() const;

		/** Sets the value of bSequentialIndirectAxesButtons. */
		void SetSequentialIndirectAxesButtons(bool bInValue) const;

		/** Returns the current value of bAdditiveIndirectAxes. */
		bool GetAdditiveIndirectAxes() const;

		/** Sets the value of bAdditiveIndirectAxes. */
		void SetAdditiveIndirectAxes(bool bInValue) const;

		/** Returns the current value of bScreenSpaceIndirectOrthographicManipulation. */
		bool GetScreenSpaceIndirectOrthographicManipulation() const;

		/** Sets the value of bScreenSpaceIndirectOrthographicManipulation. */
		void SetScreenSpaceIndirectOrthographicManipulation(bool bInValue) const;

		/** Returns the current RotateMode. */
		EAxisRotateMode::Type GetRotateMode() const;

		/** Sets the RotateMode. */
		void SetRotateMode(EAxisRotateMode::Type InRotateMode) const;

		/** Returns the current ScaleType. */
		EGizmoTransformScaleType GetScaleType() const;

		/** Sets the ScaleType. */
		void SetScaleType(EGizmoTransformScaleType InScaleType) const;
		
	private:
		template <typename ObjectType>
		void BroadcastPropertyChanged(ObjectType* InObject, const FName InPropertyName) const
		{
			FProperty* RootProperty = FindFProperty<FProperty>(ObjectType::StaticClass(), InPropertyName);
			if (!ensure(RootProperty))
			{
				return;
			}

			FEditPropertyChain PropertyChain;
			PropertyChain.AddHead(RootProperty);

			FPropertyChangedEvent ChangedEvent(RootProperty, EPropertyChangeType::ValueSet);
			FPropertyChangedChainEvent ChangedChainEvent(PropertyChain, ChangedEvent);
			InObject->PostEditChangeChainProperty(ChangedChainEvent);
		}
	};
	
}
