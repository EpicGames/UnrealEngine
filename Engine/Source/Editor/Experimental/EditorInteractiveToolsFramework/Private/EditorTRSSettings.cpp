// Copyright Epic Games, Inc. All Rights Reserved.

// UEditorTRSSettings and UEditorTRSEditorSettings have been merged into UTransformGizmoEditorSettings.
// This file is kept as an empty stub due to Perforce move/add constraints in the current CL.
// Delete this file in a follow-up.