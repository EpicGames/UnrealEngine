// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorToolDataVisualizer.h"

#include "BatchedElements.h"
#include "CanvasTypes.h"

namespace UE::Editor::InteractiveToolsFramework
{
	namespace ToolDataVisualizer
	{
		template <>
		FCanvasDashedLineItem::FCanvasDashedLineItem(const FVector2D& InStartPos, const FVector2D& InEndPos)
			: FCanvasLineItem(InStartPos, InEndPos)
		{
		}

		template <>
		FCanvasDashedLineItem::FCanvasDashedLineItem(const FVector& InStartPos, const FVector& InEndPos)
			: FCanvasLineItem(InStartPos, InEndPos)
		{
		}
		
		void FCanvasDashedLineItem::Draw(FCanvas* InCanvas)
		{
			if (DashLength <= 0.0f || DashGap <= 0.0f)
			{
				return;
			}
	
			FBatchedElements* BatchedElements = InCanvas->GetBatchedElements(FCanvas::ET_Line);
			FHitProxyId HitProxyId = InCanvas->GetHitProxyId();

			FVector CurrentPosition = Origin;
			bool bIsDash = true;
			float NextDistance = DashLength;

			bool bContinue = false;		
			while (!bContinue)
			{
				FVector ToEndDelta = EndPos - CurrentPosition;
				bool bFlipDash = false;
		
				FVector NextPosition;
				if (ToEndDelta.SquaredLength() < NextDistance * NextDistance)
				{
					NextPosition = EndPos;
					NextDistance = NextDistance - static_cast<float>(ToEndDelta.Length());
					bContinue = true;
				}
				else
				{
					NextPosition = CurrentPosition + ToEndDelta.GetSafeNormal() * NextDistance;
					bFlipDash = true;
					NextDistance = bIsDash ? DashGap : DashLength;
				}
		
				if (bIsDash)
				{
					BatchedElements->AddLine(CurrentPosition, NextPosition, Color, HitProxyId, LineThickness);
				}
		
				CurrentPosition = NextPosition;

				if (bFlipDash)
				{
					bIsDash = !bIsDash;
				}
			}
		}
	}
}
