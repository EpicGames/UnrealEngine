// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CanvasItem.h"

namespace UE::Editor::InteractiveToolsFramework
{
	namespace ToolDataVisualizer
	{
		/** A canvas draw item that renders a dashed line between two points, with configurable dash and gap lengths. */
		class FCanvasDashedLineItem : public FCanvasLineItem
		{
		public:
			FCanvasDashedLineItem() = default;

			template <typename VectorType>
			FCanvasDashedLineItem(const VectorType& InStartPos, const VectorType& InEndPos);

			/** Draws a dashed line on the canvas between the configured start and end positions. */
			virtual void Draw(FCanvas* InCanvas) override;

			/** The length of each dash segment, in pixels. */
			float DashLength = 1.0f;

			/** The length of the gap between dashes, in pixels. */
			float DashGap = 1.0f;
		};

		template <>
		FCanvasDashedLineItem::FCanvasDashedLineItem(const FVector2D& InStartPos, const FVector2D& InEndPos);

		template <>
		FCanvasDashedLineItem::FCanvasDashedLineItem(const FVector& InStartPos, const FVector& InEndPos);
	}
}
