// Copyright Epic Games, Inc. All Rights Reserved.


#include "InteractiveToolStack.h"

#include "ContextObjectStore.h"
#include "Delegates/Delegate.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Framework/Commands/Contexts/UIContentContext.h"
#include "Framework/Commands/Contexts/UIIdentifierContext.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "InteractiveToolManager.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Templates/Tuple.h"
#include "Textures/SlateIcon.h"
#include "ToolContexts/ToolStackContext.h"
#include "Toolkits/BaseToolkit.h"
#include "Tools/UEdMode.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/SNullWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InteractiveToolStack)

class SWidget;

#define LOCTEXT_NAMESPACE "FInteractiveToolStack"

void FInteractiveToolStack::RegisterToolStack(UEdMode* EdMode, UEditorInteractiveToolsContext* UseToolsContext, TSharedPtr<FUICommandInfo> UICommand, FString ToolStackIdentifier)
{
	// Note: The stack context is only freed when the tool context is, so should be fine to capture ptr
	UToolStackContext* ToolStackContext = nullptr;
	if (ensure(UseToolsContext && UseToolsContext->ContextObjectStore))
	{
		ToolStackContext = UseToolsContext->ContextObjectStore->FindContext<UToolStackContext>();
		if (!ensure(ToolStackContext))
		{
			return;
		}
	}

	auto ExecuteLastActiveStackTool = [ToolStackIdentifier, UseToolsContext, ToolStackContext]()
	{
		if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
		{
			const FString& Tool = ToolStack->GetLastActiveTool();
			if (!UseToolsContext->IsToolActive(EToolSide::Mouse, Tool))
			{
				ToolStack->NotifyToolActivated(Tool);
				UseToolsContext->StartTool(Tool);
			}
		}
	};

	auto CanExecuteLastActiveStackTool = [EdMode, ToolStackIdentifier, UseToolsContext, ToolStackContext]()
	{
		if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
		{
			const FString& Tool = ToolStack->GetLastActiveTool();
			return EdMode->ShouldToolStartBeAllowed(Tool) && UseToolsContext->ToolManager->CanActivateTool(EToolSide::Mouse, Tool);
		}

		return false;
	};

	auto IsLastStackToolActive = [ToolStackIdentifier, UseToolsContext, ToolStackContext]()
	{
		// Note: This is called per frame to check checkbox status
		if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
		{
			const FString& Tool = ToolStack->GetLastActiveTool();
			return UseToolsContext->IsToolActive(EToolSide::Mouse, Tool);
		}

		return false;
	};

	auto GetStackMenuContent = [ToolStackIdentifier, UseToolsContext, ToolStackContext](TSharedRef<FUICommandList> InCommandList)
	{
		TSharedRef<SWidget> StackMenu = SNullWidget::NullWidget;

		if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
		{
			const bool bShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);
			MenuBuilder.BeginSection(FName(ToolStackIdentifier), FText::FromString(ToolStackIdentifier));

			for (const TPair<FString, TSharedPtr<FUICommandInfo>>& Tool : ToolStack->GetStack())
			{
				const FString& ToolName = Tool.Key;
				const TSharedPtr<FUICommandInfo>& ToolCommand = Tool.Value;

				auto ExecuteStackTool = [ToolName, ToolStackIdentifier, UseToolsContext, ToolStackContext]()
				{
					if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
					{
						if (!UseToolsContext->IsToolActive(EToolSide::Mouse, ToolName))
						{
							ToolStack->NotifyToolActivated(ToolName);
							UseToolsContext->StartTool(ToolName);
						}
					}
				};

				FMenuEntryParams MenuEntryParams;
				MenuEntryParams.LabelOverride = ToolCommand->GetLabel();
				MenuEntryParams.ToolTipOverride = ToolCommand->GetDescription();
				MenuEntryParams.IconOverride = ToolCommand->GetIcon();
				MenuEntryParams.InputBindingOverride = ToolCommand->GetDefaultChord(EMultipleKeyBindingIndex::Primary).GetInputText();
				MenuEntryParams.DirectActions = { FExecuteAction::CreateWeakLambda(UseToolsContext, ExecuteStackTool) };
				MenuBuilder.AddMenuEntry(MenuEntryParams);
			}

			return MenuBuilder.MakeWidget();
		}

		return StackMenu;
	};

	auto GetStackIcon = [ToolStackIdentifier, UseToolsContext, UICommand, ToolStackContext]()
	{
		if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
		{
			return ToolStack->GetLastActiveToolIcon();
		}

		return UICommand->GetIcon();
	};

	auto GetStackLabel = [ToolStackIdentifier, UseToolsContext, UICommand, ToolStackContext]()
	{
		if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
		{
			return ToolStack->GetLastActiveToolLabel();
		}

		return UICommand->GetLabel();
	};

	auto GetStackDescription = [ToolStackIdentifier, UseToolsContext, UICommand, ToolStackContext]()
	{
		if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
		{
			return ToolStack->GetLastActiveToolDescription();
		}

		return UICommand->GetDescription();
	};
	
	TSharedPtr<FModeToolkit> Toolkit = EdMode->GetToolkit().Pin();
	if (ensure(Toolkit))
	{
		const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

		FUIAction StackAction(
			FExecuteAction::CreateWeakLambda(UseToolsContext, ExecuteLastActiveStackTool),
			FCanExecuteAction::CreateWeakLambda(UseToolsContext, CanExecuteLastActiveStackTool),
			FIsActionChecked::CreateWeakLambda(UseToolsContext, IsLastStackToolActive),
			EUIActionRepeatMode::RepeatDisabled
		);

		FUIActionContext StackActionContext;

		FUIContentContext StackContent;
		StackContent.OnGetContent = FOnGetContent::CreateWeakLambda(UseToolsContext, GetStackMenuContent, CommandList);
		StackActionContext.AddContext(MakeShared<FUIContentContext>(StackContent));

		FUIIdentifierContext StackIdentifier;
		StackIdentifier.OnGetContextIcon = FOnGetContextIcon::CreateWeakLambda(UseToolsContext, GetStackIcon);
		StackIdentifier.OnGetContextabel = FOnGetContextText::CreateWeakLambda(UseToolsContext, GetStackLabel);
		StackIdentifier.OnGetContextDescription = FOnGetContextText::CreateWeakLambda(UseToolsContext, GetStackDescription);
		StackActionContext.AddContext(MakeShared<FUIIdentifierContext>(StackIdentifier));

		CommandList->MapAction(UICommand, StackAction, StackActionContext);
	}
}

void FInteractiveToolStack::AddToolToStack(UEdMode* EdMode, UEditorInteractiveToolsContext* UseToolsContext, TSharedPtr<FUICommandInfo> UICommand, FString ToolStackIdentifier, FString ToolIdentifier, UInteractiveToolBuilder* Builder)
{
	// Note: The stack context is only freed when the tool context is, so should be fine to capture ptr
	UToolStackContext* ToolStackContext = nullptr;
	if (ensure(UseToolsContext && UseToolsContext->ContextObjectStore))
	{
		ToolStackContext = UseToolsContext->ContextObjectStore->FindContext<UToolStackContext>();
		if (!ensure(ToolStackContext))
		{
			return;
		}
	}

	if (ensure(UseToolsContext))
	{
		if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
		{
			ToolStack->AddTool(ToolIdentifier, UICommand);
			UseToolsContext->ToolManager->RegisterToolType(ToolIdentifier, Builder);

			auto ExecuteLastToolForAction = [EdMode, ToolStackIdentifier, UseToolsContext, UICommand, ToolStackContext]()
			{
				if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
				{
					const FInputChord& InputChord = UICommand->GetDefaultChord(EMultipleKeyBindingIndex::Primary);
					const FString& Tool = ToolStack->GetLastActiveToolByChord(InputChord);

					if (!UseToolsContext->IsToolActive(EToolSide::Mouse, Tool))
					{
						ToolStack->NotifyToolActivated(Tool);
						UseToolsContext->StartTool(Tool);
					}
				}
			};

			auto CanExecuteLastToolForAction = [EdMode, ToolStackIdentifier, UseToolsContext, UICommand, ToolStackContext]()
			{
				if (FInteractiveToolStack* ToolStack = ToolStackContext->AccessToolStack(ToolStackIdentifier))
				{
					// @TODO: This leaves us in a possible "deadlock" where the last active tool for a chord can't be used,
					// but other tools for a chord are valid. Seemingly unlikely scenario so address later if needed.

					const FInputChord& InputChord = UICommand->GetDefaultChord(EMultipleKeyBindingIndex::Primary);
					const FString& Tool = ToolStack->GetLastActiveToolByChord(InputChord);
					return EdMode->ShouldToolStartBeAllowed(Tool) && UseToolsContext->ToolManager->CanActivateTool(EToolSide::Mouse, Tool);
				}

				return false;
			};

			TSharedPtr<FModeToolkit> Toolkit = EdMode->GetToolkit().Pin();
			if (ensure(Toolkit))
			{
				const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
				CommandList->MapAction(UICommand,
					FExecuteAction::CreateWeakLambda(UseToolsContext, ExecuteLastToolForAction),
					FCanExecuteAction::CreateWeakLambda(UseToolsContext, CanExecuteLastToolForAction),
					FIsActionChecked(),
					EUIActionRepeatMode::RepeatDisabled);
			}
		}
	}
}

int32 FInteractiveToolStack::AddTool(const FString& InToolName, const TSharedPtr<FUICommandInfo>& InUICommand)
{
	// Store original stack order for future display
	OriginalStack.Add(TPair<FString, TSharedPtr<FUICommandInfo>>{InToolName, InUICommand});

	// Insert in reverse order, so later on we can last element for most recently used
	// Note: Not using queue since we have to arbitrarily remove tool 'n'
	return SortedToolStack.Insert(TPair<FString, TSharedPtr<FUICommandInfo>>{InToolName, InUICommand}, 0);
}

void FInteractiveToolStack::NotifyToolActivated(const FString& InToolName)
{
	auto FindByToolName = [&](const TPair<FString, TSharedPtr<FUICommandInfo>>& Element)
	{
		return Element.Key == InToolName;
	};

	TPair<FString, TSharedPtr<FUICommandInfo>> ActivatedTool = *SortedToolStack.FindByPredicate(FindByToolName);
	SortedToolStack.Remove(ActivatedTool);
	SortedToolStack.Add(ActivatedTool);
}

const FString FInteractiveToolStack::GetLastActiveToolByChord(const FInputChord& InInputChord) const
{
	// Note: Pass by value intentional, this value is volatile don't want to pass ref.
	
	// Most recently used tools at end, traverse array in reverse order
	auto StackIter = SortedToolStack.end();
	while (StackIter != SortedToolStack.begin())
	{
		--StackIter;

		const TPair<FString, TSharedPtr<FUICommandInfo>>& Tool = *StackIter;
		const FString& ToolName = Tool.Key;
		const TSharedPtr<FUICommandInfo>& UICommand = Tool.Value;

		// Test raw chord values, not references
		if (UICommand.IsValid() && UICommand->GetDefaultChord(EMultipleKeyBindingIndex::Primary) == InInputChord)
		{
			return ToolName;
		}
	}

	ensureMsgf(false, TEXT("Input chord has no corresponding tool: %s"), *InInputChord.GetInputText().ToString());
	return GetLastActiveTool();
}

const FString FInteractiveToolStack::GetLastActiveTool() const
{
	// Note: Pass by value intentional, this value is volatile don't want to pass ref.
	return SortedToolStack.Last().Key;
}

const FSlateIcon& FInteractiveToolStack::GetLastActiveToolIcon() const
{
	return SortedToolStack.Last().Value->GetIcon();
}

const FText& FInteractiveToolStack::GetLastActiveToolLabel() const
{
	return SortedToolStack.Last().Value->GetLabel();
}

const FText& FInteractiveToolStack::GetLastActiveToolDescription() const
{
	return SortedToolStack.Last().Value->GetDescription();
}

const TArray<TPair<FString, TSharedPtr<FUICommandInfo>>>& FInteractiveToolStack::GetStack() const
{
	return OriginalStack;
}

#undef LOCTEXT_NAMESPACE
