// Copyright Epic Games, Inc. All Rights Reserved.

#include "Snapping/EditorSnappingManager.h"

#include "Algo/RemoveIf.h"
#include "CollisionQueryFilterCallbackCore.h"
#include "CollisionQueryParams.h"
#include "Components/PrimitiveComponent.h"
#include "ContextObjectStore.h"
#include "DrawDebugHelpers.h"
#include "Engine/World.h"
#include "GridSnapping.h"
#include "InteractiveToolsContext.h"
#include "PolicySnapping.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "ToolContextInterfaces.h"
#include "TransformSnapping.h"
#include "VectorUtil.h"
#include "ViewportSnappingModule.h"
#include "EditorViewportClient.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorSnappingManager)

namespace UE::Editor::Gizmos
{
	bool RegisterSceneSnappingManager(UInteractiveToolsContext* InToolsContext)
	{
		if (!ensure(InToolsContext))
		{
			return false;
		}

		// Check for existing registration, and return true if found
		const UEditorSceneSnappingManager* FoundRegisteredSnappingManager = InToolsContext->ContextObjectStore->FindContext<UEditorSceneSnappingManager>();
		if (FoundRegisteredSnappingManager)
		{
			return true;
		}

		UEditorSceneSnappingManager* SnappingManager = NewObject<UEditorSceneSnappingManager>(InToolsContext->ToolManager);
		if (!ensure(SnappingManager))
		{
			return false;
		}

		SnappingManager->Initialize(InToolsContext);
		InToolsContext->ContextObjectStore->AddContextObject(SnappingManager);

		return true;
	}

	bool DeregisterSceneSnappingManager(const UInteractiveToolsContext* InToolsContext)
	{
		if (!ensure(InToolsContext))
		{
			return false;
		}

		if (UEditorSceneSnappingManager* FoundRegisteredSnappingManager = InToolsContext->ContextObjectStore->FindContext<UEditorSceneSnappingManager>())
		{
			FoundRegisteredSnappingManager->Shutdown();
			InToolsContext->ContextObjectStore->RemoveContextObject(FoundRegisteredSnappingManager);
		}

		return true;
	}

	UEditorSceneSnappingManager* FindSceneSnappingManager(const UInteractiveToolManager* InToolManager)
	{
		if (!ensure(InToolManager))
		{
			return nullptr;
		}

		if (UEditorSceneSnappingManager* FoundRegisteredSnappingManager = InToolManager->GetContextObjectStore()->FindContext<UEditorSceneSnappingManager>())
		{
			return FoundRegisteredSnappingManager;
		}

		return nullptr;
	}

	namespace Private
	{
		class FEditorSceneSnappingManagerDebug : public FTickableEditorObject
		{
		public:
			static constexpr bool bDebugEnabled = false;

		public:
			//~ Begin FTickableEditorObject
			virtual void Tick(float DeltaTime) override
			{
				if (!QueriesAPI || !DebugData.bShouldTick)
				{
					return;
				}

				if (UWorld* World = QueriesAPI->GetCurrentEditingWorld();
					DebugData.bHasValidHit && World)
				{
					const FColor DebugColor = FColor::Cyan;
					DrawDebugDirectionalArrow(World, DebugData.LineStart, DebugData.LineEnd, 80.0f, DebugColor);
					DrawDebugPoint(World, DebugData.Point, 10.0f, FColor::Cyan);
					DrawDebugCrosshairs(World, DebugData.Point, FRotator::ZeroRotator, 1000.0f, FColor::Red);

					FVector Axis0 = DebugData.AxisY;
					FVector Axis1 = DebugData.AxisX;
					DrawDebugCircle(World, DebugData.Point, 100.0f, 16, DebugColor, false, -1, 0, 0, Axis0, Axis1);

					// DrawDebugString(World, DebugData.Point, DebugData.HitLabel, nullptr, DebugColor, -1, true, 100);
					// DrawDebugCircle(World,)
				}
			}

			TStatId GetStatId() const
			{
				RETURN_QUICK_DECLARE_CYCLE_STAT(UEditorSceneSnappingManager, STATGROUP_Tickables);
			}

			ETickableTickType GetTickableTickType() const
			{
				return ETickableTickType::Conditional;
			}

			bool IsTickable() const
			{
				return DebugData.bShouldTick && FTickableEditorObject::IsTickable();
			}
			//~ End FTickableEditorObject

		public:
			const IToolsContextQueriesAPI* QueriesAPI = nullptr;

			mutable struct FDebugData
			{
				std::atomic_bool bShouldTick = false;

				bool bHasValidHit = false;
				FVector LineStart;
				FVector LineEnd;
				FVector Point;
				FVector Normal;
				FVector AxisX;
				FVector AxisY;
				FString HitLabel;

				void Reset()
				{
					bHasValidHit = false;
					LineStart = FVector::ZeroVector;
					LineEnd = FVector::ZeroVector;
					Point = FVector::ZeroVector;
					Normal = FVector::ForwardVector;
					AxisX = FVector::RightVector;
					AxisY = FVector::UpVector;
					HitLabel = TEXT("");
				}
			} DebugData;
		};
	}
}

UEditorSceneSnappingManager::UEditorSceneSnappingManager()
{
	Debug = MakeUnique<UE::Editor::Gizmos::Private::FEditorSceneSnappingManagerDebug>();
}

void UEditorSceneSnappingManager::Initialize(const TObjectPtr<UInteractiveToolsContext>& InToolsContext)
{
	UInteractiveToolManager* ToolManager = InToolsContext ? InToolsContext->ToolManager : nullptr;
	QueriesAPI = ToolManager ? ToolManager->GetContextQueriesAPI() : nullptr;

	RegisterSnapQueryTargetHandler(MakeUnique<UE::Editor::Gizmos::FGizmoGridSnapper>(InToolsContext));
	RegisterSnapQueryTargetHandler(MakeUnique<UE::Editor::Gizmos::FGizmoTransformSnapper>(InToolsContext));

	Debug->QueriesAPI = QueriesAPI;
	Debug->DebugData.bShouldTick = true;
}

void UEditorSceneSnappingManager::Shutdown()
{
	Debug->DebugData.bShouldTick = false;
	Debug->QueriesAPI = nullptr;

	QueryTargetHandlers.Reset();

	QueriesAPI = nullptr;
}

bool UEditorSceneSnappingManager::ExecuteSceneHitQueryPrimitiveTransform(const FSceneHitQueryRequest& InRequest, TArray<FSceneHitQueryResult>& OutResults) const
{
	const FCollisionObjectQueryParams ObjectQueryParams(FCollisionObjectQueryParams::AllObjects);
	FCollisionQueryParams QueryParams = FCollisionQueryParams::DefaultQueryParam;
	QueryParams.bTraceComplex = true;
	QueryParams.bReturnFaceIndex = false;

	const UWorld* World = QueriesAPI->GetCurrentEditingWorld();
	if (!World)
	{
		return false;
	}

	TArray<FHitResult> HitResults;
	const FVector RayEnd = InRequest.WorldRay.PointAt(HALF_WORLD_MAX);
	if (World->LineTraceMultiByObjectType(HitResults, (FVector)InRequest.WorldRay.Origin, RayEnd, ObjectQueryParams, QueryParams))
	{
		HitResults.SetNum(Algo::RemoveIf(HitResults, [&](const FHitResult& InHitResult)
		{
			return !InRequest.VisibilityFilter.IsVisible(InHitResult.Component.Get());
		}));

		if (HitResults.IsEmpty())
		{
			return false;
		}

		OutResults.SetNum(HitResults.Num());
		Algo::Transform(HitResults, OutResults, [&](const FHitResult& InHitResult)
		{
			FSceneHitQueryResult HitQueryResult;
			HitQueryResult.HitResult = InHitResult;
			HitQueryResult.TargetActor = InHitResult.GetActor();
			HitQueryResult.TargetComponent = InHitResult.GetComponent();
			HitQueryResult.Position = InHitResult.GetActor() ? InHitResult.GetActor()->GetActorLocation() : FVector(InHitResult.ImpactPoint);

			return HitQueryResult;
		});

		// Update Debug
		using namespace UE::Editor::Gizmos::Private;
		if (FEditorSceneSnappingManagerDebug::bDebugEnabled && !OutResults.IsEmpty())
		{
			Algo::StableSort(HitResults, [WorldRay = InRequest.WorldRay](const FHitResult& A, const FHitResult& B)
			{
				return FVector::Distance(WorldRay.Origin, A.GetActor()->GetActorLocation()) < FVector::Distance(WorldRay.Origin, B.GetActor()->GetActorLocation());
			});

			const FHitResult* FirstValidHitResult = Algo::FindBy(
				HitResults,
				true,
				[&](const FHitResult& InHitResult)
				{
					return InHitResult.GetActor() != nullptr;
				});

			Debug->DebugData.Reset();

			if (FirstValidHitResult)
			{
				FViewCameraState CameraState;
				QueriesAPI->GetCurrentViewState(CameraState);

				FPlane HitPlane(FirstValidHitResult->GetActor()->GetActorLocation(), CameraState.Forward());

				FEditorSceneSnappingManagerDebug::FDebugData& DebugData = Debug->DebugData;

				DebugData.bHasValidHit = true;
				DebugData.Point = FirstValidHitResult->GetActor()->GetActorLocation();
				DebugData.Normal = HitPlane.GetNormal();
				DebugData.AxisX = CameraState.Up();
				DebugData.AxisY = CameraState.Right();
				DebugData.LineStart = InRequest.WorldRay.Origin;
				DebugData.LineEnd = FirstValidHitResult->Location;
				DebugData.HitLabel = FString::Printf(TEXT("Actor: %s"), *FirstValidHitResult->GetActor()->GetActorLabel());

				// UE_LOGF(LogTemp, Warning, "Hit: %ls", *OutResult.HitResult.GetActor()->GetActorLabel());
			}
		}

		return true;
	}

	return false;
}

bool UEditorSceneSnappingManager::ExecuteSceneHitQuery(const FSceneHitQueryRequest& InRequest, FSceneHitQueryResult& OutResult) const
{
	TArray<FSceneHitQueryResult> HitResults;
	const bool bResult = ExecuteSceneHitQuery(InRequest, HitResults);
	if (bResult && !HitResults.IsEmpty())
	{
		OutResult = HitResults[0];
	}

	return bResult;
}

bool UEditorSceneSnappingManager::ExecuteSceneHitQuery(const FSceneHitQueryRequest& InRequest, TArray<FSceneHitQueryResult>& OutResults) const
{
	if (!QueriesAPI)
	{
		return false;
	}

	// Actor/Primitive Transform
	if (EnumHasAnyFlags(InRequest.TargetTypes, ESceneHitQueryTargetType::PrimitiveTransform))
	{
		if (ExecuteSceneHitQueryPrimitiveTransform(InRequest, OutResults))
		{
			return true;
		}
	}

	return false;
}

bool UEditorSceneSnappingManager::ExecuteSceneSnapQuery(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const
{
	if (!QueriesAPI)
	{
		return false;
	}

	// Early-exit if snapping is globally disabled. Editor Snapping always operates on current settings
	if (!QueriesAPI->GetCurrentSnappingSettings().bIsSnappingActive)
	{
		return false;
	}

	switch (InRequest.RequestType)
	{
	case ESceneSnapQueryType::Position:
		return ExecuteSceneSnapQueryPosition(InRequest, OutResults);
	case ESceneSnapQueryType::Rotation:
		return ExecuteSceneSnapQueryRotation(InRequest, OutResults);
	case ESceneSnapQueryType::RotationAngle:
		return ExecuteSceneSnapQueryRotationAngle(InRequest, OutResults);
	case ESceneSnapQueryType::Scale:
		return ExecuteSceneSnapQueryScale(InRequest, OutResults);
	default:
		ensureMsgf(false, TEXT("Only Position, Rotation and Scale Snap Queries are supported"));
	}

	return false;
}

void UEditorSceneSnappingManager::RegisterSnapQueryTargetHandler(TUniquePtr<FSceneSnapQueryTargetHandler>&& InHandler)
{
	QueryTargetHandlers.Emplace(MoveTemp(InHandler));
}

void UEditorSceneSnappingManager::DeregisterSnapQueryTargetHandler(const FName InHandlerName)
{
	const int32 HandlerIdx = QueryTargetHandlers.IndexOfByPredicate(
			[InHandlerName](const TUniquePtr<FSceneSnapQueryTargetHandler>& InHandler)
			{
				return InHandler->TargetName == InHandlerName;
			});
	
	if (HandlerIdx != INDEX_NONE)
	{
		QueryTargetHandlers.RemoveAt(HandlerIdx);
	}
}

bool UEditorSceneSnappingManager::ExecuteSceneSnapQueryPosition(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const
{
	using namespace UE::Editor::Gizmos;

	if (!QueriesAPI)
	{
		return false;
	}

	for (const TUniquePtr<FSceneSnapQueryTargetHandler>& Handler : QueryTargetHandlers)
	{
		if (Handler->IsQueryTypeSupported(ESceneSnapQueryType::Position))
		{
			if (Handler->SnapPosition(InRequest, OutResults) == ESceneSnapQueryTargetResult::Snapped)
			{
				return true;
			}
		}
	}

	return false;
}

bool UEditorSceneSnappingManager::ExecuteSceneSnapQueryRotation(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const
{
	using namespace UE::Editor::Gizmos;

	if (!QueriesAPI)
	{
		return false;
	}

	for (const TUniquePtr<FSceneSnapQueryTargetHandler>& Handler : QueryTargetHandlers)
	{
		if (Handler->IsQueryTypeSupported(ESceneSnapQueryType::Rotation))
		{
			if (Handler->SnapRotation(InRequest, OutResults) == ESceneSnapQueryTargetResult::Snapped)
			{
				return true;
			}
		}
	}

	return false;
}

bool UEditorSceneSnappingManager::ExecuteSceneSnapQueryRotationAngle(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const
{
	using namespace UE::Editor::Gizmos;

	if (!QueriesAPI)
	{
		return false;
	}

	// It's assumed the input AxisList only has a single active axis
	const EAxis::Type Axis = EAxis::FromAxisList(InRequest.AxisList);
	if (Axis == EAxis::None && InRequest.AxisList != EAxisList::Screen)
	{
		return false;
	}

	for (const TUniquePtr<FSceneSnapQueryTargetHandler>& Handler : QueryTargetHandlers)
	{
		if (Handler->IsQueryTypeSupported(ESceneSnapQueryType::RotationAngle))
		{
			double SnappedAngle;
			if (Handler->SnapRotationAxisAngle(InRequest.RotationAngle, SnappedAngle, InRequest.AxisList) == ESceneSnapQueryTargetResult::Snapped)
			{
				FSceneSnapQueryResult SnapResult;
				SnapResult.TargetType = Handler->GetTargetTypes();
				SnapResult.RotationAngle = SnappedAngle;

				OutResults.Emplace(SnapResult);

				return true;
			}
		}
	}

	return false;
}

bool UEditorSceneSnappingManager::ExecuteSceneSnapQueryScale(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const
{
	using namespace UE::Editor::Gizmos;

	if (!QueriesAPI)
	{
		return false;
	}

	for (const TUniquePtr<FSceneSnapQueryTargetHandler>& Handler : QueryTargetHandlers)
	{
		if (Handler->IsQueryTypeSupported(ESceneSnapQueryType::Scale))
		{
			if (Handler->SnapScale(InRequest, OutResults) == ESceneSnapQueryTargetResult::Snapped)
			{
				return true;
			}
		}
	}

	return false;
}
