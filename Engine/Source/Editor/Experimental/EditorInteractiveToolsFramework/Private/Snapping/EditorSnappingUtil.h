// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"
#include "ToolContextInterfaces.h"

namespace UE::EditorSnappingUtil
{
	/**
	 * Utility function to get all components of the given type from one or more Actors.
	 * @return true if any components found.
	 */
	template <typename ComponentType UE_REQUIRES(std::is_base_of_v<UActorComponent, std::decay_t<ComponentType>>)>
	bool GetActorComponents(const TConstArrayView<AActor*>& InActors, TArray<const ComponentType*>& OutComponents)
	{
		OutComponents.Reset();
		OutComponents.Reserve(InActors.Num());

		for (const AActor* Actor : InActors)
		{
			if (!Actor)
			{
				continue;
			}

			TArray<const ComponentType*> ComponentsInActor;
			Actor->GetComponents<const ComponentType>(ComponentsInActor, true);

			OutComponents.Append(ComponentsInActor);
		}

		return !OutComponents.IsEmpty();
	}
}
