// Copyright Epic Games, Inc. All Rights Reserved.

#include "GridSnapping.h"

#include "BaseGizmos/GizmoMath.h"
#include "InteractiveToolsContext.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "ToolContextInterfaces.h"

namespace UE::Editor::Gizmos
{
	FGizmoGridSnapper::FGizmoGridSnapper(const UInteractiveToolsContext* InToolsContext)
	{
		TargetName = "Grid";

		UInteractiveToolManager* ToolManager = InToolsContext ? InToolsContext->ToolManager.Get() : nullptr;
		QueriesAPI = ToolManager ? ToolManager->GetContextQueriesAPI() : nullptr;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapPosition(const FGizmoGridSnapperRequest& InTargetRequest, const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList) const
	{
		OutSnappedPosition = InPosition;

		const bool bSnapX = InAxisList & EAxisList::X || InAxisList & EAxisList::Screen;
		const bool bSnapY = InAxisList & EAxisList::Y || InAxisList & EAxisList::Screen;
		const bool bSnapZ = InAxisList & EAxisList::Z || InAxisList & EAxisList::Screen;

		if (!bSnapX && !bSnapY && !bSnapZ)
		{
			return ESceneSnapQueryTargetResult::NotSnapped;
		}

		const FVector GridSize = InTargetRequest.PositionGridSize;

		OutSnappedPosition.X = bSnapX ? ::GizmoMath::SnapToIncrement(InPosition.X, GridSize.X) : InPosition.X;
		OutSnappedPosition.Y = bSnapY ? ::GizmoMath::SnapToIncrement(InPosition.Y, GridSize.Y) : InPosition.Y;
		OutSnappedPosition.Z = bSnapZ ? ::GizmoMath::SnapToIncrement(InPosition.Z, GridSize.Z) : InPosition.Z;

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapPosition(const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList) const
	{
		OutSnappedPosition = InPosition;

		if (!QueriesAPI)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		if (const FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();
			SnappingConfig.IsPositionGridSnappingActive())
		{
			FGizmoGridSnapperRequest GridRequest;
			GridRequest.PositionGridSize = SnappingConfig.PositionGridDimensions;

			return SnapPosition(GridRequest, InPosition, OutSnappedPosition, InAxisList);
		}

		return ESceneSnapQueryTargetResult::Disabled;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapPositionAxis(const FGizmoGridSnapperRequest& InTargetRequest, const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const
	{
		OutSnappedAxisValue = InAxisValue;

		if (!ensure(InAxis != EAxisList::Screen))
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		const FVector GridSize = InTargetRequest.PositionGridSize;

		OutSnappedAxisValue = ::GizmoMath::SnapToIncrement(InAxisValue, GridSize.GetComponentForAxis(EAxis::FromAxisList(InAxis)));

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapPositionAxis(const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const
	{
		OutSnappedAxisValue = InAxisValue;

		if (!QueriesAPI)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		if (const FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();
			SnappingConfig.IsPositionGridSnappingActive())
		{
			FGizmoGridSnapperRequest GridRequest;
			GridRequest.PositionGridSize = SnappingConfig.PositionGridDimensions;

			return SnapPositionAxis(GridRequest, InAxisValue, OutSnappedAxisValue, InAxis);
		}

		return ESceneSnapQueryTargetResult::Disabled;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapRotation(const FGizmoGridSnapperRequest& InTargetRequest, const FQuat& InRotation, FQuat& OutSnappedRotation, const EAxisList::Type InAxisList) const
	{
		OutSnappedRotation = InRotation;

		const bool bSnapRoll = InAxisList & EAxisList::X || InAxisList & EAxisList::Screen;
		const bool bSnapPitch = InAxisList & EAxisList::Y || InAxisList & EAxisList::Screen;
		const bool bSnapYaw = InAxisList & EAxisList::Z || InAxisList & EAxisList::Screen;

		if (!bSnapRoll && !bSnapPitch && !bSnapYaw)
		{
			return ESceneSnapQueryTargetResult::NotSnapped;
		}

		const FRotator GridSize = InTargetRequest.RotationGridSize;

		UE::Math::TVector<double> Axis;
		double AngleRad;
		InRotation.ToAxisAndAngle(Axis, AngleRad);

		double AngleDeg = FMath::RadiansToDegrees(AngleRad);

		AngleDeg = ::GizmoMath::SnapToIncrement(AngleDeg, GridSize.Pitch);
		AngleRad = FMath::DegreesToRadians(AngleDeg);

		OutSnappedRotation = FQuat(Axis, AngleRad);

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapRotation(const FQuat& InRotation, FQuat& OutSnappedRotation, const EAxisList::Type InAxisList) const
	{
		OutSnappedRotation = InRotation;
		
		if (!QueriesAPI)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		if (const FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();
			SnappingConfig.IsRotationGridSnappingActive())
		{
			FGizmoGridSnapperRequest GridRequest;
			GridRequest.RotationGridSize = SnappingConfig.RotationGridAngles;

			return SnapRotation(GridRequest, InRotation, OutSnappedRotation, InAxisList);
		}

		return ESceneSnapQueryTargetResult::Disabled;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapRotationAxisAngle(const FGizmoGridSnapperRequest& InTargetRequest, const double& InAngle, double& OutSnappedAngle, const EAxisList::Type InAxis) const
	{
		OutSnappedAngle = InAngle;

		const FRotator GridSize = InTargetRequest.RotationGridSize;

		// If we're in screen space, just use the first component (pitch)
		const double SnapDelta = InAxis == EAxisList::Screen ? GridSize.Pitch : GridSize.GetComponentForAxis(EAxis::FromAxisList(InAxis));

		OutSnappedAngle = FMath::RadiansToDegrees(InAngle);
		OutSnappedAngle = ::GizmoMath::SnapToIncrement(OutSnappedAngle, SnapDelta);
		OutSnappedAngle = FMath::DegreesToRadians(OutSnappedAngle);

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapRotationAxisAngle(const double& InAngle, double& OutSnappedAngle, const EAxisList::Type InAxis) const
	{
		OutSnappedAngle = InAngle;
		
		if (!QueriesAPI)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		if (const FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();
			SnappingConfig.IsRotationGridSnappingActive())
		{
			FSceneSnapQueryResult SnapResult;
			SnapResult.TargetType = ESceneSnapQueryTargetType::Grid;

			FGizmoGridSnapperRequest GridRequest;
			GridRequest.RotationGridSize = SnappingConfig.RotationGridAngles;

			return SnapRotationAxisAngle(GridRequest, InAngle, OutSnappedAngle, InAxis);
		}

		return ESceneSnapQueryTargetResult::Disabled;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapScale(const FGizmoGridSnapperRequest& InTargetRequest, const FVector& InScale, FVector& OutSnappedScale, const EAxisList::Type InAxisList) const
	{
		OutSnappedScale = InScale;

		const bool bSnapX = InAxisList & EAxisList::X || InAxisList & EAxisList::Screen;
		const bool bSnapY = InAxisList & EAxisList::Y || InAxisList & EAxisList::Screen;
		const bool bSnapZ = InAxisList & EAxisList::Z || InAxisList & EAxisList::Screen;

		if (!bSnapX && !bSnapY && !bSnapZ)
		{
			return ESceneSnapQueryTargetResult::NotSnapped;
		}

		const FVector GridSize = InTargetRequest.ScaleGridSize;

		OutSnappedScale.X = bSnapX ? ::GizmoMath::SnapToIncrement(InScale.X, GridSize.X) : InScale.X;
		OutSnappedScale.Y = bSnapY ? ::GizmoMath::SnapToIncrement(InScale.Y, GridSize.Y) : InScale.Y;
		OutSnappedScale.Z = bSnapZ ? ::GizmoMath::SnapToIncrement(InScale.Z, GridSize.Z) : InScale.Z;

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapScale(const FVector& InScale, FVector& OutSnappedScale, const EAxisList::Type InAxisList) const
	{
		OutSnappedScale = InScale;

		if (!QueriesAPI)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		if (const FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();
			SnappingConfig.IsScaleGridSnappingActive())
		{
			FSceneSnapQueryResult SnapResult;
			SnapResult.TargetType = ESceneSnapQueryTargetType::Grid;

			FGizmoGridSnapperRequest GridRequest;
			GridRequest.ScaleGridSize = FVector(SnappingConfig.ScaleGridSize);

			return SnapScale(GridRequest, InScale, OutSnappedScale, InAxisList);
		}

		return ESceneSnapQueryTargetResult::Disabled;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapScaleAxis(const FGizmoGridSnapperRequest& InTargetRequest, const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const
	{
		OutSnappedAxisValue = InAxisValue;

		if (!ensure(InAxis != EAxisList::Screen))
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		const FVector GridSize = InTargetRequest.ScaleGridSize;

		OutSnappedAxisValue = ::GizmoMath::SnapToIncrement(InAxisValue, GridSize.GetComponentForAxis(EAxis::FromAxisList(InAxis)));

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoGridSnapper::SnapScaleAxis(const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const
	{
		OutSnappedAxisValue = InAxisValue;

		if (!QueriesAPI)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		if (const FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();
			SnappingConfig.IsScaleGridSnappingActive())
		{
			FSceneSnapQueryResult SnapResult;
			SnapResult.TargetType = ESceneSnapQueryTargetType::Grid;

			FGizmoGridSnapperRequest GridRequest;
			GridRequest.ScaleGridSize = FVector(SnappingConfig.ScaleGridSize);

			return SnapScaleAxis(GridRequest, InAxisValue, OutSnappedAxisValue, InAxis);
		}

		return ESceneSnapQueryTargetResult::Disabled;
	}

	bool FGizmoGridSnapper::IsQueryTypeSupported(const ESceneSnapQueryType InQueryType) const
	{
		return InQueryType != ESceneSnapQueryType::Transform;
	}
}
