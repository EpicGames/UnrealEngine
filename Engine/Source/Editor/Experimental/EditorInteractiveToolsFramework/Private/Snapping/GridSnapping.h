// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "SceneQueries/SceneSnappingManager.h"

class UInteractiveToolsContext;
struct FSceneSnapQueryResult;
struct FSceneSnapQueryRequest;
struct FSceneHitQueryResult;
struct FSceneHitQueryRequest;
class USceneSnappingManager;
class UInteractiveGizmoManager;
class IToolsContextQueriesAPI;

class UEditorSceneSnappingManager;

namespace UE::Editor::Gizmos
{
	struct FGizmoGridSnapperRequest
	{
		FVector PositionGridSize;
		FRotator RotationGridSize;
		FVector ScaleGridSize;
	};

	class FGizmoGridSnapper final : public TSceneSnapQueryTargetHandler<FGizmoGridSnapperRequest>
	{
	public:
		explicit FGizmoGridSnapper(const UInteractiveToolsContext* InToolsContext);
		virtual ~FGizmoGridSnapper() override = default;

		//~ Begin FSceneSnapPolicyTargetHandler
		virtual ESceneSnapQueryTargetResult SnapPosition(const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList = EAxisList::All) const override;
		virtual ESceneSnapQueryTargetResult SnapPositionAxis(const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const override;
		virtual ESceneSnapQueryTargetResult SnapRotation(const FQuat& InRotation, FQuat& OutSnappedRotation, const EAxisList::Type InAxisList = EAxisList::All) const override;
		virtual ESceneSnapQueryTargetResult SnapRotationAxisAngle(const double& InAngle, double& OutSnappedAngle, const EAxisList::Type InAxis) const override;
		virtual ESceneSnapQueryTargetResult SnapScale(const FVector& InScale, FVector& OutSnappedScale, const EAxisList::Type InAxisList = EAxisList::All) const override;
		virtual ESceneSnapQueryTargetResult SnapScaleAxis(const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const override;
		virtual bool IsQueryTypeSupported(const ESceneSnapQueryType InQueryType) const override;
		//~ End FSceneSnapPolicyTargetHandler

		//~ Begin TSceneSnapQueryTargetHandler
		virtual ESceneSnapQueryTargetResult SnapPosition(const FGizmoGridSnapperRequest& InTargetRequest, const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList = EAxisList::All) const override;
		virtual ESceneSnapQueryTargetResult SnapPositionAxis(const FGizmoGridSnapperRequest& InTargetRequest, const double InAxisValue, double& OutSnappedPositionAxis, const EAxisList::Type InAxis) const override;
		virtual ESceneSnapQueryTargetResult SnapRotation(const FGizmoGridSnapperRequest& InTargetRequest, const FQuat& InRotation, FQuat& OutSnappedRotation, const EAxisList::Type InAxisList = EAxisList::All) const override;
		virtual ESceneSnapQueryTargetResult SnapRotationAxisAngle(const FGizmoGridSnapperRequest& InTargetRequest, const double& InAngle, double& OutSnappedAngle, const EAxisList::Type InAxis) const override;
		virtual ESceneSnapQueryTargetResult SnapScale(const FGizmoGridSnapperRequest& InTargetRequest, const FVector& InScale, FVector& OutSnappedScale, const EAxisList::Type InAxisList = EAxisList::All) const override;
		virtual ESceneSnapQueryTargetResult SnapScaleAxis(const FGizmoGridSnapperRequest& InTargetRequest, const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const override;
		//~ End TSceneSnapQueryTargetHandler

	private:
		const IToolsContextQueriesAPI* QueriesAPI = nullptr;
	};
}
