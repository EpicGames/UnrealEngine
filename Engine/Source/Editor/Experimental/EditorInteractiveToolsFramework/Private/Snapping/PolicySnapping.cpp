// Copyright Epic Games, Inc. All Rights Reserved.

#include "Snapping/PolicySnapping.h"

#include "Editor.h"
#include "ISnappingPolicy.h"
#include "SceneQueries/SceneSnappingManager.h"

namespace UE::Editor::Gizmos
{
	FGizmoPolicySnapper::FGizmoPolicySnapper(const TSharedPtr<ISnappingPolicy>& InSnappingPolicy)
		: SnappingPolicy(InSnappingPolicy)
	{
		TargetName = "Policy";
	}

	ESceneSnapQueryTargetResult FGizmoPolicySnapper::SnapPosition(const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList) const
	{
		OutSnappedPosition = InPosition;

		const FVector GridSize = FVector(GEditor->GetGridSize());
		SnappingPolicy->SnapPointToGrid(OutSnappedPosition, GridSize);

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoPolicySnapper::SnapPositionAxis(const double InAxisDelta, double& OutSnappedAxisDelta, const EAxisList::Type InAxis) const
	{
		if (InAxis == EAxisList::Screen)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		const EAxis::Type Axis = EAxis::FromAxisList(InAxis); 

		FVector SnappedPosition;
		SnappedPosition.SetComponentForAxis(Axis, InAxisDelta);

		const FVector GridSize = FVector(GEditor->GetGridSize());
		SnappingPolicy->SnapPointToGrid(SnappedPosition, GridSize);

		OutSnappedAxisDelta = SnappedPosition.GetComponentForAxis(Axis);

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoPolicySnapper::SnapRotation(const FQuat& InRotation, FQuat& OutRotation, const EAxisList::Type InAxisList) const
	{
		FRotator Rotation = InRotation.Rotator();
		SnappingPolicy->SnapRotatorToGrid(Rotation);

		OutRotation = Rotation.Quaternion();

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoPolicySnapper::SnapRotationAxisAngle(const double& InAngleDelta, double& OutSnappedAngleDelta, const EAxisList::Type InAxis) const
	{
		const EAxis::Type Axis = EAxis::FromAxisList(InAxis); 
		
		FRotator Rotation;
		Rotation.SetComponentForAxis(Axis, InAngleDelta);

		SnappingPolicy->SnapRotatorToGrid(Rotation);

		OutSnappedAngleDelta = Rotation.GetComponentForAxis(Axis);

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoPolicySnapper::SnapScale(const FVector& InScale, FVector& OutSnappedScale, const EAxisList::Type InAxisList) const
	{
		OutSnappedScale = InScale;

		const FVector GridSize = FVector(GEditor->GetScaleGridSize());
		SnappingPolicy->SnapScale(OutSnappedScale, GridSize);

		return ESceneSnapQueryTargetResult::Snapped;
	}

	ESceneSnapQueryTargetResult FGizmoPolicySnapper::SnapScaleAxis(const double InAxisDelta, double& OutSnappedAxisDelta, const EAxisList::Type InAxis) const
	{
		if (InAxis == EAxisList::Screen)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		const EAxis::Type Axis = EAxis::FromAxisList(InAxis); 
		
		FVector SnappedScale;
		SnappedScale.SetComponentForAxis(Axis, InAxisDelta);

		const FVector GridSize = FVector(GEditor->GetScaleGridSize());
		SnappingPolicy->SnapScale(SnappedScale, GridSize);

		OutSnappedAxisDelta = SnappedScale.GetComponentForAxis(Axis);

		return ESceneSnapQueryTargetResult::Snapped;
	}

	bool FGizmoPolicySnapper::IsQueryTypeSupported(const ESceneSnapQueryType InQueryType) const
	{
		return true;
	}
}
