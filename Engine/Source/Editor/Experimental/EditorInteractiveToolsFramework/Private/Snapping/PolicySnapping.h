// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "SceneQueries/SceneSnappingManager.h"

class UInteractiveToolsContext;
struct FSceneSnapQueryResult;
struct FSceneSnapQueryRequest;
struct FSceneHitQueryResult;
struct FSceneHitQueryRequest;
class USceneSnappingManager;
class UInteractiveGizmoManager;
class IToolsContextQueriesAPI;

class UEditorSceneSnappingManager;

namespace UE::Editor::Gizmos
{
	/** Adapter for a given ISnappingPolicy. */
	struct FGizmoPolicySnapper final : public FSceneSnapQueryTargetHandler
	{
	public:
		explicit FGizmoPolicySnapper(const TSharedPtr<ISnappingPolicy>& InSnappingPolicy);
		virtual ~FGizmoPolicySnapper() override = default;

		//~ Begin FSceneSnapPolicyTarget
		virtual ESceneSnapQueryTargetResult SnapPosition(const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList) const override;
		virtual ESceneSnapQueryTargetResult SnapPositionAxis(const double InAxisDelta, double& OutSnappedAxisDelta, const EAxisList::Type InAxis) const override;
		virtual ESceneSnapQueryTargetResult SnapRotation(const FQuat& InRotation, FQuat& OutRotation, const EAxisList::Type InAxisList) const override;
		virtual ESceneSnapQueryTargetResult SnapRotationAxisAngle(const double& InAngleDelta, double& OutSnappedAngleDelta, const EAxisList::Type InAxis) const override;
		virtual ESceneSnapQueryTargetResult SnapScale(const FVector& InScale, FVector& OutSnappedScale, const EAxisList::Type InAxisList) const override;
		virtual ESceneSnapQueryTargetResult SnapScaleAxis(const double InAxisDelta, double& OutSnappedAxisDelta, const EAxisList::Type InAxis) const override;
		virtual bool IsQueryTypeSupported(const ESceneSnapQueryType InQueryType) const override;
		//~ End FSceneSnapPolicyTarget

	private:
		TSharedPtr<ISnappingPolicy> SnappingPolicy;
	};
}
