// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformSnapping.h"

#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "Components/PrimitiveComponent.h"
#include "ContextObjectStore.h"
#include "Editor/Experimental/EditorInteractiveToolsFramework/Internal/EditorGizmos/EditorGizmoMath.h"
#include "EditorSnappingUtil.h"
#include "GameFramework/Actor.h"
#include "InteractiveToolsContext.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "ToolContextInterfaces.h"

namespace UE::Editor::Gizmos
{
	FGizmoTransformSnapper::FGizmoTransformSnapper(const UInteractiveToolsContext* InToolsContext)
	{
		TargetName = "ObjectTransform";

		if (ToolManager = InToolsContext->ToolManager;
			ToolManager.IsValid())
		{
			QueriesAPI = ToolManager->GetContextQueriesAPI();
		}
	}

	ESceneSnapQueryTargetResult FGizmoTransformSnapper::SnapPosition(const FGizmoTransformSnapperRequest& InTargetRequest, const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList) const
	{
		FVector WorldDelta = InPosition;
		if (InTargetRequest.CoordinateSpace != EToolContextCoordinateSystem::World)
		{
			WorldDelta = InTargetRequest.CoordinateTransform.GetRotation() * WorldDelta;
		}
		OutSnappedPosition = WorldDelta;

		if (!QueriesAPI || !ToolManager.IsValid())
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		UGizmoViewContext* ViewContext = ToolManager->GetContextObjectStore()->FindContext<UGizmoViewContext>();
		if (!ViewContext)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		const bool bSnapX = InAxisList & EAxisList::X || InAxisList & EAxisList::Screen;
		const bool bSnapY = InAxisList & EAxisList::Y || InAxisList & EAxisList::Screen;
		const bool bSnapZ = InAxisList & EAxisList::Z || InAxisList & EAxisList::Screen;

		if (!bSnapX && !bSnapY && !bSnapZ)
		{
			return ESceneSnapQueryTargetResult::NotSnapped;
		}

		FToolBuilderState ToolBuilderState;
		QueriesAPI->GetCurrentSelectionState(ToolBuilderState);

		TArray<AActor*> SelectedActors = ToolBuilderState.SelectedActors;

		TArray<const UPrimitiveComponent*> ComponentsToIgnore;
		UE::EditorSnappingUtil::GetActorComponents(SelectedActors, ComponentsToIgnore);

		FSceneQueryVisibilityFilter HitFilter(&ComponentsToIgnore);

		// Whether to use the gizmo position for uniform axis only (and cursor location for axis, planar).
		constexpr bool bUseDirectionToGizmoForUniformOnly = true;
		const bool bUseDirectionToGizmo = bUseDirectionToGizmoForUniformOnly
			? (InAxisList == EAxisList::All || InAxisList == EAxisList::Screen)
			: true;

		// Get absolute position (base + delta)
		const FVector QueryPosition = InTargetRequest.Transform.GetLocation() + WorldDelta;

		// Ray according to the view direction and gizmo position, or cursor position if not uniform axis translation.
		FVector RayDirection = bUseDirectionToGizmo ? QueryPosition - InTargetRequest.WorldRay.Origin : InTargetRequest.WorldRay.Direction;

		// Always fallback to input direction if coincident.
		if (RayDirection.IsNearlyZero())
		{
			RayDirection = InTargetRequest.WorldRay.Direction;
		}
		else
		{
			RayDirection = RayDirection.GetSafeNormal();
		}

		FRay Ray = FRay(InTargetRequest.WorldRay.Origin, RayDirection, true);

		FSceneHitQueryRequest HitRequest(Ray, ESceneHitQueryTargetType::PrimitiveTransform, {}, false, HitFilter);

		TArray<FSceneHitQueryResult> HitResults;
		if (USceneSnappingManager* SnappingManager = ToolManager->GetContextObjectStore()->FindContext<USceneSnappingManager>())
		{
			if (SnappingManager->ExecuteSceneHitQuery(HitRequest, HitResults);
				!HitResults.IsEmpty())
			{
				// Get best hit result by screen space proximity to gizmo
				const FVector2D ScreenPosition = ViewContext->WorldToPixel(bUseDirectionToGizmo ? QueryPosition : Ray.PointAt(UE_LARGE_HALF_WORLD_MAX1));
				Algo::StableSort(HitResults, [ScreenPosition, ViewContext](const FSceneHitQueryResult& InHitResultA, const FSceneHitQueryResult& InHitResultB)
				{
					const AActor* ActorA = InHitResultA.TargetActor;
					const AActor* ActorB = InHitResultB.TargetActor;

					if (ActorA == ActorB)
					{
						return false;
					}

					if (ActorA == nullptr)
					{
						return false;
					}
					
					if (ActorB == nullptr)
					{
						return true;
					}

					const FVector ActorLocationA = ActorA->GetActorLocation();
					const FVector ActorLocationB = ActorB->GetActorLocation();

					const FVector2D ActorScreenLocationA = ViewContext->WorldToPixel(ActorLocationA);
					const FVector2D ActorScreenLocationB = ViewContext->WorldToPixel(ActorLocationB);

					return FVector2D::Distance(ScreenPosition, ActorScreenLocationA) < FVector2D::Distance(ScreenPosition, ActorScreenLocationB);
				});

				const FSceneHitQueryResult* FirstHitResult = &HitResults[0];
				if (FirstHitResult->TargetActor)
				{
					const FVector2D::FReal DistanceToSnappedTransform = FVector2D::Distance(ScreenPosition, ViewContext->WorldToPixel(FirstHitResult->TargetActor->GetActorLocation()));
					if (DistanceToSnappedTransform <= InTargetRequest.SnapDistance)
					{
						FTransform TargetTransform = FirstHitResult->TargetActor->GetActorTransform();
						FVector TargetLocation = TargetTransform.GetLocation();

						FVector Delta = TargetLocation - InTargetRequest.Transform.GetLocation();

						if (InTargetRequest.CoordinateSpace != EToolContextCoordinateSystem::World)
						{
							Delta = InTargetRequest.CoordinateTransform.InverseTransformVectorNoScale(Delta);
						}

						if (InAxisList != EAxisList::All && InAxisList != EAxisList::Screen)
						{
							const FVector AxisDeltaMultiplier = GizmoMath::GetAxisMultiplier(InAxisList);
							Delta *= AxisDeltaMultiplier;
						}

						if (InTargetRequest.CoordinateSpace != EToolContextCoordinateSystem::World)
						{
							Delta = InTargetRequest.CoordinateTransform.TransformVectorNoScale(Delta);
						}

						// Convert absolute back to delta
						OutSnappedPosition = Delta;

						return ESceneSnapQueryTargetResult::Snapped;
					}
				}
			}
		}

		return ESceneSnapQueryTargetResult::NotSnapped;
	}

	ESceneSnapQueryTargetResult FGizmoTransformSnapper::SnapPosition(
		const FSceneSnapQueryRequest& InRequest,
		TArray<FSceneSnapQueryResult>& OutResults) const
	{
		if (!QueriesAPI)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		if (const FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();
			SnappingConfig.IsObjectTransformSnappingActive())
		{
			FGizmoTransformSnapperRequest SnapRequest;
			SnapRequest.WorldRay = InRequest.WorldRay;
			SnapRequest.SnapDistance = SnappingConfig.ObjectTransform.SnapDistance;
			SnapRequest.Transform = InRequest.Transform;
			SnapRequest.CoordinateSpace = InRequest.RequestCoordinateSpace;
			SnapRequest.CoordinateTransform = InRequest.CoordinateTransform;

			FSceneSnapQueryResult SnapResult;
	        SnapResult.TargetType = GetTargetTypes();

			FVector SnappedPosition;
			ESceneSnapQueryTargetResult TargetResult = SnapPosition(SnapRequest, InRequest.Position, SnappedPosition, InRequest.AxisList);
			if (TargetResult == ESceneSnapQueryTargetResult::Snapped)
			{
				SnapResult.CoordinateSpace = EToolContextCoordinateSystem::World;
				SnapResult.Position = SnappedPosition;
				OutResults.Emplace(SnapResult);
			}

			return TargetResult;
		}

		return ESceneSnapQueryTargetResult::Disabled;
	}

	ESceneSnapQueryTargetResult FGizmoTransformSnapper::SnapPosition(const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList) const
	{
		OutSnappedPosition = InPosition;

		if (!QueriesAPI)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		if (const FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();
			SnappingConfig.IsObjectTransformSnappingActive())
		{
			FSceneSnapQueryResult SnapResult;
			SnapResult.TargetType = ESceneSnapQueryTargetType::ObjectTransform;

			FGizmoTransformSnapperRequest SnapRequest;
			SnapRequest.SnapDistance = SnappingConfig.ObjectTransform.SnapDistance;

			return SnapPosition(SnapRequest, InPosition, OutSnappedPosition, InAxisList);
		}

		return ESceneSnapQueryTargetResult::Disabled;
	}

	ESceneSnapQueryTargetResult FGizmoTransformSnapper::SnapPositionAxis(const FGizmoTransformSnapperRequest& InTargetRequest, const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const
	{
		OutSnappedAxisValue = InAxisValue;

		if (InAxis == EAxisList::Screen)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		const EAxis::Type Axis = EAxis::FromAxisList(InAxis);

		FVector Position = FVector::ZeroVector;
		Position.SetComponentForAxis(Axis, InAxisValue);

		FVector SnappedPosition;
		const ESceneSnapQueryTargetResult TargetResult = SnapPosition(InTargetRequest, Position, SnappedPosition, InAxis);

		if (TargetResult == ESceneSnapQueryTargetResult::Snapped)
		{
			OutSnappedAxisValue = SnappedPosition.GetComponentForAxis(Axis);
		}

		return TargetResult;
	}

	ESceneSnapQueryTargetResult FGizmoTransformSnapper::SnapPositionAxis(const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const
	{
		OutSnappedAxisValue = InAxisValue;

		if (!QueriesAPI)
		{
			return ESceneSnapQueryTargetResult::Unsupported;
		}

		if (const FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();
			SnappingConfig.IsObjectTransformSnappingActive())
		{
			FSceneSnapQueryResult SnapResult;
			SnapResult.TargetType = ESceneSnapQueryTargetType::ObjectTransform;

			FViewCameraState ViewState;
			QueriesAPI->GetCurrentViewState(ViewState);

			FGizmoTransformSnapperRequest SnapRequest;
			SnapRequest.WorldRay = FRay(ViewState.Position, ViewState.Forward(), true);
			SnapRequest.SnapDistance = SnappingConfig.ObjectTransform.SnapDistance;

			return SnapPositionAxis(SnapRequest, InAxisValue, OutSnappedAxisValue, InAxis);
		}

		return ESceneSnapQueryTargetResult::Disabled;
	}

	bool FGizmoTransformSnapper::IsQueryTypeSupported(const ESceneSnapQueryType InQueryType) const
	{
		return InQueryType == ESceneSnapQueryType::Position;
	}
}
