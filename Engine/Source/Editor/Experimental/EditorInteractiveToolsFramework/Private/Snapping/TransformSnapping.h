// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "SceneQueries/SceneSnappingManager.h"

class IToolsContextQueriesAPI;
class UGizmoViewContext;
class UInteractiveGizmoManager;
class UInteractiveToolsContext;
class USceneSnappingManager;
struct FSceneHitQueryRequest;
struct FSceneHitQueryResult;
struct FSceneSnapQueryRequest;
struct FSceneSnapQueryResult;

class UEditorSceneSnappingManager;

namespace UE::Editor::Gizmos
{
	struct FGizmoTransformSnapperRequest
	{
		EToolContextCoordinateSystem CoordinateSpace = EToolContextCoordinateSystem::World;
		FTransform Transform;
		FTransform CoordinateTransform;
		float SnapDistance = 0.0f;
		FRay WorldRay;
	};

	class FGizmoTransformSnapper final : public TSceneSnapQueryTargetHandler<FGizmoTransformSnapperRequest>
	{
	public:
		explicit FGizmoTransformSnapper(const UInteractiveToolsContext* InToolsContext);
		virtual ~FGizmoTransformSnapper() override = default;

		//~ Begin FSceneSnapPolicyTargetHandler
		virtual ESceneSnapQueryTargetResult SnapPosition(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const override;
		virtual ESceneSnapQueryTargetResult SnapPosition(const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList = EAxisList::All) const override;
		virtual ESceneSnapQueryTargetResult SnapPositionAxis(const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const override;
		virtual bool IsQueryTypeSupported(const ESceneSnapQueryType InQueryType) const override;
		//~ End FSceneSnapPolicyTargetHandler

		//~ Begin TSceneSnapQueryTargetHandler
		virtual ESceneSnapQueryTargetResult SnapPosition(const FGizmoTransformSnapperRequest& InTargetRequest, const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList = EAxisList::All) const override;
		virtual ESceneSnapQueryTargetResult SnapPositionAxis(const FGizmoTransformSnapperRequest& InTargetRequest, const double InAxisValue, double& OutSnappedPositionAxis, const EAxisList::Type InAxis) const override;
		//~ End TSceneSnapQueryTargetHandler

		virtual ESceneSnapQueryTargetType GetTargetTypes() const override { return ESceneSnapQueryTargetType::ObjectTransform; }

	private:
		TWeakObjectPtr<UInteractiveToolManager> ToolManager = nullptr;
		const IToolsContextQueriesAPI* QueriesAPI = nullptr;
	};
}
