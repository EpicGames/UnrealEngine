// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolContexts/ToolStackContext.h"

#include "Tools/EdModeInteractiveToolsContext.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "InteractiveToolManager.h"
#include "InteractiveToolStack.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Tuple.h"
#include "Toolkits/BaseToolkit.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolStackContext)

class UInteractiveToolBuilder;

UToolStackContext::~UToolStackContext()
{
	if (UEdMode* EdmodePtr = EdMode.Get())
	{
		if (TSharedPtr<FModeToolkit> Toolkit = EdmodePtr->GetToolkit().Pin())
		{
			const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
			for (auto& RegisteredTool : RegisteredEditorTools)
			{
				CommandList->UnmapAction(RegisteredTool.Key);
				EdmodePtr->GetInteractiveToolsContext(ToolContextScope)->ToolManager->UnregisterToolType(RegisteredTool.Value);
			}
		}
	}
}

void UToolStackContext::RegisterToolStack(TSharedPtr<FUICommandInfo> UICommand, const FString& ToolStackIdentifier, EToolsContextScope ToolScope)
{
	ensureMsgf(UICommand && !UICommand->GetDefaultChord(EMultipleKeyBindingIndex::Primary).IsValidChord(),
		TEXT("Tool stack command for %s should be invalid, a stack cannot have a meaningful input chord binding"),
		*ToolStackIdentifier);

	if (UEdMode* EdmodePtr = EdMode.Get())
	{
		if (!EdmodePtr->GetToolkit().IsValid())
		{
			return;
		}

		if (ToolScope == EToolsContextScope::Default)
		{
			ToolScope = EdmodePtr->GetDefaultToolScope();
		}

		UEditorInteractiveToolsContext* UseToolsContext = EdmodePtr->GetInteractiveToolsContext(ToolScope);
		if (ensure(UseToolsContext != nullptr) == false)
		{
			return;
		}

		if (ensure(!ToolStacks.Contains(ToolStackIdentifier)))
		{
			ToolStacks.Add(ToolStackIdentifier, {});
		}

		FInteractiveToolStack::RegisterToolStack(EdmodePtr, UseToolsContext, UICommand, ToolStackIdentifier);
		if (ToolScope == EToolsContextScope::Editor)
		{
			RegisteredEditorToolStacks.Emplace(UICommand, ToolStackIdentifier);
		}
	}
}

void UToolStackContext::AddToolToStack(TSharedPtr<FUICommandInfo> UICommand, const FString& ToolStackIdentifier, const FString& ToolIdentifier, UInteractiveToolBuilder* Builder, EToolsContextScope ToolScope)
{
	// Note: We don't enforce the stack context to match the tool context
	// this is since the tool manager is not aware contexts exist

	if (UEdMode* EdmodePtr = EdMode.Get())
	{
		if (!EdmodePtr->GetToolkit().IsValid())
		{
			return;
		}

		if (ToolScope == EToolsContextScope::Default)
		{
			ToolScope = EdmodePtr->GetDefaultToolScope();
		}

		UEditorInteractiveToolsContext* UseToolsContext = EdmodePtr->GetInteractiveToolsContext(ToolScope);
		if (ensure(UseToolsContext != nullptr) == false)
		{
			return;
		}

		FInteractiveToolStack::AddToolToStack(EdmodePtr, UseToolsContext, UICommand, ToolStackIdentifier, ToolIdentifier, Builder);
		if (ToolScope == EToolsContextScope::Editor)
		{
			RegisteredEditorTools.Emplace(UICommand, ToolStackIdentifier);
		}
	}
}

void UToolStackContext::UnregisterToolStack(const FString& Identifier)
{
	ToolStacks.Remove(Identifier);
}

FInteractiveToolStack* UToolStackContext::AccessToolStack(const FString& Identifier)
{
	return ToolStacks.Find(Identifier);
}
