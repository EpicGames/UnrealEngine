// Copyright Epic Games, Inc. All Rights Reserved.


#include "ToolContexts/WidgetToolsContext.h"

#include "BaseBehaviors/Widgets/WidgetBaseBehavior.h"
#include "Containers/Array.h"
#include "InteractiveToolManager.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetToolsContext)

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
struct FCaptureLostEvent;
struct FCharacterEvent;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;



bool UWidgetToolsContext::OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				return WidgetTool->OnKeyChar(MyGeometry, InCharacterEvent);
			}
		}
	}

	return false;
}

bool UWidgetToolsContext::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				return WidgetTool->OnKeyDown(MyGeometry, InKeyEvent);
			}
		}
	}

	return false;
}

bool UWidgetToolsContext::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				return WidgetTool->OnKeyUp(MyGeometry, InKeyEvent);
			}
		}
	}

	return false;
}

bool UWidgetToolsContext::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				return WidgetTool->OnMouseButtonDown(MyGeometry, MouseEvent);
			}
		}
	}

	return false;
}

bool UWidgetToolsContext::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				return WidgetTool->OnMouseButtonUp(MyGeometry, MouseEvent);
			}
		}
	}

	return false;
}

bool UWidgetToolsContext::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				return WidgetTool->OnMouseMove(MyGeometry, MouseEvent);
			}
		}
	}

	return false;
}

bool UWidgetToolsContext::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				return WidgetTool->OnMouseWheel(MyGeometry, MouseEvent);
			}
		}
	}

	return false;
}

bool UWidgetToolsContext::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				return WidgetTool->OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
			}
		}
	}

	return false;
}

bool UWidgetToolsContext::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				return WidgetTool->OnDragDetected(MyGeometry, MouseEvent);
			}
		}
	}

	return false;
}

void UWidgetToolsContext::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				WidgetTool->OnMouseCaptureLost(CaptureLostEvent);
			}
		}
	}
}

int32 UWidgetToolsContext::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		if (EdModeContext && EdModeContext->ToolManager)
		{
			if (IWidgetBaseBehavior* WidgetTool = Cast<IWidgetBaseBehavior>(EdModeContext->ToolManager->GetActiveTool(EToolSide::Mouse)))
			{
				return WidgetTool->OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
			}
		}
	}

	return LayerId;
}
