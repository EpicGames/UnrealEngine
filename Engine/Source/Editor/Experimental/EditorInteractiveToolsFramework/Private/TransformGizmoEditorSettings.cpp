// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformGizmoEditorSettings.h"
#include "Editor.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorModeManager.h"
#include "Settings/LevelEditorViewportSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformGizmoEditorSettings)

UTransformGizmoEditorSettings::UTransformGizmoEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (ULevelEditorViewportSettings* const ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		SyncFromLevelEditorSettings();

		ViewportSettings->OnSettingChanged().AddUObject(this, &UTransformGizmoEditorSettings::OnLegacySettingChanged);
	}
}

void UTransformGizmoEditorSettings::PostInitProperties()
{
	Super::PostInitProperties();
	
	// Freeze Camera CVar
	{
		constexpr const TCHAR* CVarName = TEXT("Gizmos.FreezeView");
		if (IConsoleVariable* FoundCVar = IConsoleManager::Get().FindConsoleVariable(CVarName, false))
		{
			FoundCVar->Set(GizmosParameters.Debug.bFreezeCamera);
		}
	}

	// Debug Toggle CVar
	{
		constexpr const TCHAR* CVarName = TEXT("Gizmos.DebugDraw");
		if (IConsoleVariable* FoundCVar = IConsoleManager::Get().FindConsoleVariable(CVarName, false))
		{
			FoundCVar->Set(GizmosParameters.Debug.bDrawDebug);
			FoundCVar->OnChangedDelegate().AddWeakLambda(this, [WeakThis = MakeWeakObjectPtr(this)](IConsoleVariable* InCVar)
			{
				if (WeakThis.IsValid())
				{
					WeakThis->GizmosParameters.Debug.bDrawDebug = InCVar->GetBool();
					WeakThis->SaveConfig();
				}
			});
		}
	}
}

#if WITH_EDITOR
void UTransformGizmoEditorSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.PropertyChain.GetHead())
	{
		if (const FProperty* const Property = InPropertyChangedEvent.PropertyChain.GetHead()->GetValue())
		{
			const FName ChangedPropertyName = Property->GetFName();
			static const FName GizmosParametersName = GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, GizmosParameters);

			if (GizmosParametersName == ChangedPropertyName)
			{
				// Sync FreezeView CVar when debug settings change
				if (const FEditPropertyChain& Chain = InPropertyChangedEvent.PropertyChain;
					Chain.Num() >= 2)
				{
					static const FName FreezeCameraName = GET_MEMBER_NAME_CHECKED(FGizmoDebugSettings, bFreezeCamera);
					if (const FProperty* LeafProperty = Chain.GetTail()->GetValue();
						LeafProperty && LeafProperty->GetFName() == FreezeCameraName)
					{
						constexpr const TCHAR* CVarName = TEXT("Gizmos.FreezeView");
						if (IConsoleVariable* FoundCVar = IConsoleManager::Get().FindConsoleVariable(CVarName, false))
						{
							FoundCVar->Set(GizmosParameters.Debug.bFreezeCamera);
						}
					}
				}

				BroadcastGizmosParametersChange();
			}
		}
	}
}

void UTransformGizmoEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	static const FName UseExperimentalGizmoName = GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, bUseExperimentalGizmo);
	static const FName UseEditorTRSGizmoName = GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, bUseEditorTRSGizmo);
	static const FName EnableArcballRotateName = GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, bEnableArcballRotate);
	static const FName EnableScreenRotateName = GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, bEnableScreenRotate);
	static const FName EnableAxisDrawingName = GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, bEnableAxisDrawing);
	static const FName EnableCombinedTranslateRotateName = GET_MEMBER_NAME_CHECKED(UTransformGizmoEditorSettings, bEnableCombinedTranslateRotate);

	if (!InPropertyChangedEvent.Property)
	{
		BroadcastGizmosParametersChange();
		return;
	}

	const FName ChangedPropertyName = InPropertyChangedEvent.Property->GetFName();

	if (ChangedPropertyName == UseExperimentalGizmoName)
	{
		BroadcastNewTRSGizmoChange();
	}
	else if (ChangedPropertyName == UseEditorTRSGizmoName)
	{
		// Force gizmo rebuild by cycling experimental gizmos off/on.
		// This tears down existing gizmos and recreates them with the new builder path.
		const bool bWasUsingExperimental = bUseExperimentalGizmo;
		SetUseExperimentalGizmo(false);
		SetUseExperimentalGizmo(bWasUsingExperimental);
	}
	else if (ChangedPropertyName == EnableArcballRotateName)
	{
		if (ULevelEditorViewportSettings* const ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
		{
			ViewportSettings->bAllowArcballRotate = bEnableArcballRotate;
		}
	}
	else if (ChangedPropertyName == EnableScreenRotateName)
	{
		if (ULevelEditorViewportSettings* const ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
		{
			ViewportSettings->bAllowScreenRotate = bEnableScreenRotate;
		}
	}
	else if (ChangedPropertyName == EnableAxisDrawingName)
	{
		if (ULevelEditorViewportSettings* const ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
		{
			ViewportSettings->bAllowEditWidgetAxisDisplay = bEnableAxisDrawing;
		}
	}
	else if (ChangedPropertyName == EnableCombinedTranslateRotateName)
	{
		if (ULevelEditorViewportSettings* const ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
		{
			ViewportSettings->bAllowTranslateRotateZWidget = bEnableCombinedTranslateRotate;
			
			// Mirror behavior in ULevelEditorViewportSettings::PostEditChangeProperty() for responding to this property
			if (bEnableCombinedTranslateRotate)
			{
				// The combined gizmo only works with the legacy gizmo stack
				UEditorInteractiveGizmoManager::SetTransformGizmoVersion(0);
				GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_TranslateRotateZ);
			}
			else if (GLevelEditorModeTools().GetWidgetMode() == UE::Widget::WM_TranslateRotateZ)
			{
				GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_Translate);
			}
			ViewportSettings->SaveConfig();
		}
	}
}


void UTransformGizmoEditorSettings::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);

	SyncFromLevelEditorSettings();

	BroadcastNewTRSGizmoChange();
	BroadcastGizmosParametersChange();

	if (GEditor)
	{
		GEditor->RedrawAllViewports();
	}
}
#endif // WITH_EDITOR

void UTransformGizmoEditorSettings::SetUseExperimentalGizmo(bool bInUseExperimentalGizmo)
{
	if (bUseExperimentalGizmo != bInUseExperimentalGizmo)
	{
		bUseExperimentalGizmo = bInUseExperimentalGizmo;
		SaveConfig();

		BroadcastNewTRSGizmoChange();

		if (GEditor)
		{
			GEditor->RedrawAllViewports();
		}
	}
}

void UTransformGizmoEditorSettings::SetUseNewTRSGizmo(bool bInUseNewTRSGizmo)
{
	if (bInUseNewTRSGizmo != bUseEditorTRSGizmo)
	{
		bUseEditorTRSGizmo = bInUseNewTRSGizmo;
		SaveConfig();

		BroadcastNewTRSGizmoChange();

		if (GEditor)
		{
			GEditor->RedrawAllViewports();
		}
	}
}

bool UTransformGizmoEditorSettings::UsesLegacyGizmo() const
{
	return !(UsesExperimentalGizmo() || UsesNewTRSGizmo());
}

bool UTransformGizmoEditorSettings::UsesExperimentalGizmo() const
{
	return bUseExperimentalGizmo;
}

bool UTransformGizmoEditorSettings::UsesNewTRSGizmo() const
{
	return bUseEditorTRSGizmo;
}

void UTransformGizmoEditorSettings::SetGizmosParameters(const FGizmosParameters& InGizmosParameters)
{
	GizmosParameters = InGizmosParameters;
	SaveConfig();

	BroadcastGizmosParametersChange();
}

void UTransformGizmoEditorSettings::SetTransformGizmoSize(float InTransformGizmoSize)
{
	if (TransformGizmoSize != InTransformGizmoSize)
	{
		TransformGizmoSize = InTransformGizmoSize;

		// Fire off BroadcastGizmosParametersChange 
		BroadcastGizmosParametersChange();

		SaveConfig();

		if (GEditor)
		{
			GEditor->RedrawAllViewports();
		}
	}
}

void UTransformGizmoEditorSettings::SetTransformGizmoAxisMultiplier(float InAxisMultiplier, bool bInteractive)
{
	if (GizmosParameters.Style.AxisSizeMultiplier != InAxisMultiplier)
	{
		GizmosParameters.Style.AxisSizeMultiplier = InAxisMultiplier;
		BroadcastGizmosParametersChange();
		if (GEditor)
		{
			GEditor->RedrawAllViewports();
		}
	}

	// Always save when non-interactive
	if (!bInteractive)
	{
		SaveConfig();
	}
}

float UTransformGizmoEditorSettings::GetTransformGizmoAxisMultiplier() const
{
	return GizmosParameters.Style.AxisSizeMultiplier;
}

void UTransformGizmoEditorSettings::BroadcastNewTRSGizmoChange() const
{
	UEditorInteractiveGizmoManager::OnUsesNewTRSGizmosChangedDelegate().Broadcast(!UsesLegacyGizmo());
}

void UTransformGizmoEditorSettings::BroadcastGizmosParametersChange() const
{
	UEditorInteractiveGizmoManager::OnGizmosParametersChangedDelegate().Broadcast(GizmosParameters);
}

void UTransformGizmoEditorSettings::SyncFromLevelEditorSettings()
{
	if (ULevelEditorViewportSettings* const ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		bEnableArcballRotate = ViewportSettings->bAllowArcballRotate;
		bEnableScreenRotate = ViewportSettings->bAllowScreenRotate;
		bEnableAxisDrawing = ViewportSettings->bAllowEditWidgetAxisDisplay;
		bEnableCombinedTranslateRotate = ViewportSettings->bAllowTranslateRotateZWidget;
	}
}

void UTransformGizmoEditorSettings::OnLegacySettingChanged(FName InPropertyName)
{
	// Retrieving names from ULevelEditorViewportSettings
	static const FName AllowArcballRotateName = GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bAllowArcballRotate);
	static const FName AllowScreenRotateName = GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bAllowScreenRotate);
	static const FName AllowEditWidgetAxisDisplayName = GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bAllowEditWidgetAxisDisplay);
	static const FName AllowTranslateRotateZWidgetName = GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bAllowTranslateRotateZWidget);

	if (InPropertyName == AllowArcballRotateName)
	{
		if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
		{
			bEnableArcballRotate = ViewportSettings->bAllowArcballRotate;
		}
	}
	else if (InPropertyName == AllowScreenRotateName)
	{
		if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
		{
			bEnableScreenRotate = ViewportSettings->bAllowScreenRotate;
		}
	}
	else if (InPropertyName == AllowEditWidgetAxisDisplayName)
	{
		if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
		{
			bEnableAxisDrawing = ViewportSettings->bAllowEditWidgetAxisDisplay;
		}
	}
	else if (InPropertyName == AllowTranslateRotateZWidgetName)
	{
		if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
		{
			bEnableCombinedTranslateRotate = ViewportSettings->bAllowTranslateRotateZWidget;
			
			// The combined translate/rotate widget is a legacy-gizmo feature. Enabling it should disable the new gizmo stack.
			if (bEnableCombinedTranslateRotate)
			{
				UEditorInteractiveGizmoManager::SetTransformGizmoVersion(0);
			}
		}
	}
}
