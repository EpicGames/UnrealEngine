// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportCameraRotateInteraction.h"
#include "BaseBehaviors/KeyInputBehavior.h"
#include "CameraController.h"
#include "EditorViewportClient.h"
#include "ViewportClientNavigationHelper.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportCameraRotateInteraction::UViewportCameraRotateInteraction()
{
	InteractionName = UE::Editor::ViewportInteractions::CameraRotate;
	Groups = { UE::Editor::ViewportInteractions::CameraFly };

	UKeyInputBehavior* KeyInputBehavior = NewObject<UKeyInputBehavior>();
	KeyInputBehavior->Initialize(this, GetKeys());
	KeyInputBehavior->SetDefaultPriority(UE::Editor::ViewportInteractions::VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY);
	KeyInputBehavior->bRequireAllKeys = false;

	KeyInputBehavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState)
	{
		if (FInputDeviceState::IsAltKeyDown(InputDeviceState) || FInputDeviceState::IsCtrlKeyDown(InputDeviceState)
			|| FInputDeviceState::IsShiftKeyDown(InputDeviceState) || FInputDeviceState::IsCmdKeyDown(InputDeviceState))
		{
			return false;
		}

		bool bResult = true;

		if (InputDeviceState.InputDevice == EInputDevices::Keyboard)
		{
			using namespace UE::Editor::ViewportInteractions;

			const FKey& ActiveKey = InputDeviceState.Keyboard.ActiveKey.Button;

			// These commands should only be capture if a mouse button is down (not mapped by default)
			if (CommandMatchesKey(FViewportNavigationCommands::Get().RotateUp, ActiveKey)
				|| CommandMatchesKey(FViewportNavigationCommands::Get().RotateDown, ActiveKey)
				|| CommandMatchesKey(FViewportNavigationCommands::Get().RotateLeft, ActiveKey)
				|| CommandMatchesKey(FViewportNavigationCommands::Get().RotateRight, ActiveKey))
			{
				bool bIsGizmoDragging = false;
				if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
				{
					bIsGizmoDragging = BehaviorSource->IsGizmoDragging();
				}

				bResult = IsAnyMouseButtonDown() && !bIsGizmoDragging;
				// If a mouse button is down, but mouse hasn't been dragged yet, we need to tell the behavior source that
				// User is performing mouse look or at least camera movement. This also prevents regular clicks from performing their action on button release
				// e.g. we don't want a RMB release to summon a context menu after mouse looking
				if (bResult)
				{
					if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
					{
						BehaviorSource->SetIsMouseLooking(true);
					}
				}
			}
		}

		return bResult;
	};

	KeyInputBehaviorWeak = KeyInputBehavior;

	RegisterInputBehavior(KeyInputBehavior);
}

void UViewportCameraRotateInteraction::OnKeyPressed(const FKey& InKeyID)
{
	if (IsEnabled())
	{
		constexpr bool bIsPressed = true;
		UpdateKeyState(InKeyID, bIsPressed);
	}
}

void UViewportCameraRotateInteraction::OnKeyReleased(const FKey& InKeyID)
{
	if (IsEnabled())
	{
		constexpr bool bIsPressed = false;
		UpdateKeyState(InKeyID, bIsPressed);
	}
}

void UViewportCameraRotateInteraction::Tick(float InDeltaTime) const
{
	UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource();
	if (!BehaviorSource)
	{
		return;
	}

	if (BehaviorSource->IsGizmoDragging())
	{
		return;
	}

	if (RotateYawImpulse != 0 || RotatePitchImpulse != 0)
	{
		BehaviorSource->SetCameraHasMoved(true);

		if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
		{
			if (FViewportClientNavigationHelper* NavigationHelper = EditorViewportClient->GetViewportNavigationHelper())
			{
				NavigationHelper->ImpulseDataDelta.RotateYawImpulse += RotateYawImpulse;
				NavigationHelper->ImpulseDataDelta.RotatePitchImpulse += RotatePitchImpulse;
			}
		}
	}
}

void UViewportCameraRotateInteraction::UpdateKeyState(const FKey& InKeyID, bool bInIsPressed)
{
	using namespace UE::Editor::ViewportInteractions;

	constexpr float ImpulseValue = 1.0f;

	const bool bMouseLookingOrRelease = IsMouseLooking() || !bInIsPressed;

	// Rotate Up/Down
	if (CommandMatchesKey(FViewportNavigationCommands::Get().RotateUp, InKeyID) && bMouseLookingOrRelease)
	{
		RotatePitchImpulse = bInIsPressed ? ImpulseValue : 0.0f;
	}
	else if (CommandMatchesKey(FViewportNavigationCommands::Get().RotateDown, InKeyID) && bMouseLookingOrRelease)
	{
		RotatePitchImpulse = bInIsPressed ? -ImpulseValue : 0.0f;
	}
	// Rotate Left/Right
	else if (CommandMatchesKey(FViewportNavigationCommands::Get().RotateLeft, InKeyID) && bMouseLookingOrRelease)
	{
		RotateYawImpulse = bInIsPressed ? -ImpulseValue : 0.0f;
	}
	else if (CommandMatchesKey(FViewportNavigationCommands::Get().RotateRight, InKeyID) && bMouseLookingOrRelease)
	{
		RotateYawImpulse = bInIsPressed ? ImpulseValue : 0.0f;
	}
}

TArray<FKey> UViewportCameraRotateInteraction::GetKeys() const
{
	TArray<FKey> Keys;
	for (int32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);

		TArray<FKey> KeysFromChords;
		for (const TSharedPtr<FUICommandInfo>& Command : GetCommands())
		{
			KeysFromChords.Add(Command->GetActiveChord(ChordIndex)->Key);
		}

		Keys.Append(KeysFromChords);
	}

	return Keys;
}

void UViewportCameraRotateInteraction::OnCommandChordChanged()
{
	if (TStrongObjectPtr<UKeyInputBehavior> KeyInputBehaviorPinned = KeyInputBehaviorWeak.Pin())
	{
		KeyInputBehaviorPinned->Initialize(this, GetKeys());
	}
}

TArray<TSharedPtr<FUICommandInfo>> UViewportCameraRotateInteraction::GetCommands() const
{
	if (FViewportNavigationCommands::IsRegistered())
	{
		return { FViewportNavigationCommands::Get().RotateUp,
				 FViewportNavigationCommands::Get().RotateDown,
				 FViewportNavigationCommands::Get().RotateLeft,
				 FViewportNavigationCommands::Get().RotateRight };
	}
	return {};
}
