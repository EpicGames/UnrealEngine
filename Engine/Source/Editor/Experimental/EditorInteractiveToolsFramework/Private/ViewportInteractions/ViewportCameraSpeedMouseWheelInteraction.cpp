// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportCameraSpeedMouseWheelInteraction.h"

#include "BaseBehaviors/MouseWheelModifierInputBehavior.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "EditorViewportClient.h"

UViewportCameraSpeedMouseWheelInteraction::UViewportCameraSpeedMouseWheelInteraction()
{
	InteractionName	= TEXT("Mouse Wheel Camera Speed");
	Groups = { UE::Editor::ViewportInteractions::CameraFly };

	// Set camera speed using mouse wheel
	UMouseWheelInputBehavior* MouseWheelInputBehavior = NewObject<UMouseWheelModifierInputBehavior>();
	MouseWheelInputBehavior->Initialize(this);
	MouseWheelInputBehavior->SetDefaultPriority(UE::Editor::ViewportInteractions::VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY);

	RegisterInputBehavior(MouseWheelInputBehavior);
}

FInputRayHit UViewportCameraSpeedMouseWheelInteraction::ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos)
{
	return IsAnyMouseButtonDown() ? FInputRayHit(TNumericLimits<float>::Max()) : FInputRayHit();
}

void UViewportCameraSpeedMouseWheelInteraction::OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos)
{
	if (!IsAnyMouseButtonDown())
	{
		return;
	}

	if (const FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		if (EditorViewportClient->IsOrtho())
		{
			return;
		}
	}

	UE_LOGF(LogITFViewportInteractions, Verbose, "INCREASE CAMERA SPEED");
	UpdateCameraSpeed(0.1f);
}

void UViewportCameraSpeedMouseWheelInteraction::OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos)
{
	if (!IsAnyMouseButtonDown())
	{
		return;
	}

	if (const FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		if (EditorViewportClient->IsOrtho())
		{
			return;
		}
	}

	UE_LOGF(LogITFViewportInteractions, Verbose, "DECREASE CAMERA SPEED");
	UpdateCameraSpeed(-0.1f);
}

void UViewportCameraSpeedMouseWheelInteraction::UpdateCameraSpeed(float InUpdateFactor) const
{
	if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
	{
		FEditorViewportCameraSpeedSettings SpeedSettings = EditorViewportClient->GetCameraSpeedSettings();
		const float Speed = SpeedSettings.GetCurrentSpeed();

		float Delta = Speed * InUpdateFactor;

		SpeedSettings.SetCurrentSpeed(Speed + Delta);
		EditorViewportClient->SetCameraSpeedSettings(SpeedSettings);
	}
}
