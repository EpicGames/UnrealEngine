// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportCameraTranslateInteraction.h"
#include "BaseBehaviors/KeyInputBehavior.h"
#include "CameraController.h"
#include "EditorViewportClient.h"
#include "ViewportClientNavigationHelper.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "Framework/Application/SlateApplication.h"

namespace UE::Editor::ViewportInteractions::Private
{
	bool KeyBehaviorHasCommand(const UKeyInputBehavior* KeyBehavior, const TSharedPtr<FUICommandInfo>& Command)
	{
		for (int32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
		{
			EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
			if (KeyBehavior->IsKeyPressed(Command->GetActiveChord(ChordIndex)->Key))
			{
				return true;
			}
		}

		return false;
	}
}

UViewportCameraTranslateInteraction::UViewportCameraTranslateInteraction()
{
	InteractionName = UE::Editor::ViewportInteractions::CameraTranslate;
	Groups = { UE::Editor::ViewportInteractions::CameraFly };

	// Camera translate
	UKeyInputBehavior* KeyInputBehavior = NewObject<UKeyInputBehavior>();
	KeyInputBehavior->Initialize(this, GetKeys());
	KeyInputBehavior->SetDefaultPriority(UE::Editor::ViewportInteractions::VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY);
	KeyInputBehavior->bRequireAllKeys = false;
	KeyInputBehavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState) 
	{
		if (FInputDeviceState::IsAltKeyDown(InputDeviceState) || FInputDeviceState::IsCtrlKeyDown(InputDeviceState)
			|| FInputDeviceState::IsShiftKeyDown(InputDeviceState) || FInputDeviceState::IsCmdKeyDown(InputDeviceState))
		{
			return false;
		}

		bool bResult = true;
		
		if (FEditorViewportClient* ViewportClient = GetEditorViewportClient())
		{
			if (ViewportClient->IsCameraLocked())
			{
				return false;
			}
		}
		
		if (InputDeviceState.InputDevice == EInputDevices::Keyboard)
		{
			using namespace UE::Editor::ViewportInteractions;

			const FKey& ActiveKey = InputDeviceState.Keyboard.ActiveKey.Button;

			// These commands should only be capture if a mouse button is down (defaults are W/A/S/D/R/F/Q/E)
			if (CommandMatchesKey(FViewportNavigationCommands::Get().Forward, ActiveKey) ||
				CommandMatchesKey(FViewportNavigationCommands::Get().Backward, ActiveKey) ||
				CommandMatchesKey(FViewportNavigationCommands::Get().Right, ActiveKey) ||
				CommandMatchesKey(FViewportNavigationCommands::Get().Left, ActiveKey) ||
				CommandMatchesKey(FViewportNavigationCommands::Get().WorldUp, ActiveKey) ||
				CommandMatchesKey(FViewportNavigationCommands::Get().WorldDown, ActiveKey) ||
				CommandMatchesKey(FViewportNavigationCommands::Get().LocalUp, ActiveKey) ||
				CommandMatchesKey(FViewportNavigationCommands::Get().LocalDown, ActiveKey))
			{
				bool bIsGizmoDragging = false;
				if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
				{
					bIsGizmoDragging = BehaviorSource->IsGizmoDragging();
				}

				bResult = IsAnyMouseButtonDown() && !bIsGizmoDragging;
				// If a mouse button is down, but mouse hasn't been dragged yet, we need to tell the behavior source that
				// User is performing mouse look or at least camera movement. This also prevents regular clicks from performing their action on button release
				// e.g. we don't want a RMB release to summon a context menu after mouse looking
				if (bResult)
				{
					if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
					{
						BehaviorSource->SetIsMouseLooking(!BehaviorSource->IsGizmoDragging());
					}
				}
			}
		}

		return bResult || FSlateApplication::Get().IsUsingTrackpad();
	};
	KeyInputBehaviorWeak = KeyInputBehavior;

	RegisterInputBehavior(KeyInputBehavior);
}

void UViewportCameraTranslateInteraction::OnForceEndCapture()
{
}

void UViewportCameraTranslateInteraction::Tick(float InDeltaTime) const
{
	UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource();
	if (!BehaviorSource)
	{
		return;
	}

	if (BehaviorSource->IsGizmoDragging())
	{
		return;
	}

	using namespace UE::Editor::ViewportInteractions::Private;

	constexpr float ImpulseValue = 1.0f;

	const bool bIsMouseLooking = IsMouseLooking();
	
	float ForwardBackwardImpulse = 0.0f;
	float RightLeftImpulse = 0.0f;
	float WorldUpDownImpulse = 0.0f;
	float LocalUpDownImpulse = 0.0f;
	
	// TODO: This could be better handled by either a bespoke behavior or a behavior that can track and
	// expose multiple sets of keys at once (perhaps in a similar fashion as IModifierToggleBehaviorTarget
	if (UKeyInputBehavior* KeyBehavior = KeyInputBehaviorWeak.Get())
	{
		// Forward/Backward
		if ((bIsMouseLooking && KeyBehaviorHasCommand(KeyBehavior, FViewportNavigationCommands::Get().Forward))
			|| KeyBehavior->IsKeyPressed(EKeys::NumPadEight) || KeyBehavior->IsKeyPressed(EKeys::Up))
		{
			ForwardBackwardImpulse += ImpulseValue;
		}
		if ((bIsMouseLooking && KeyBehaviorHasCommand(KeyBehavior, FViewportNavigationCommands::Get().Backward))
			|| KeyBehavior->IsKeyPressed(EKeys::NumPadTwo) || KeyBehavior->IsKeyPressed(EKeys::Down))
		{
			ForwardBackwardImpulse -= ImpulseValue;
		}
		// Right/Left
		if ((KeyBehaviorHasCommand(KeyBehavior, FViewportNavigationCommands::Get().Right))
			|| KeyBehavior->IsKeyPressed(EKeys::NumPadSix) || KeyBehavior->IsKeyPressed(EKeys::Right))
		{
			RightLeftImpulse += ImpulseValue;
		}
		if ((KeyBehaviorHasCommand(KeyBehavior, FViewportNavigationCommands::Get().Left))
			|| KeyBehavior->IsKeyPressed(EKeys::NumPadFour) || KeyBehavior->IsKeyPressed(EKeys::Left))
		{
			RightLeftImpulse -= ImpulseValue;
		}
		// World Up/Down
		if ((KeyBehaviorHasCommand(KeyBehavior, FViewportNavigationCommands::Get().WorldUp))
			|| KeyBehavior->IsKeyPressed(EKeys::NumPadNine))
		{
			WorldUpDownImpulse += ImpulseValue;
		}
		if ((KeyBehaviorHasCommand(KeyBehavior, FViewportNavigationCommands::Get().WorldDown))
			|| KeyBehavior->IsKeyPressed(EKeys::NumPadSeven))
		{
			WorldUpDownImpulse -= ImpulseValue;
		}
		// Local Up/Down
		if (KeyBehaviorHasCommand(KeyBehavior, FViewportNavigationCommands::Get().LocalUp))
		{
			LocalUpDownImpulse += ImpulseValue;
		}
		if (KeyBehaviorHasCommand(KeyBehavior, FViewportNavigationCommands::Get().LocalDown))
		{
			LocalUpDownImpulse -= ImpulseValue;
		}
	}

	if (ForwardBackwardImpulse != 0 || RightLeftImpulse != 0 || WorldUpDownImpulse != 0 || LocalUpDownImpulse != 0)
	{
		BehaviorSource->SetCameraHasMoved(true);

		if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
		{
			if (FViewportClientNavigationHelper* NavigationHelper = EditorViewportClient->GetViewportNavigationHelper())
			{
				FCameraControllerUserImpulseData& ImpulseDataDelta = NavigationHelper->ImpulseDataDelta;
				ImpulseDataDelta.MoveForwardBackwardImpulse += ForwardBackwardImpulse;
				ImpulseDataDelta.MoveRightLeftImpulse += RightLeftImpulse;
				ImpulseDataDelta.MoveWorldUpDownImpulse += WorldUpDownImpulse;
				ImpulseDataDelta.MoveLocalUpDownImpulse += LocalUpDownImpulse;
			}
		}
	}
}

TArray<FKey> UViewportCameraTranslateInteraction::GetKeys() const
{
	TArray<FKey> Keys = { EKeys::Add, EKeys::Subtract };

	// Key Mappings (Command Info based)
	// Iterate through all key mappings to generate key state flags
	for (int32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);

		// Retrieve command based keys
		for (const TSharedPtr<FUICommandInfo>& Command : GetCommands())
		{
			Keys.Add(Command->GetActiveChord(ChordIndex)->Key);
		}
	}

	// Hardcoded numpad keys
	if (bUseNumpadKey)
	{
		TArray<FKey> OtherKeys = { EKeys::NumPadTwo,   EKeys::NumPadFour,  EKeys::NumPadSix,
								   EKeys::NumPadSeven, EKeys::NumPadEight, EKeys::NumPadNine };

		Keys.Append(OtherKeys);
	}

	// Hardcoded
	Keys.Append({ EKeys::Up, EKeys::Down, EKeys::Right, EKeys::Left });

	return Keys;
}

void UViewportCameraTranslateInteraction::OnCommandChordChanged()
{
	if (TStrongObjectPtr<UKeyInputBehavior> KeyInputBehaviorPinned = KeyInputBehaviorWeak.Pin())
	{
		KeyInputBehaviorPinned->Initialize(this, GetKeys());
	}
}

TArray<TSharedPtr<FUICommandInfo>> UViewportCameraTranslateInteraction::GetCommands() const
{
	if (FViewportNavigationCommands::IsRegistered())
	{
		return {
			FViewportNavigationCommands::Get().Forward, FViewportNavigationCommands::Get().Backward,
			FViewportNavigationCommands::Get().Right,   FViewportNavigationCommands::Get().Left,
			FViewportNavigationCommands::Get().WorldUp, FViewportNavigationCommands::Get().WorldDown,
			FViewportNavigationCommands::Get().LocalUp, FViewportNavigationCommands::Get().LocalDown,
		};
	}

	return {};
}
