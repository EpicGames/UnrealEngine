// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportClickInteraction.h"

#include "BaseBehaviors/DoubleClickBehavior.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "EditorViewportClient.h"
#include "SceneView.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

void UViewportClickInteraction::Initialize(UViewportInteractionsBehaviorSource* const InViewportInteractionsBehaviorSource)
{
	Super::Initialize(InViewportInteractionsBehaviorSource);
	
	BuildBehaviors();
}

void UViewportClickInteraction::BuildBehaviors()
{
	const FName UniqueName = MakeUniqueObjectName(GetTransientPackageAsObject(), UViewportClickBehavior::StaticClass(), *FString::Printf(TEXT("%s_Behavior"), *GetClass()->GetName()));
	UViewportClickBehavior* ClickBehavior = NewObject<UViewportClickBehavior>(GetTransientPackageAsObject(), UniqueName);
	ClickBehavior->Initialize(this);
	
	ClickBehavior->SetAllowsCaptureStealing(true);
	RegisterInputBehavior(ClickBehavior);

	ClickBehaviorWeak = ClickBehavior;
}

FInputRayHit UViewportClickInteraction::IsHitByClick(const FInputDeviceRay& InClickPos)
{
	if (IsEnabled())
	{
		return FInputRayHit(0.0f);
	}
	return FInputRayHit();
}

void UViewportClickInteraction::OnClickDown(const FInputDeviceRay& InClickPos)
{
}

void UViewportClickInteraction::OnStateUpdated(const FInputDeviceState& InInputDeviceState)
{
	auto CacheChange = [this](const FDeviceButtonState& ButtonState) -> bool
	{
		if (ButtonState.bDoubleClicked)
		{
			LastChangedMouseButton = ButtonState.Button;
			LastInputEvent = IE_DoubleClick;
			return true;
		}
		if (ButtonState.bPressed)
		{
			LastChangedMouseButton = ButtonState.Button;
			LastInputEvent = IE_Pressed;
			return true;
		}
		if (ButtonState.bReleased)
		{
			LastChangedMouseButton = ButtonState.Button;
			LastInputEvent = IE_Released;
			return true;
		}
		return false;
	};

	CacheChange(InInputDeviceState.Mouse.Left) || CacheChange(InInputDeviceState.Mouse.Right) || CacheChange(InInputDeviceState.Mouse.Middle);
}

HHitProxy* UViewportClickInteraction::GetHitProxy(const FInputDeviceRay& InClickPos) const
{
	if (FEditorViewportClient* Client = GetEditorViewportClient())
	{
		if (Client->Viewport)
		{
			const int32 CurrentClickPosX = static_cast<int32>(InClickPos.ScreenPosition.X);
			const int32 CurrentClickPosY = static_cast<int32>(InClickPos.ScreenPosition.Y);
		
			return Client->Viewport->GetHitProxy(CurrentClickPosX, CurrentClickPosY);
		}		
	}
	return nullptr;
}

void UViewportClickInteraction::OnClickUp(const FInputDeviceRay& InClickPos)
{
	if (UViewportInteractionsBehaviorSource* const ViewportInteractionBehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		ViewportInteractionBehaviorSource->ClearMouseCursorOverride();
	}

	if (IsMouseLooking())
	{
		if (UViewportInteractionsBehaviorSource* const ViewportInteractionBehaviorSource = GetViewportInteractionsBehaviorSource())
		{
			// Store info about camera movement locally, since SetIsMouseLooking call will reset it
			const bool bCameraHasMoved = ViewportInteractionBehaviorSource->HasCameraMoved();
			ViewportInteractionBehaviorSource->SetIsMouseLooking(false);

			// If camera has moved, this was definitely not a click
			if (bCameraHasMoved)
			{
				return;
			}
		}
	}

	const int32 CurrentClickPosX = static_cast<int32>(InClickPos.ScreenPosition.X);
	const int32 CurrentClickPosY = static_cast<int32>(InClickPos.ScreenPosition.Y);

	if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
	{
		if (FViewport* Viewport = EditorViewportClient->Viewport)
		{
			TRefCountPtr<HHitProxy> HitProxy = Viewport->GetHitProxy(CurrentClickPosX, CurrentClickPosY);

			// Compute a view.
			FSceneViewFamilyContext ViewFamily(
				FSceneViewFamily::ConstructionValues(Viewport, EditorViewportClient->GetScene(), EditorViewportClient->EngineShowFlags)
					.SetRealtimeUpdate(EditorViewportClient->IsRealtime())
			);

			if (FSceneView* View = EditorViewportClient->CalcSceneView(&ViewFamily))
			{
				const FViewportClick ViewportClick(
					View, EditorViewportClient, LastChangedMouseButton, LastInputEvent, CurrentClickPosX, CurrentClickPosY
				);

				ProcessClick_Internal(ViewportClick, HitProxy, *View);
			}
		}
	}
	
	if (UViewportInteractionsBehaviorSource* const ViewportInteractionBehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		ViewportInteractionBehaviorSource->ClearMouseCursorOverride();
	}
}

void UViewportClickInteraction::OnForceEndCapture()
{
	if (UViewportInteractionsBehaviorSource* const ViewportInteractionBehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		ViewportInteractionBehaviorSource->ClearMouseCursorOverride();
	}
}
