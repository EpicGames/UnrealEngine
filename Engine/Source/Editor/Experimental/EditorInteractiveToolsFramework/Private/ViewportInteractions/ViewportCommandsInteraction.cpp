// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportCommandsInteraction.h"
#include "BaseBehaviors/KeyInputBehavior.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"

UViewportCommandsInteraction::UViewportCommandsInteraction()
{
	InteractionName = UE::Editor::ViewportInteractions::Commands;

	UKeyInputBehavior* KeyInputBehavior = NewObject<UKeyInputBehavior>();
	KeyInputBehavior->Initialize(this, GetKeys());

	KeyInputBehavior->SetDefaultPriority(UE::Editor::ViewportInteractions::VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY);
	KeyInputBehavior->bRequireAllKeys = false;
	KeyInputBehavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState)
	{
		// We don't want commands to work while controlling the camera with the mouse.
		return CanBeActivated();
	};
	KeyInputBehaviorWeak = KeyInputBehavior;

	RegisterInputBehavior(KeyInputBehavior);

}

void UViewportCommandsInteraction::SetCommands(
	const TSharedPtr<FUICommandList>& InEditorCommandList, const TArray<TSharedPtr<FUICommandInfo>>& InCommands
)
{
	EditorCommandList = InEditorCommandList;
	Commands = InCommands;

	if (TStrongObjectPtr<UKeyInputBehavior> KeyInputBehaviorPinned = KeyInputBehaviorWeak.Pin())
	{
		KeyInputBehaviorPinned->Initialize(this, GetKeys());
	}
}

void UViewportCommandsInteraction::OnKeyPressed(const FKey& InKeyID)
{
	if (IsEnabled() && bExecuteOnPress)
	{
		ExecuteCommand(InKeyID);
	}
}

void UViewportCommandsInteraction::OnKeyReleased(const FKey& InKeyID)
{
	if (IsEnabled() && !bExecuteOnPress)
	{
		ExecuteCommand(InKeyID);
	}
}

void UViewportCommandsInteraction::OnCommandChordChanged()
{
	if (TStrongObjectPtr<UKeyInputBehavior> KeyInputBehaviorPinned = KeyInputBehaviorWeak.Pin())
	{
		KeyInputBehaviorPinned->Initialize(this, GetKeys());
	}
}

TArray<TSharedPtr<FUICommandInfo>> UViewportCommandsInteraction::GetCommands() const
{
	return Commands;
}

TArray<FKey> UViewportCommandsInteraction::GetKeys()
{
	TArray<FKey> Keys;
	for (int32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
		for (const TSharedPtr<FUICommandInfo>& Command : Commands)
		{
			if (Command)
			{
				TSharedRef<const FInputChord> Chord = Command->GetActiveChord(ChordIndex);
				Keys.AddUnique(Chord->Key);
			}
		}
	}

	return Keys;
}

void UViewportCommandsInteraction::ExecuteCommand(const FKey& InKeyID)
{
	if (!EditorCommandList)
	{
		return;
	}

	for (const TSharedPtr<FUICommandInfo>& Command : Commands)
	{
		if (Command.IsValid())
		{
			if (MatchesCommandKey(Command, InKeyID))
			{
				if (EditorCommandList->CanExecuteAction(Command.ToSharedRef()))
				{
					EditorCommandList->ExecuteAction(Command.ToSharedRef());
					return;
				}
			}
		}
	}
}

bool UViewportCommandsInteraction::MatchesCommandKey(const TSharedPtr<FUICommandInfo>& InCommandInfo, const FKey& InKeyID) const
{
	for (int32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
		TSharedRef<const FInputChord> InputChord = InCommandInfo->GetActiveChord(ChordIndex);
		if (InputChord->Key == InKeyID)
		{
			if (!InputChord->HasAnyModifierKeys())
			{
				return true;
			}
			bool bSuccess = false;
			bSuccess |= InputChord->NeedsShift() && IsShiftDown();
			bSuccess |= InputChord->NeedsControl() && IsCtrlDown();
			bSuccess |= InputChord->NeedsAlt() && IsAltDown();
			return bSuccess;
		}
	}
	return false;
}

bool UViewportCommandsInteraction::CanBeActivated() const
{
	return !IsAnyMouseButtonDown() && !IsMouseLooking();
}
