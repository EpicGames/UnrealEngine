// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportDollyInteraction.h"

#include "ViewportClientNavigationHelper.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "EditorViewportClient.h"

UViewportDollyInteraction::UViewportDollyInteraction()
{
	using namespace UE::Editor::ViewportInteractions;
	InteractionName = PerspectiveDolly;
	
	NormalBindings = FButtonBindings({
		FButtonBinding(EKeys::RightMouseButton).TriggersStart(),
		FButtonBinding(EKeys::LeftAlt).RequiredToContinue(false)
	});
	
	CameraLockedBindings = FButtonBindings({
		FButtonBinding(EKeys::RightMouseButton).TriggersStart(),
		FButtonBinding(EKeys::LeftAlt).Required(false)
	});
	
	ViewportClickDragBehavior->SetBindings(UViewportClickDragBehavior::FGetBindingsDelegate::CreateUObject(this, &UViewportDollyInteraction::GetBindings));
	
	// Legacy also included this combination binding:
	UViewportClickDragBehavior* AlternativeBindingBehavior = NewObject<UViewportClickDragBehavior>();
	
	AlternativeBindingBehavior->Initialize(this);
	AlternativeBindingBehavior->SetBindings({
		FButtonBinding(EKeys::LeftMouseButton).TriggersStart(),
		FButtonBinding(EKeys::MiddleMouseButton).TriggersStart(),
		FButtonBinding(EKeys::LeftAlt).RequiredToContinue(false)
	});
	
	RegisterInputBehavior(AlternativeBindingBehavior);
}

bool UViewportDollyInteraction::CanBeActivated() const
{
	if (IsEnabled())
	{
		if (FEditorViewportClient* Client = GetEditorViewportClient())
		{
			return Client->IsPerspective();
		}
	}
	return false;
}

void UViewportDollyInteraction::OnDragStart(const FInputDeviceRay& InDragStartPos)
{
	UViewportDragInteraction::OnDragStart(InDragStartPos);
	
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(true);
	}
}

void UViewportDollyInteraction::OnDragEnd(const FInputDeviceRay& InDragEndPos)
{
	UViewportDragInteraction::OnDragEnd(InDragEndPos);
	
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(false);
	}
}

void UViewportDollyInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	if (FEditorViewportClient* Client = GetEditorViewportClient())
	{
		if (FViewportClientNavigationHelper* Helper = Client->GetViewportNavigationHelper())
		{
			const ULevelEditorViewportSettings* Settings = GetDefault<ULevelEditorViewportSettings>(); 
			const float Direction = Settings->bInvertRightMouseDollyYAxis ? -1.0f : 1.0f;
			// TODO: Switch to using Settings->MouseSensitivty?
			Helper->OrbitDelta.Z += (InMouseDeltaX + InMouseDeltaY * Direction);
		}
	}
}

const UE::Editor::ViewportInteractions::FButtonBindings& UViewportDollyInteraction::GetBindings() const
{
	if (FEditorViewportClient* Client = GetEditorViewportClient())
	{
		if (Client->IsCameraLocked())
		{
			return CameraLockedBindings;
		}
	}
	return NormalBindings;
}

