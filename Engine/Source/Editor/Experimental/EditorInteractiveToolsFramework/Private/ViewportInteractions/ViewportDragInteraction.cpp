// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportDragInteraction.h"
#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "Behaviors/ViewportClickDragBehavior.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportDragInteraction::UViewportDragInteraction()
{
	bIsDragging = false;
	const FName UniqueName = MakeUniqueObjectName(GetTransientPackageAsObject(), UViewportClickDragBehavior::StaticClass(), *FString::Printf(TEXT("%s_Behavior"), *GetClass()->GetName()));

	UViewportClickDragBehavior* ClickDragBehavior = NewObject<UViewportClickDragBehavior>(GetTransientPackageAsObject(), UniqueName);
	ClickDragBehavior->Initialize(this);

	// Other higher priority drag interactions might steal capture (e.g. Gizmo Indirect Manipulation)
	ClickDragBehavior->SetAllowsCaptureStealing(true);
	ClickDragBehavior->SetUsesUnboundedCursor(true);

	ViewportClickDragBehavior = ClickDragBehavior;

	RegisterInputBehavior(ClickDragBehavior);
}

FInputRayHit UViewportDragInteraction::CanBeginClickDragSequence(const FInputDeviceRay& InPressPos)
{
	return CanBeActivated() ? FInputRayHit(TNumericLimits<double>::Max()) : FInputRayHit();
}

void UViewportDragInteraction::OnDragStart(const FInputDeviceRay& InDragStartPos)
{
	if (!IsEnabled())
	{
		bIsDragging = false;
		return;
	}

	bIsDragging = true;

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetMouseCursorOverride(EMouseCursor::None);
	}
}

void UViewportDragInteraction::OnDrag(const FDragArgs& InDrag)
{
	if (!IsEnabled())
	{
		return;
	}
	
	OnDragDelta(static_cast<float>(InDrag.ScreenDelta.X), static_cast<float>(InDrag.ScreenDelta.Y));
}

void UViewportDragInteraction::OnDragEnd(const FInputDeviceRay& InDragEndPos)
{
	bIsDragging = false;
}

void UViewportDragInteraction::OnEndCapture(EEndCaptureReason InReason)
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->ClearMouseCursorOverride();
	}
}
