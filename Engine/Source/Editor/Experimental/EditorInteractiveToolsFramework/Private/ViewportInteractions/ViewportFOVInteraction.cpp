// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportFOVInteraction.h"
#include "BaseBehaviors/KeyInputBehavior.h"
#include "EditorViewportClient.h"
#include "ViewportClientNavigationHelper.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportFOVInteraction::UViewportFOVInteraction()
{
	InteractionName = UE::Editor::ViewportInteractions::FOV;
	Groups = { UE::Editor::ViewportInteractions::CameraFly };

	// Separating keys which work only when Mouse Looking from numpad ones and arrows, which always work.
	// This prevents this IKeyInputBehaviorTarget from capturing unwanted inputs

	// Mouse Looking only FOV controls
	UKeyInputBehavior* KeyInputBehavior = NewObject<UKeyInputBehavior>();
	KeyInputBehavior->Initialize(this, GetKeys());
	KeyInputBehavior->SetDefaultPriority(UE::Editor::ViewportInteractions::VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY);
	KeyInputBehavior->bRequireAllKeys = false;
	KeyInputBehavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState) 
	{
		if (FInputDeviceState::IsAltKeyDown(InputDeviceState) || FInputDeviceState::IsCtrlKeyDown(InputDeviceState)
			|| FInputDeviceState::IsShiftKeyDown(InputDeviceState) || FInputDeviceState::IsCmdKeyDown(InputDeviceState))
		{
			return false;
		}

		bool bResult = true;

		if (InputDeviceState.InputDevice == EInputDevices::Keyboard)
		{
			using namespace UE::Editor::ViewportInteractions;

			const FKey& ActiveKey = InputDeviceState.Keyboard.ActiveKey.Button;

			// These commands should only be capture if a mouse button is down (defaults are Z/C)
			if (CommandMatchesKey(FViewportNavigationCommands::Get().FovZoomOut, ActiveKey) ||
				CommandMatchesKey(FViewportNavigationCommands::Get().FovZoomIn, ActiveKey))
			{
				bool bIsGizmoDragging = false;
				if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
				{
					bIsGizmoDragging = BehaviorSource->IsGizmoDragging();
				}

				bResult = IsAnyMouseButtonDown() && !bIsGizmoDragging;
				// If a mouse button is down, but mouse hasn't been dragged yet, we need to tell the behavior source that
				// User is performing mouse look or at least camera movement. This also prevents regular clicks from performing their action on button release
				// e.g. we don't want a RMB release to summon a context menu after mouse looking
				if (bResult)
				{
					if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
					{
						BehaviorSource->SetIsMouseLooking(true);
					}
				}
			}
		}

		return bResult;
	};
	KeyInputBehaviorWeak = KeyInputBehavior;

	RegisterInputBehavior(KeyInputBehavior);
}

void UViewportFOVInteraction::BeginDestroy()
{
	Super::BeginDestroy();

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->OnMouseLookingStateChanged().RemoveAll(this);
	}
}

void UViewportFOVInteraction::OnKeyPressed(const FKey& InKeyID)
{
	if (IsEnabled())
	{
		constexpr bool bIsPressed = true;
		UpdateKeyState(InKeyID, bIsPressed);
	}
}

void UViewportFOVInteraction::OnKeyReleased(const FKey& InKeyID)
{
	if (IsEnabled())
	{
		constexpr bool bIsPressed = false;
		UpdateKeyState(InKeyID, bIsPressed);
	}
}

void UViewportFOVInteraction::OnForceEndCapture()
{
	ResetImpulse();
}

void UViewportFOVInteraction::Tick(float InDeltaTime) const
{
	if (const FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		if (FViewportClientNavigationHelper* const NavigationHelper = EditorViewportClient->GetViewportNavigationHelper())
		{
			if (ZoomOutInImpulse != 0)
			{
				if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
				{
					BehaviorSource->SetCameraHasMoved(true);
				}

				NavigationHelper->ImpulseDataDelta.ZoomOutInImpulse += ZoomOutInImpulse;
			}
		}
	}
}
void UViewportFOVInteraction::Initialize(UViewportInteractionsBehaviorSource* const InViewportInteractionsBehaviorSource)
{
	Super::Initialize(InViewportInteractionsBehaviorSource);

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->OnMouseLookingStateChanged().AddUObject(this, &UViewportFOVInteraction::OnMouseLookingChanged);
	}
}

void UViewportFOVInteraction::UpdateKeyState(const FKey& InKeyID, bool bInIsPressed)
{
	using namespace UE::Editor::ViewportInteractions;

	if ((CommandMatchesKey(FViewportNavigationCommands::Get().FovZoomOut, InKeyID) && IsMouseLooking())
		|| InKeyID == EKeys::NumPadOne)
	{
		ZoomOutInImpulse = bInIsPressed ? 1.0f : 0.0f;
	}
	else if ((CommandMatchesKey(FViewportNavigationCommands::Get().FovZoomIn, InKeyID) && IsMouseLooking())
			 || InKeyID == EKeys::NumPadThree)
	{
		ZoomOutInImpulse = bInIsPressed ? -1.0f : 0.0f;
	}
}

TArray<FKey> UViewportFOVInteraction::GetKeys() const
{
	TArray<FKey> Keys;

	for (int32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);

		for (const TSharedPtr<FUICommandInfo>& Command : GetCommands())
		{
			Keys.Add(Command->GetActiveChord(ChordIndex)->Key);
		}
	}

	// Hardcoded
	Keys.Append({ EKeys::Add, EKeys::Subtract });
	Keys.Append(GetNumpadKeys());

	return Keys;
}

TArray<FKey> UViewportFOVInteraction::GetNumpadKeys() const
{
	//TODO: currently hardcoded
	return { EKeys::NumPadOne, EKeys::NumPadThree };
}

void UViewportFOVInteraction::OnCommandChordChanged()
{
	if (TStrongObjectPtr<UKeyInputBehavior> KeyInputBehaviorPinned = KeyInputBehaviorWeak.Pin())
	{
		KeyInputBehaviorPinned->Initialize(this, GetKeys());
	}
}

TArray<TSharedPtr<FUICommandInfo>> UViewportFOVInteraction::GetCommands() const
{
	if (FViewportNavigationCommands::IsRegistered())
	{
		return { FViewportNavigationCommands::Get().FovZoomOut, FViewportNavigationCommands::Get().FovZoomIn };
	}

	return {};
}

void UViewportFOVInteraction::OnMouseLookingChanged(bool bInIsMouseLooking)
{
	ResetImpulse();
}

void UViewportFOVInteraction::ResetImpulse()
{
	ZoomOutInImpulse = 0.0f;
}
