// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportInteraction.h"

#include "EditorModeManager.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Commands/UICommandInfo.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportInteraction::UViewportInteraction()
{
	// Register to chord changes, so we can re-initialize behaviors with new inputs accordingly
	OnChordChangedDelegateHandle = FInputBindingManager::Get().RegisterUserDefinedChordChanged(
		FOnUserDefinedChordChanged::FDelegate::CreateUObject(this, &UViewportInteraction::OnUserDefinedChordChanged)
	);
}

bool UViewportInteraction::IsShiftDown() const
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		return BehaviorSource->IsShiftDown();
	}

	return false;
}

bool UViewportInteraction::IsAltDown() const
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		return BehaviorSource->IsAltDown();
	}

	return false;
}

bool UViewportInteraction::IsCtrlDown() const
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		return BehaviorSource->IsCtrlDown();
	}

	return false;
}

bool UViewportInteraction::IsLeftMouseButtonDown() const
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		return BehaviorSource->IsLeftMouseButtonDown();
	}

	return false;
}

bool UViewportInteraction::IsMiddleMouseButtonDown() const
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		return BehaviorSource->IsMiddleMouseButtonDown();
	}

	return false;
}

bool UViewportInteraction::IsRightMouseButtonDown() const
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		return BehaviorSource->IsRightMouseButtonDown();
	}

	return false;
}

bool UViewportInteraction::IsAnyMouseButtonDown() const
{
	return IsLeftMouseButtonDown() || IsMiddleMouseButtonDown() || IsRightMouseButtonDown();
}

bool UViewportInteraction::IsMouseLooking() const
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		return BehaviorSource->IsMouseLooking();
	}

	return false;
}

void UViewportInteraction::SetEnabled(bool bInEnabled)
{
	bEnabled = bInEnabled;
}

bool UViewportInteraction::IsEnabled() const
{
	return bEnabled;
}

FName UViewportInteraction::GetInteractionName() const
{
	return InteractionName.IsNone() ? StaticClass()->GetFName() : InteractionName;
}

const TArray<FName>& UViewportInteraction::GetInteractionGroups() const
{
	return Groups;
}

void UViewportInteraction::Initialize(UViewportInteractionsBehaviorSource* const InViewportInteractionsBehaviorSource)
{
	ViewportInteractionsBehaviorSource = InViewportInteractionsBehaviorSource;

	SetEnabled(true);
}

UViewportInteractionsBehaviorSource* UViewportInteraction::GetViewportInteractionsBehaviorSource() const
{
	if (TStrongObjectPtr<UViewportInteractionsBehaviorSource> BehaviorSource = ViewportInteractionsBehaviorSource.Pin())
	{
		return BehaviorSource.Get();
	}

	return nullptr;
}

void UViewportInteraction::BeginDestroy()
{
	Super::BeginDestroy();

	if (OnChordChangedDelegateHandle.IsValid())
	{
		FInputBindingManager::Get().UnregisterUserDefinedChordChanged(OnChordChangedDelegateHandle);
	}
}

FEditorModeTools* UViewportInteraction::GetModeTools() const
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		return BehaviorSource->GetModeTools();
	}

	return nullptr;
}

FEditorViewportClient* UViewportInteraction::GetEditorViewportClient() const
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		return BehaviorSource->GetEditorViewportClient();
	}

	return nullptr;
}

void UViewportInteraction::SetMouseCursorOverride(EMouseCursor::Type InMouseCursor)
{
	if (UViewportInteractionsBehaviorSource* Source = GetViewportInteractionsBehaviorSource())
	{
		Source->SetMouseCursorOverride(InMouseCursor);
	}
}

void UViewportInteraction::ClearMouseCursorOverride()
{
	if (UViewportInteractionsBehaviorSource* Source = GetViewportInteractionsBehaviorSource())
	{
		Source->ClearMouseCursorOverride();
	}
}

void UViewportInteraction::OnUserDefinedChordChanged(const FUICommandInfo& InCommandInfo)
{
	for (const TSharedPtr<FUICommandInfo>& Command : GetCommands())
	{
		if (Command.IsValid())
		{
			// Is this enough to ensure we're only triggering for the right commands - and we're not skipping when required?
			if (Command->GetBindingContext() == InCommandInfo.GetBindingContext()
				&& Command->GetCommandName() == InCommandInfo.GetCommandName())
			{
				OnCommandChordChanged();
			}
		}
	}
}

void UViewportInteraction::RegisterInputBehavior(UInputBehavior* InBehavior)
{
	InputBehaviors.AddUnique(InBehavior);
}

void UViewportInteraction::RegisterInputBehaviors(TArray<UInputBehavior*> InBehaviors)
{
	for (UInputBehavior* InputBehavior : InBehaviors)
	{
		RegisterInputBehavior(InputBehavior);
	}
}

bool UViewportInteraction::IsCurrentModeSupported(const FEditorModeTools* InModeTools) const
{
	if (InModeTools)
	{
		for (const FEditorModeID UnsupportedMode : GetUnsupportedModes())
		{
			if (InModeTools->IsModeActive(UnsupportedMode))
			{
				return false;
			}
		}
	}

	return true;
}
