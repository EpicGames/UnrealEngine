// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

#include "BaseBehaviors/KeyAsModifierInputBehavior.h"
#include "ContextObjectStore.h"
#include "EditorGizmos/EditorTransformGizmo.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "UnrealClient.h"
#include "ViewportClientNavigationHelper.h"
#include "ViewportInteractions/ViewportInteraction.h"

namespace UE::Editor::ViewportInteractions
{
// CVar initializer
static int32 UseITFViewportInteractions = 0;
static FAutoConsoleVariableRef CVarEnableITFViewportInteractions(
	TEXT("ViewportInteractions.EnableITFInteractions"),
	UseITFViewportInteractions,
	TEXT("Are ITF Viewport Interactions enabled?"),
	FConsoleVariableDelegate::CreateLambda(
		[](const IConsoleVariable* InVariable)
		{
			ToggleEditorViewportInteractions(UseITFViewportInteractions == 1);
		}
	)
);

// CVar initializer
static int32 VerboseITFViewportInteractions = 0;
static FAutoConsoleVariableRef CVarEnableVerboseITFViewportInteractions(
	TEXT("ViewportInteractions.Verbose"),
	VerboseITFViewportInteractions,
	TEXT("Enables verbose logging for ITF Viewport Interactions"),
	FConsoleVariableDelegate::CreateLambda(
		[](const IConsoleVariable* InVariable)
		{
		}
	)
);
	
TAutoConsoleVariable<int32> CVarMouseRestoration(
	TEXT("ViewportInteractions.MouseRestoration"),
	1,
	TEXT("Set how the viewport handles the mouse when the cursor stops being invisible\n")
	TEXT("	0 - Do nothing\n")
	TEXT("	1 - Reset to initial position (default)\n")
	TEXT("	2 - Reset to initial position if outside the viewport\n")	
);

bool UseEditorViewportInteractions()
{
	return UseITFViewportInteractions == 1;
}

bool IsVerbose()
{
	return VerboseITFViewportInteractions == 1;
}

void ToggleEditorViewportInteractions(bool bInEnable)
{
	UseITFViewportInteractions = bInEnable;
	if (UseITFViewportInteractions)
	{
		OnEditorViewportInteractionsActivated().Broadcast();
	}
	else
	{
		OnEditorViewportInteractionsDeactivated().Broadcast();
	}
}

FOnEditorViewportInteractionsToggleDelegate& OnEditorViewportInteractionsActivated()
{
	static FOnEditorViewportInteractionsToggleDelegate OnViewportInteractionsActivated;
	return OnViewportInteractionsActivated;
}

FOnEditorViewportInteractionsToggleDelegate& OnEditorViewportInteractionsDeactivated()
{
	static FOnEditorViewportInteractionsToggleDelegate OnViewportInteractionsDeactivated;
	return OnViewportInteractionsDeactivated;
}

bool CommandMatchesKey(const TSharedPtr<FUICommandInfo>& InCommandInfo, const FKey& InKeyID)
{
	for (int32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
		if (InCommandInfo->GetActiveChord(ChordIndex)->Key == InKeyID)
		{
			return true;
		}
	}

	return false;
}

static constexpr int MODIFIERS_PRIORITY = VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY;

} // namespace UE::Editor::ViewportInteractions

DEFINE_LOG_CATEGORY(LogITFViewportInteractions)

UToolsContextCursorAPI* GetCursorAPI(const FEditorViewportClient* InViewportClient)
{
	if (InViewportClient)
	{
		if (FEditorModeTools* ModeTools = InViewportClient->GetModeTools())
		{
			if (const UModeManagerInteractiveToolsContext* const InteractiveToolsContext = ModeTools->GetInteractiveToolsContext())
			{
				if (const UContextObjectStore* const ContextObjectStore = InteractiveToolsContext->ContextObjectStore)
				{
					return ContextObjectStore->FindContext<UToolsContextCursorAPI>();
				}
			}
		}
	}

	return nullptr;
}

void UViewportInteractionsBehaviorSource::OnUpdateModifierState(int InModifierID, bool bInIsOn)
{
	if (InModifierID == UE::Editor::ViewportInteractions::ShiftKeyMod)
	{
		SetShiftKeyState(bInIsOn);
	}
	else if (InModifierID == UE::Editor::ViewportInteractions::CtrlKeyMod)
	{
		SetCtrlKeyState(bInIsOn);
	}
	else if (InModifierID == UE::Editor::ViewportInteractions::AltKeyMod)
	{
		SetAltKeyState(bInIsOn);
	}
	else if (InModifierID == UE::Editor::ViewportInteractions::LeftMouseButtonMod)
	{
		bIsLeftMouseButtonDown = bInIsOn;
	}
	else if (InModifierID == UE::Editor::ViewportInteractions::MiddleMouseButtonMod)
	{
		bIsMiddleMouseButtonDown = bInIsOn;
	}
	else if (InModifierID == UE::Editor::ViewportInteractions::RightMouseButtonMod)
	{
		bIsRightMouseButtonDown = bInIsOn;
	}
}

void UViewportInteractionsBehaviorSource::OnForceEndCapture()
{
	// We reset modifiers only if editor has lost focus
	if (!FApp::HasFocus())
	{
		SetShiftKeyState(false);
		SetCtrlKeyState(false);
		SetAltKeyState(false);
	}

	bIsLeftMouseButtonDown = false;
	bIsMiddleMouseButtonDown = false;
	bIsRightMouseButtonDown = false;

	SetIsMouseLooking(false);
}

void UViewportInteractionsBehaviorSource::Initialize(UEditorInteractiveToolsContext* InInteractiveToolsContext)
{
	if (!InInteractiveToolsContext)
	{
		return;
	}

	EditorInteractiveToolsContextWeak = InInteractiveToolsContext;

	BehaviorSet = NewObject<UInputBehaviorSet>();

	// Modifiers handling, directly from "this"
	{
		UKeyAsModifierInputBehavior* KeyAsModifierInputBehavior = NewObject<UKeyAsModifierInputBehavior>();
		KeyAsModifierInputBehavior->Initialize(
			this, UE::Editor::ViewportInteractions::ShiftKeyMod, FInputDeviceState::IsShiftKeyDown
		);
		KeyAsModifierInputBehavior->Initialize(
			this, UE::Editor::ViewportInteractions::CtrlKeyMod, FInputDeviceState::IsCtrlKeyDown
		);
		KeyAsModifierInputBehavior->Initialize(
			this, UE::Editor::ViewportInteractions::AltKeyMod, FInputDeviceState::IsAltKeyDown
		);
		KeyAsModifierInputBehavior->SetDefaultPriority(UE::Editor::ViewportInteractions::MODIFIERS_PRIORITY);
		CoreBehaviors.Add(KeyAsModifierInputBehavior);

		UMouseButtonAsModifierInputBehavior* MouseButtonAsModifierInputBehavior =
			NewObject<UMouseButtonAsModifierInputBehavior>();
		MouseButtonAsModifierInputBehavior->Initialize(
			this,
			UE::Editor::ViewportInteractions::LeftMouseButtonMod,
			[](const FInputDeviceState& InputState)
			{
				return InputState.Mouse.Left.bDown || InputState.Mouse.Left.bPressed;
			}
		);
		MouseButtonAsModifierInputBehavior->Initialize(
			this,
			UE::Editor::ViewportInteractions::MiddleMouseButtonMod,
			[](const FInputDeviceState& InputState)
			{
				return InputState.Mouse.Middle.bDown || InputState.Mouse.Middle.bPressed;
			}
		);
		MouseButtonAsModifierInputBehavior->Initialize(
			this,
			UE::Editor::ViewportInteractions::RightMouseButtonMod,
			[](const FInputDeviceState& InputState)
			{
				return InputState.Mouse.Right.bDown || InputState.Mouse.Right.bPressed;
			}
		);
		MouseButtonAsModifierInputBehavior->SetDefaultPriority(UE::Editor::ViewportInteractions::MODIFIERS_PRIORITY);
		CoreBehaviors.Add(MouseButtonAsModifierInputBehavior);
	}

	Reset();
}

void UViewportInteractionsBehaviorSource::RegisterBehaviorSources()
{
	DeregisterBehaviorSources();

	if (const UEditorInteractiveToolsContext* InteractiveToolsContext = EditorInteractiveToolsContextWeak.Get())
	{
		if (UInputRouter* InputRouter = InteractiveToolsContext->InputRouter)
		{
			InputRouter->RegisterSource(this);
		}

		// Gizmos delegates
		{
			if (UInteractiveToolManager* ToolManager = InteractiveToolsContext->ToolManager)
			{
				// Trying to gather Transform gizmo
				if (UInteractiveGizmoManager* PairedGizmoManager = ToolManager->GetPairedGizmoManager())
				{
					const TArray<UInteractiveGizmo*> Gizmos =
						PairedGizmoManager->FindAllGizmosOfType("EditorTransformGizmoBuilder");

					if (!Gizmos.IsEmpty())
					{
						TransformGizmo = Cast<UEditorTransformGizmo>(Gizmos[0]);
						RegisterProxyDelegates();
					}
				}

				// If not available yet, register to creation delegate so we can setup as soon as it exists
				if (!TransformGizmo)
				{
					// If New TRS Gizmos are not enabled, we need to be able to know as soon as they are enabled
					// This allows to retrieve the Transform Gizmo, from which we register to drag Begin and End delegates
					if (UContextObjectStore* ContextStore = ToolManager->GetContextObjectStore())
					{
						if (UEditorTransformGizmoContextObject* ContextObject =
								ContextStore->FindContext<UEditorTransformGizmoContextObject>())
						{
							ContextObject->OnGizmoCreatedDelegate().AddUObject(
								this, &UViewportInteractionsBehaviorSource::OnGizmoCreatedDelegate
							);
						}
					}
				}
			}
		}
	}
}

void UViewportInteractionsBehaviorSource::DeregisterBehaviorSources()
{
	if (const UEditorInteractiveToolsContext* InteractiveToolsContext = EditorInteractiveToolsContextWeak.Get())
	{
		if (UInputRouter* InputRouter = InteractiveToolsContext->InputRouter)
		{
			InputRouter->DeregisterSource(this);
		}

		DeregisterProxyDelegates();
	}
}

void UViewportInteractionsBehaviorSource::Tick(float InDeltaTime)
{
	for (const TPair<FName, TObjectPtr<UViewportInteraction>>& Pair : ViewportInteractions)
	{
		if (UViewportInteraction* Interaction = Pair.Value)
		{
			Interaction->Tick(InDeltaTime);
		}
	}
	
	if (CursorOverride != LastCursorOverride)
	{
		if (LastCursorOverride.Get(EMouseCursor::Default) == EMouseCursor::None)
		{
			RestoreCursorPosition();
		}
		else if (CursorOverride.Get(EMouseCursor::Default) == EMouseCursor::None)
		{
			CacheCursorPosition();
		}
		
		LastCursorOverride = CursorOverride;
	}
}

void UViewportInteractionsBehaviorSource::RenderTools(IToolsContextRenderAPI* InRenderAPI) const
{
	for (const TPair<FName, TObjectPtr<UViewportInteraction>>& Pair : ViewportInteractions)
	{
		if (UViewportInteraction* Interaction = Pair.Value)
		{
			Interaction->Render(InRenderAPI);
		}
	}
}

void UViewportInteractionsBehaviorSource::DrawTools(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) const
{
	for (const TPair<FName, TObjectPtr<UViewportInteraction>>& Pair : ViewportInteractions)
	{
		if (UViewportInteraction* Interaction = Pair.Value)
		{
			Interaction->Draw(InCanvas, InRenderAPI);
		}
	}
}

void UViewportInteractionsBehaviorSource::AddInteractions(const TArray<const UClass*>& InInteractions, bool bInReregister)
{
	for (const UClass* InteractionClass : InInteractions)
	{
		AddInteraction(InteractionClass);
	}

	if (bInReregister)
	{
		// Refreshing registered behaviors
		RegisterBehaviorSources();
	}
}

UViewportInteraction* UViewportInteractionsBehaviorSource::AddInteraction(const UClass* InInteractionClass, bool bInReregister)
{
	if (!BehaviorSet && EditorInteractiveToolsContextWeak.IsValid())
	{
		Initialize(EditorInteractiveToolsContextWeak.Get());
	}

	// Make sure the required interaction should be added
	{
		const UViewportInteraction* DefaultInteraction = GetDefault<UViewportInteraction>(InInteractionClass);

		if (ViewportInteractions.Contains(DefaultInteraction->GetInteractionName()))
		{
			UE_LOGF(LogITFViewportInteractions, Warning, "Interaction named \"%ls\" has already been added.", *DefaultInteraction->GetInteractionName().ToString());
			return nullptr;
		}

		if (const FEditorModeTools* ModeTools = GetModeTools())
		{
			if (!DefaultInteraction->IsCurrentModeSupported(ModeTools))
			{
				// Current mode does not support this interaction: skip it
				return nullptr;
			}
		}

		if (UnsupportedInteractions.Contains(DefaultInteraction->GetInteractionName()))
		{
			return nullptr;
		}
	}

	UViewportInteraction* Interaction = NewObject<UViewportInteraction>(GetTransientPackage(), InInteractionClass);

	Interaction->Initialize(this);

	UE_LOGF(LogITFViewportInteractions, Verbose, "Initializing Viewport Interaction \"%ls\"", *InInteractionClass->GetName());
	ViewportInteractions.Add(Interaction->GetInteractionName(), Interaction);
	
	for (const FName& GroupName : Interaction->GetInteractionGroups())
	{
		FViewportInteractionGroup& Group = ViewportInteractionGroups.FindOrAdd(GroupName);
		Group.Interactions.Add(Interaction);
	}

	for (UInputBehavior* const InputBehavior : Interaction->GetInputBehaviors())
	{
		BehaviorSet->Add(InputBehavior, Interaction);
	}

	if (bInReregister)
	{
		// Refreshing registered behaviors
		RegisterBehaviorSources();
	}

	return Interaction;
}

UViewportInteraction* UViewportInteractionsBehaviorSource::FindInteraction(const UClass* InInteractionClass) const
{
	for (const TPair<FName, TObjectPtr<UViewportInteraction>>& Pair : ViewportInteractions)
	{
		if (Pair.Value->IsA(InInteractionClass))
		{
			return Pair.Value;
		}
	}
	return nullptr;
}

UViewportInteraction* UViewportInteractionsBehaviorSource::FindInteraction(const FName& InName) const
{
	return ViewportInteractions.FindRef(InName);
}

bool UViewportInteractionsBehaviorSource::IsAnyInteractionInGroupEnabled(const FName& InName) const
{
	if (const FViewportInteractionGroup* Group = ViewportInteractionGroups.Find(InName))
	{
		for (const UViewportInteraction* Interaction : Group->Interactions)
		{
			if (Interaction->IsEnabled())
			{
				return true;
			}
		}
	}
	return false;
}

bool UViewportInteractionsBehaviorSource::IsAnyInteractionInGroupActive(const FName& InName) const
{
	if (const FViewportInteractionGroup* Group = ViewportInteractionGroups.Find(InName))
	{
		for (const UViewportInteraction* Interaction : Group->Interactions)
		{
			if (Interaction->IsActive())
			{
				return true;
			}
		}
	}
	return false;
}

void UViewportInteractionsBehaviorSource::RemoveInteraction(const FName InInteractionName, bool bInReregister)
{
	if (UViewportInteraction* Interaction = ViewportInteractions.FindRef(InInteractionName))
	{
		UE_LOGF(LogITFViewportInteractions, Verbose, "Removing Viewport Interaction \"%ls\"", *Interaction->GetInteractionName().ToString());

		ViewportInteractions.Remove(Interaction->GetInteractionName());
		
		for (const FName& GroupName : Interaction->GetInteractionGroups())
		{
			if (FViewportInteractionGroup* Group = ViewportInteractionGroups.Find(GroupName))
			{
				Group->Interactions.Remove(Interaction);
			}
		}
		
		Interaction->Shutdown();

		BehaviorSet->RemoveBySource(Interaction);

		if (bInReregister)
		{
			// Refreshing registered behaviors
			RegisterBehaviorSources();
		}
	}
}

void UViewportInteractionsBehaviorSource::Reset()
{
	DeregisterProxyDelegates();
	DeregisterBehaviorSources();
	BehaviorSet->RemoveAll();

	// We are about to reset interactions list, but they might be garbage collected later.
	// We need to make sure they are properly cleaned up, e.g. in case they are listening to some delegates
	for (const TTuple<FName, TObjectPtr<UViewportInteraction>>& Pair : ViewportInteractions)
	{
		if (UViewportInteraction* const Interaction = Pair.Value)
		{
			Interaction->Shutdown();
		}
	}

	ViewportInteractions.Reset();
	ViewportInteractionGroups.Reset();
	
	for (const TObjectPtr<UInputBehavior>& Behavior : CoreBehaviors)
	{
		BehaviorSet->Add(Behavior.Get(), this);
	}
}

void UViewportInteractionsBehaviorSource::SetMouseCursorOverride(EMouseCursor::Type InMouseCursor)
{
	if (!CursorOverride.IsSet() || CursorOverride.GetValue() != InMouseCursor)
	{
		if (UToolsContextCursorAPI* CursorAPI = GetCursorAPI())
		{
			CursorAPI->SetCursorOverride(InMouseCursor);
			CursorOverride = InMouseCursor;
		}
	}
}

void UViewportInteractionsBehaviorSource::ClearMouseCursorOverride()
{
	if (UToolsContextCursorAPI* CursorAPI = GetCursorAPI())
	{
		CursorAPI->ClearCursorOverride();
		CursorOverride.Reset();
	}
}

UToolsContextCursorAPI* UViewportInteractionsBehaviorSource::GetCursorAPI()
{
	if (UToolsContextCursorAPI* CachedValue = WeakCursorAPI.Get())
	{
		return CachedValue;
	}
	
	if (const UEditorInteractiveToolsContext* Context = EditorInteractiveToolsContextWeak.Get())
	{
		if (const UContextObjectStore* const ContextObjectStore = Context->ContextObjectStore)
		{
			if (UToolsContextCursorAPI* FoundApi = ContextObjectStore->FindContext<UToolsContextCursorAPI>())
			{
				WeakCursorAPI = FoundApi;
				return FoundApi;
			}
		}
	}
	
	return nullptr;
}

void UViewportInteractionsBehaviorSource::CacheCursorPosition()
{
	if (const FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
	{
		HiddenCursorPosition = FIntPoint(
			EditorViewportClient->Viewport->GetMouseX(),
			EditorViewportClient->Viewport->GetMouseY()
		);
	}
}

void UViewportInteractionsBehaviorSource::RestoreCursorPosition()
{
	if (!HiddenCursorPosition.IsSet())
	{
		return;
	}

	if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
	{
		const FIntPoint CursorPosition = HiddenCursorPosition.GetValue();
		
		switch (UE::Editor::ViewportInteractions::CVarMouseRestoration.GetValueOnAnyThread())
		{
		case 1: // Reset the mouse back to its original position (parity with legacy)
			EditorViewportClient->Viewport->SetMouse(CursorPosition.X, CursorPosition.Y);
			break;
		case 2: // Reset the mouse to original position only when the mouse is no longer within the viewport
			{
				FIntPoint CurrentMousePosition;
				EditorViewportClient->Viewport->GetMousePos(CurrentMousePosition);
		
				const FIntPoint Size = EditorViewportClient->Viewport->GetSizeXY();
		
				if (CurrentMousePosition.X < 0 || CurrentMousePosition.X > Size.X
					|| CurrentMousePosition.Y < 0 || CurrentMousePosition.Y > Size.Y)
				{
					EditorViewportClient->Viewport->SetMouse(CursorPosition.X, CursorPosition.Y);				
				}
				break;
			}
		default:
			break;
		}
	}
	
	HiddenCursorPosition.Reset();
}

bool UViewportInteractionsBehaviorSource::IsMouseLooking() const
{
	if (FSlateApplication::Get().IsUsingTrackpad())
	{
		return true;
	}

	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		return EditorViewportClient->GetViewportNavigationHelper()->bIsMouseLooking;
	}

	return false;
}

void UViewportInteractionsBehaviorSource::SetIsMouseLooking(bool bInIsLooking)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		if (bInIsLooking)
		{
			if (CursorOverride.IsSet())
			{
				SetMouseCursorOverride(CursorOverride.GetValue());
			}
		}
		else
		{
			bCameraHasMoved = false;
			ClearMouseCursorOverride();
		}

		EditorViewportClient->GetViewportNavigationHelper()->bIsMouseLooking = bInIsLooking;
	}

	OnMouseLookingStateChanged().Broadcast(bInIsLooking);
}

void UViewportInteractionsBehaviorSource::SetCameraHasMoved(bool bInHasMoved)
{
	bCameraHasMoved = bInHasMoved;
}

FEditorModeTools* UViewportInteractionsBehaviorSource::GetModeTools() const
{
	if (const UEditorInteractiveToolsContext* const InteractiveToolsContext = EditorInteractiveToolsContextWeak.Get())
	{
		return InteractiveToolsContext->GetParentEditorModeManager();
	}
	return nullptr;
}

FEditorViewportClient* UViewportInteractionsBehaviorSource::GetEditorViewportClient() const
{
	FEditorViewportClient* EditorViewportClient = nullptr;

	if (const UEditorInteractiveToolsContext* const InteractiveToolsContext = EditorInteractiveToolsContextWeak.Get())
	{
		if (const FEditorModeTools* const ModeManager = InteractiveToolsContext->GetParentEditorModeManager())
		{
			EditorViewportClient = ModeManager->GetFocusedViewportClient();
		}

		if (!EditorViewportClient)
		{
			if (UInteractiveToolManager* const ToolManager = InteractiveToolsContext->ToolManager)
			{
				if (const IToolsContextQueriesAPI* const ContextQueriesAPI = ToolManager->GetContextQueriesAPI())
				{
					if (const FViewport* const Viewport = ContextQueriesAPI->GetFocusedViewport())
					{
						EditorViewportClient = static_cast<FEditorViewportClient*>(Viewport->GetClient());
					}
				}
			}
		}
	}

	return EditorViewportClient;
}

FEditorViewportClient* UViewportInteractionsBehaviorSource::GetHoveredEditorViewportClient() const
{
	FEditorViewportClient* EditorViewportClient = nullptr;

	if (const UEditorInteractiveToolsContext* const InteractiveToolsContext = EditorInteractiveToolsContextWeak.Get())
	{
		if (const FEditorModeTools* const ModeManager = InteractiveToolsContext->GetParentEditorModeManager())
		{
			EditorViewportClient = ModeManager->GetHoveredViewportClient();
		}

		if (!EditorViewportClient)
		{
			if (UInteractiveToolManager* const ToolManager = InteractiveToolsContext->ToolManager)
			{
				if (const IToolsContextQueriesAPI* const ContextQueriesAPI = ToolManager->GetContextQueriesAPI())
				{
					if (const FViewport* const Viewport = ContextQueriesAPI->GetHoveredViewport())
					{
						EditorViewportClient = static_cast<FEditorViewportClient*>(Viewport->GetClient());
					}
				}
			}
		}
	}

	return EditorViewportClient;
}

void UViewportInteractionsBehaviorSource::SetUnsupportedViewportInteractions(const TArray<FName>& InUnsupportedInteractions)
{
	UnsupportedInteractions = InUnsupportedInteractions;
}

void UViewportInteractionsBehaviorSource::SetAltKeyState(bool bInDown)
{
	bIsAltDown = bInDown;
	OnAltKeyStateChangedDelegate.Broadcast(bIsAltDown);
}

void UViewportInteractionsBehaviorSource::SetCtrlKeyState(bool bInDown)
{
	bIsCtrlDown = bInDown;
	OnCtrlKeyStateChangedDelegate.Broadcast(bIsCtrlDown);
}

void UViewportInteractionsBehaviorSource::SetShiftKeyState(bool bInDown)
{
	bIsShiftDown = bInDown;
	OnShiftKeyStateChangedDelegate.Broadcast(bIsShiftDown);
}

void UViewportInteractionsBehaviorSource::RegisterProxyDelegates()
{
	if (TransformGizmo)
	{
		// Register Proxy Delegates
		if (TObjectPtr<UTransformProxy> TransformProxy = TransformGizmo->ActiveTarget)
		{
			TransformProxy->OnBeginTransformEdit.AddUObject(this, &UViewportInteractionsBehaviorSource::OnGizmoMovementBegin);
			TransformProxy->OnEndTransformEdit.AddUObject(this, &UViewportInteractionsBehaviorSource::OnGizmoMovementEnd);
		}
	}
}

void UViewportInteractionsBehaviorSource::DeregisterProxyDelegates()
{
	if (TransformGizmo)
	{
		// Unregister Proxy delegates
		if (TObjectPtr<UTransformProxy> TransformProxy = TransformGizmo->ActiveTarget)
		{
			TransformProxy->OnBeginTransformEdit.RemoveAll(this);
			TransformProxy->OnTransformChanged.RemoveAll(this);
			TransformProxy->OnEndTransformEdit.RemoveAll(this);
		}

		TransformGizmo = nullptr;
	}

	if (TStrongObjectPtr<const UEditorInteractiveToolsContext> InteractiveToolsContext = EditorInteractiveToolsContextWeak.Pin())
	{
		if (UInteractiveToolManager* ToolManager = InteractiveToolsContext->ToolManager)
		{
			if (UContextObjectStore* ContextStore = ToolManager->GetContextObjectStore())
			{
				if (UEditorTransformGizmoContextObject* ContextObject = ContextStore->FindContext<UEditorTransformGizmoContextObject>())
				{
					ContextObject->OnGizmoCreatedDelegate().RemoveAll(this);
				}
			}
		}
	}
}

void UViewportInteractionsBehaviorSource::OnGizmoCreatedDelegate(UTransformGizmo* InTransformGizmo)
{
	if (!TransformGizmo)
	{
		TransformGizmo = Cast<UEditorTransformGizmo>(InTransformGizmo);

		RegisterProxyDelegates();
	}
}

void UViewportInteractionsBehaviorSource::OnGizmoMovementBegin(UTransformProxy* InTransformProxy)
{
	bGizmoDragging = true;
}

void UViewportInteractionsBehaviorSource::OnGizmoMovementEnd(UTransformProxy* InTransformProxy)
{
	bGizmoDragging = false;
}
