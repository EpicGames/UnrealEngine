// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportInteractionsUtilities.h"

#include "EditorModeManager.h"
#include "EditorDragTools/BoxSelectInteraction.h"
#include "EditorDragTools/FrustumSelectInteraction.h"
#include "EditorDragTools/MeasureToolInteraction.h"
#include "EditorDragTools/ViewportChangeInteraction.h"
#include "ViewportInteractions/ViewportCameraRotateInteraction.h"
#include "ViewportInteractions/ViewportCameraSpeedMouseWheelInteraction.h"
#include "ViewportInteractions/ViewportCameraTranslateInteraction.h"
#include "ViewportInteractions/ViewportDollyInteraction.h"
#include "ViewportInteractions/ViewportFOVInteraction.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportMoveYawInteraction.h"
#include "ViewportInteractions/ViewportOrbitInteraction.h"
#include "ViewportInteractions/ViewportOrthoPanInteraction.h"
#include "ViewportInteractions/ViewportPanInteraction.h"
#include "ViewportInteractions/ViewportSnapToggleInteraction.h"
#include "ViewportInteractions/ViewportViewAngleInteraction.h"
#include "ViewportInteractions/ViewportZoomInteraction.h"

namespace UE::Editor::ViewportInteractions
{

void AddDefaultCameraMovementInteractions(
	UViewportInteractionsBehaviorSource* InInteractionsBehaviorSource
)
{
	if (InInteractionsBehaviorSource)
	{
		InInteractionsBehaviorSource->AddInteractions(
			{UViewportCameraSpeedMouseWheelInteraction::StaticClass(),
				UViewportOrbitInteraction::StaticClass(),
				UViewportMoveYawInteraction::StaticClass(),
				UViewportMovePanInteraction::StaticClass(),
				UViewportTrackpadPanInteraction::StaticClass(),
				UViewportViewAngleInteraction::StaticClass(),
				UViewportOrthoPanInteraction::StaticClass(),
				UViewportPanInteraction::StaticClass(),
				UViewportDollyInteraction::StaticClass(),
				UViewportZoomInteraction::StaticClass(),
				UViewportCameraTranslateInteraction::StaticClass(),
				UViewportCameraRotateInteraction::StaticClass(),
				UViewportFOVInteraction::StaticClass(),
				UViewportSnapToggleInteraction::StaticClass()}
		);
	}
}

void AddDefaultDragToolsInteractions(UViewportInteractionsBehaviorSource* InInteractionsBehaviorSource)
{
	if (InInteractionsBehaviorSource)
	{
		InInteractionsBehaviorSource->AddInteraction<UFrustumSelectInteraction>();
		InInteractionsBehaviorSource->AddInteraction<UBoxSelectInteraction>();
		InInteractionsBehaviorSource->AddInteraction<UMeasureToolInteraction>();
		InInteractionsBehaviorSource->AddInteraction<UViewportChangeInteraction>();
	}
}

}
