// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportLegacyClickInteraction.h"

#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "BaseBehaviors/MouseHoverBehavior.h"

UViewportLegacyClickInteraction::UViewportLegacyClickInteraction()
{
	InteractionName = TEXT("Legacy Viewport Click");
	Groups = { UE::Editor::ViewportInteractions::ViewportClick };
}

void UViewportLegacyClickInteraction::BuildBehaviors()
{
	Super::BuildBehaviors();
	
	using namespace UE::Editor::ViewportInteractions;
	
	if (UViewportClickBehavior* ClickBehavior = ClickBehaviorWeak.Get())
	{
		// Lower priority by one step so that custom UViewportClickInteractions do not have to compete.
		ClickBehavior->SetDefaultPriority(FInputCapturePriority(CLICK_PRIORITY).MakeLower());
		
		ClickBehavior->SetBindings({
			FButtonBinding(EKeys::LeftMouseButton).Required(false).TriggersStart(true),
			FButtonBinding(EKeys::RightMouseButton).Required(false).TriggersStart(true),
			FButtonBinding(EKeys::MiddleMouseButton).Required(false).TriggersStart(true),
			
			FButtonBinding(EKeys::LeftControl).Required(false),
			FButtonBinding(EKeys::LeftCommand).Required(false),
			FButtonBinding(EKeys::LeftAlt).Required(false),
			FButtonBinding(EKeys::LeftShift).Required(false)
		});
	}
	
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(CLICK_PRIORITY);
	RegisterInputBehavior(HoverBehavior);
}

void UViewportLegacyClickInteraction::OnClickDown(const FInputDeviceRay& InClickPos)
{
	if (!OnUpdateHover(InClickPos))
	{
		SetMouseCursorOverride(EMouseCursor::Default);
	}
}

FInputRayHit UViewportLegacyClickInteraction::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	if (GetHitProxy(PressPos))
	{
		return FInputRayHit(0.0);
	}
	return FInputRayHit();
}

void UViewportLegacyClickInteraction::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	OnUpdateHover(DevicePos);
}

bool UViewportLegacyClickInteraction::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (HHitProxy* HitProxy = GetHitProxy(DevicePos))
	{
		SetMouseCursorOverride(HitProxy->GetMouseCursor());
		return true;
	}
	
	return false;
}

void UViewportLegacyClickInteraction::OnEndHover()
{
	ClearMouseCursorOverride();
}

void UViewportLegacyClickInteraction::ProcessClick_Internal(const FViewportClick& InViewportClick, HHitProxy* InHitProxy, FSceneView& View)
{
	if (FEditorViewportClient* ViewportClient = InViewportClick.GetViewportClient())
	{
		ViewportClient->ProcessClick(View, InHitProxy, InViewportClick.GetKey(), InViewportClick.GetEvent(), InViewportClick.GetClickPos().X, InViewportClick.GetClickPos().Y);
	}
}

