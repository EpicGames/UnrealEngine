// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportMoveYawInteraction.h"

#include "EditorModes.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "EditorViewportClient.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "ViewportClientNavigationHelper.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportMoveYawInteraction::UViewportMoveYawInteraction()
{
	InteractionName = UE::Editor::ViewportInteractions::PerspectiveMoveYaw;
	Groups = {
		UE::Editor::ViewportInteractions::CameraDrag,
		UE::Editor::ViewportInteractions::CameraFly,
	};

	ViewportClickDragBehavior->SetBindings({ EKeys::LeftMouseButton });
}

bool UViewportMoveYawInteraction::CanBeActivated() const
{
	if (Super::CanBeActivated())
	{
		if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
		{
			return EditorViewportClient->IsPerspective() && !EditorViewportClient->IsCameraLocked();
		}
	}
	return false;
}

void UViewportMoveYawInteraction::OnDragStart(const FInputDeviceRay& InDragStartPos)
{
	UViewportDragInteraction::OnDragStart(InDragStartPos);

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(true);
	}
}

void UViewportMoveYawInteraction::OnDragEnd(const FInputDeviceRay& InDragEndPos)
{
	UViewportDragInteraction::OnDragEnd(InDragEndPos);

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(false);
	}
}

void UViewportMoveYawInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		FVector LocationDelta = FVector::ZeroVector;
		FRotator RotDelta = FRotator::ZeroRotator;

		const double YawRadians = FMath::DegreesToRadians(EditorViewportClient->GetViewRotation().Yaw);
		LocationDelta.X = -InMouseDeltaY * FMath::Cos(YawRadians);
		LocationDelta.Y = -InMouseDeltaY * FMath::Sin(YawRadians);

		const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
		RotDelta.Yaw = InMouseDeltaX * ViewportSettings->MouseSensitivty;

		EditorViewportClient->GetViewportNavigationHelper()->LocationDelta += LocationDelta;
		EditorViewportClient->GetViewportNavigationHelper()->RotationDelta += RotDelta;
	}
}

TArray<FEditorModeID> UViewportMoveYawInteraction::GetUnsupportedModes() const
{
	const TArray<FEditorModeID> UnsupportedModes = { FBuiltinEditorModes::EM_Landscape, FBuiltinEditorModes::EM_Foliage };
	return UnsupportedModes;
}
