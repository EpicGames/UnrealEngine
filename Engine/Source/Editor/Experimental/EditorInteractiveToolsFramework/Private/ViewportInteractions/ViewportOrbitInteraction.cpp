// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportOrbitInteraction.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "EditorViewportClient.h"
#include "ViewportClientNavigationHelper.h"
#include "Behaviors/ViewportClickDragBehavior.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportOrbitInteraction::UViewportOrbitInteraction()
{
	InteractionName = UE::Editor::ViewportInteractions::Orbit;
	Groups = { UE::Editor::ViewportInteractions::CameraDrag };
	
	using namespace UE::Editor::ViewportInteractions;
	
	NormalBindings = FButtonBindings({
		FButtonBinding(EKeys::LeftMouseButton).TriggersStart(),
    	FButtonBinding(EKeys::LeftAlt).RequiredToContinue(false)
	});
	
	CameraLockedBindings = FButtonBindings({
		FButtonBinding(EKeys::LeftMouseButton).TriggersStart(),
		FButtonBinding(EKeys::LeftAlt).Required(false)
	});
	
	ViewportClickDragBehavior->SetBindings(UViewportClickDragBehavior::FGetBindingsDelegate::CreateUObject(this, &UViewportOrbitInteraction::GetBindings));
}

bool UViewportOrbitInteraction::CanBeActivated() const
{
	if (Super::CanBeActivated())
	{
		if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
		{
			return EditorViewportClient->IsPerspective();
		}
	}
	return false;
}

void UViewportOrbitInteraction::OnDragStart(const FInputDeviceRay& InDragStartPos)
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(true);
	}

	UViewportDragInteraction::OnDragStart(InDragStartPos);
}

void UViewportOrbitInteraction::OnDragEnd(const FInputDeviceRay& InDragEndPos)
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(false);
	}

	UViewportDragInteraction::OnDragEnd(InDragEndPos);
}

void UViewportOrbitInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
		const float Direction = ViewportSettings->bInvertOrbitYAxis ? -1.0f : 1.0f;

		// Change the viewing angle
		FVector OrbitDelta = FVector::ZeroVector;
		OrbitDelta.X = InMouseDeltaX * ViewportSettings->MouseSensitivty;
		OrbitDelta.Y = -InMouseDeltaY * ViewportSettings->MouseSensitivty * Direction;
	
		EditorViewportClient->GetViewportNavigationHelper()->OrbitDelta += OrbitDelta;
	}
}

const UE::Editor::ViewportInteractions::FButtonBindings& UViewportOrbitInteraction::GetBindings() const
{
	if (FEditorViewportClient* Client = GetEditorViewportClient())
	{
		if (Client->IsCameraLocked())
		{
			return CameraLockedBindings;
		}
	}
	return NormalBindings;
}

