// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportOrthoPanInteraction.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "EditorViewportClient.h"
#include "ViewportClientNavigationHelper.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportOrthoPanInteraction::UViewportOrthoPanInteraction()
{
	using namespace UE::Editor::ViewportInteractions;

	InteractionName = OrthographicPan;
	Groups = { CameraDrag };

	ViewportClickDragBehavior->SetBindings({
		FButtonBinding(EKeys::RightMouseButton),
		FButtonBinding(EKeys::LeftShift).Required(false)
	});
}

bool UViewportOrthoPanInteraction::CanBeActivated() const
{
	if (Super::CanBeActivated())
	{
		if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
		{
			return EditorViewportClient->IsOrtho();
		}	
	}
	return false;
}

void UViewportOrthoPanInteraction::OnDragStart(const FInputDeviceRay& InDragStartPos)
{
	UViewportDragInteraction::OnDragStart(InDragStartPos);

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetMouseCursorOverride(EMouseCursor::Type::GrabHandClosed);
		BehaviorSource->SetIsMouseLooking(true);
	}
}

void UViewportOrthoPanInteraction::OnDragEnd(const FInputDeviceRay& InDragEndPos)
{
	UViewportDragInteraction::OnDragEnd(InDragEndPos);

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(false);
	}
}

void UViewportOrthoPanInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		// based off FEditorViewportClient::TranslateDelta

		const float UnitsPerPixel = EditorViewportClient->GetOrthoUnitsPerPixel(EditorViewportClient->Viewport);
		FVector LocationDelta(-InMouseDeltaX * UnitsPerPixel, InMouseDeltaY * UnitsPerPixel, 0.0f);

		switch (EditorViewportClient->GetViewportType())
		{
		case LVT_OrthoTop:
			LocationDelta = FVector(-InMouseDeltaY * UnitsPerPixel, InMouseDeltaX * UnitsPerPixel, 0.0f);
			break;
		case LVT_OrthoLeft:
			LocationDelta = FVector(-InMouseDeltaX * UnitsPerPixel, 0.0f, InMouseDeltaY * UnitsPerPixel);
			break;
		case LVT_OrthoBack:
			LocationDelta = FVector(0.0f, -InMouseDeltaX * UnitsPerPixel, InMouseDeltaY * UnitsPerPixel);
			break;
		case LVT_OrthoBottom:
			LocationDelta = FVector(-InMouseDeltaY * UnitsPerPixel, -InMouseDeltaX * UnitsPerPixel, 0.0f);
			break;
		case LVT_OrthoRight:
			LocationDelta = FVector(InMouseDeltaX * UnitsPerPixel, 0.0f, InMouseDeltaY * UnitsPerPixel);
			break;
		case LVT_OrthoFront:
			LocationDelta = FVector(0.0f, InMouseDeltaX * UnitsPerPixel, InMouseDeltaY * UnitsPerPixel);
			break;
		case LVT_OrthoFreelook:
		case LVT_Perspective:
			break;
		}

		// Invert when Alt or Shift are down
		if (IsShiftDown())
		{
			LocationDelta *= -1.0f;
		}

		FVector& ViewportLocationDelta = EditorViewportClient->GetViewportNavigationHelper()->LocationDelta;
		ViewportLocationDelta += LocationDelta;
	}
}
