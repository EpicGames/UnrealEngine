// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportPanInteraction.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "Behaviors/ViewportClickDragBehavior.h"
#include "EditorViewportClient.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "ViewportClientNavigationHelper.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "Framework/Application/SlateApplication.h"

UViewportPanInteraction::UViewportPanInteraction()
{
	InteractionName = UE::Editor::ViewportInteractions::PerspectivePan;
	Groups = {
		UE::Editor::ViewportInteractions::CameraDrag,
		UE::Editor::ViewportInteractions::CameraFly,
	};

	using namespace UE::Editor::ViewportInteractions;
	ViewportClickDragBehavior->SetBindings({
		FButtonBinding(EKeys::MiddleMouseButton),
		FButtonBinding(EKeys::LeftAlt).Required(false)
	});
}

bool UViewportPanInteraction::CanBeActivated() const
{
	if (Super::CanBeActivated())
	{
		if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
		{
			return EditorViewportClient->IsPerspective();
		}
	}
	return false;
}

void UViewportPanInteraction::OnDragStart(const FInputDeviceRay& InDragStartPos)
{
	UViewportDragInteraction::OnDragStart(InDragStartPos);

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(true);
	}
}

void UViewportPanInteraction::OnDragEnd(const FInputDeviceRay& InDragEndPos)
{
	UViewportDragInteraction::OnDragEnd(InDragEndPos);

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(false);
	}
}

void UViewportPanInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		// TODO: In legacy, invert would also occur when using the trackpad - possibly need to handle this in a different way with ITF?
		const ULevelEditorViewportSettings* Settings = GetDefault<ULevelEditorViewportSettings>();
		PanCamera(*EditorViewportClient,
			InMouseDeltaX, InMouseDeltaY,
			Settings->bInvertMiddleMousePan,
			// Alt should always make the pan relative to the camera
			!IsAltDown() && Settings->bWorldSpaceVerticalPan
		);
	}
}
	
void UViewportPanInteraction::PanCamera(FEditorViewportClient& Client, float MouseDeltaX, float MouseDeltaY, bool bInvert, bool bUseWorldSpaceUp)
{
	FVector LocationDelta;

	const FRotator& ViewRotation = Client.GetViewRotation();
	if (bUseWorldSpaceUp)
	{
		const float Direction = bInvert ? 1.0f : -1.0f;
		const double YawRadians = FMath::DegreesToRadians(ViewRotation.Yaw);
		LocationDelta.X = MouseDeltaX * Direction * FMath::Sin(YawRadians);
		LocationDelta.Y = MouseDeltaX * -Direction * FMath::Cos(YawRadians);
		LocationDelta.Z = Direction * MouseDeltaY;
	}
	else
	{
		LocationDelta = FQuat(ViewRotation) * FVector( 0.0f, MouseDeltaX, -MouseDeltaY) * (bInvert ? -1.0f : 1.0f);
	}

	FVector& ViewportLocationDelta = Client.GetViewportNavigationHelper()->LocationDelta;
	ViewportLocationDelta += LocationDelta;
}

UViewportMovePanInteraction::UViewportMovePanInteraction()
{
	InteractionName = TEXT("Perspective Move/Pan");
	
	ViewportClickDragBehavior->SetBindings({
		EKeys::LeftMouseButton,
		EKeys::RightMouseButton
	});
}

void UViewportMovePanInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		PanCamera(*EditorViewportClient, InMouseDeltaX, InMouseDeltaY, /* Invert */ false, /* World Space up */ true);
	}
}

UViewportTrackpadPanInteraction::UViewportTrackpadPanInteraction()
{
	InteractionName = TEXT("Perspective Trackpad Pan");
	
	ViewportClickDragBehavior->SetBindings({ EKeys::RightMouseButton });
}

bool UViewportTrackpadPanInteraction::CanBeActivated() const
{
	return IsEnabled() && FSlateApplication::Get().IsUsingTrackpad();	
}
