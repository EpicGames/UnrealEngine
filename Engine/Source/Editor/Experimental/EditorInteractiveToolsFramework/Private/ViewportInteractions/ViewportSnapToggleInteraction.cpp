// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportSnapToggleInteraction.h"

#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "EditorViewportCommands.h"
#include "HAL/IConsoleManager.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "UnrealEdGlobals.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

namespace UE::ViewportSnapToggle
{
	TAutoConsoleVariable<bool> CVarTemporarySnapDisableAvailable(
		TEXT("Editor.Gizmo.TemporarySnapDisableAvailable"),
		false,
		TEXT("When enabled, temporary snapping held keys can also temporarily disable snapping when it is currently enabled.")
	);
} // namespace UE::ViewportSnapToggle

UViewportSnapToggleInteraction::UViewportSnapToggleInteraction()
{
	const FName UniqueName = MakeUniqueObjectName(GetTransientPackageAsObject(), UViewportSnapToggleInteraction::StaticClass(), *FString::Printf(TEXT("%s_Behavior"), *GetClass()->GetName()));
	UViewportQuickToggleInputBehavior* QuickToggleBehavior = NewObject<UViewportQuickToggleInputBehavior>(GetTransientPackageAsObject(), UniqueName);

	QuickToggleBehavior->bRequireAllKeys = false;
	QuickToggleBehavior->Initialize(this, GetKeys());
	QuickToggleBehavior->SetDefaultPriority(UE::Editor::ViewportInteractions::VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY);

	QuickToggleBehavior->ModifierCheckFunc = [](const FInputDeviceState& InputDeviceState) -> bool
	{
		return !InputDeviceState.bCtrlKeyDown && !InputDeviceState.bShiftKeyDown && !InputDeviceState.bAltKeyDown && !InputDeviceState.bCmdKeyDown;
	};

	QuickToggleBehaviorWeak = QuickToggleBehavior;

	RegisterInputBehavior(QuickToggleBehavior);
}

void UViewportSnapToggleInteraction::Shutdown()
{
	if (bTRSKeyHeld)
	{
		SetTRSSnapping(bCachedTRSSnapping);
		bTRSKeyHeld = false;
	}

	if (bSurfaceKeyHeld)
	{
		SetSurfaceSnapping(bCachedSurfaceSnapping);
		bSurfaceKeyHeld = false;
	}

	Super::Shutdown();
}

void UViewportSnapToggleInteraction::OnQuickToggle(const FKey& InKey)
{
	if (!FEditorViewportCommands::IsRegistered())
	{
		return;
	}

	using namespace UE::Editor::ViewportInteractions;
	const FEditorViewportCommands& ViewportCommands = FEditorViewportCommands::Get();

	if (CommandMatchesKey(ViewportCommands.CurrentTRSSnapToggle, InKey))
	{
		// bTRSKeyHeld is false: this means we are quick toggling ON --> OFF
		// while temporary disable is not available
		if (!bTRSKeyHeld)
		{
			SetTRSSnapping(false);
		}

		bTRSKeyHeld = false;
	}
	else if (CommandMatchesKey(ViewportCommands.SurfaceSnapping, InKey))
	{
		// bSurfaceKeyHeld is false: this means we are quick toggling ON --> OFF
		// while temporary disable is not available
		if (!bSurfaceKeyHeld)
		{
			SetSurfaceSnapping(false);
		}
		bSurfaceKeyHeld = false;
	}
}

void UViewportSnapToggleInteraction::OnKeyPressed(const FKey& InKeyID)
{
	// CVarTemporarySnapDisableAvailable: 0 (default)
	// - OFF-->ON : we can set snapping ON as soon as key is pressed. This works both for Quick Toggle and Temp Snapping
	// - ON -->OFF: we cannot know if this is going to be a quick toggle or a Temp Snapping. Skip setting snapping, and do that on Quick Toggle if needed.

	// CVarTemporarySnapDisableAvailable: 1
	// - We can always set snapping right away. Snapping will not be set on Quick Toggle.

	using namespace UE::Editor::ViewportInteractions;

	if (!FEditorViewportCommands::IsRegistered())
	{
		return;
	}

	const FEditorViewportCommands& ViewportCommands = FEditorViewportCommands::Get();

	if (!bTRSKeyHeld && CommandMatchesKey(ViewportCommands.CurrentTRSSnapToggle, InKeyID))
	{
		CacheTRSSnapping();

		// When temporary disable is not available, do not set snapping on key pressed
		// ON --> OFF should only work for Quick Toggle. Leave bTRSKeyHeld set to false.
		if (bCachedTRSSnapping && !CanTemporaryDisableSnapping())
		{
			return;
		}

		SetTRSSnapping(!bCachedTRSSnapping);
		bTRSKeyHeld = true;
	}
	else if (!bSurfaceKeyHeld && CommandMatchesKey(ViewportCommands.SurfaceSnapping, InKeyID))
	{
		CacheSurfaceSnapping();

		if (bCachedSurfaceSnapping && !CanTemporaryDisableSnapping())
		{
			return;
		}

		SetSurfaceSnapping(!bCachedSurfaceSnapping);
		bSurfaceKeyHeld = true;
	}
}

void UViewportSnapToggleInteraction::OnKeyReleased(const FKey& InKeyID)
{
	using namespace UE::Editor::ViewportInteractions;

	if (!FEditorViewportCommands::IsRegistered())
	{
		return;
	}

	const FEditorViewportCommands& ViewportCommands = FEditorViewportCommands::Get();

	// We check key(s) hold state, since OnQuickToggle could have already set the flag to false.
	// In that case, do nothing.

	if (bTRSKeyHeld && CommandMatchesKey(ViewportCommands.CurrentTRSSnapToggle, InKeyID))
	{
		SetTRSSnapping(bCachedTRSSnapping);
		bTRSKeyHeld = false;
	}
	else if (bSurfaceKeyHeld && CommandMatchesKey(ViewportCommands.SurfaceSnapping, InKeyID))
	{
		SetSurfaceSnapping(bCachedSurfaceSnapping);
		bSurfaceKeyHeld = false;
	}
}

void UViewportSnapToggleInteraction::OnForceEndCapture()
{
	if (bTRSKeyHeld)
	{
		SetTRSSnapping(bCachedTRSSnapping);
		bTRSKeyHeld = false;
	}

	if (bSurfaceKeyHeld)
	{
		SetSurfaceSnapping(bCachedSurfaceSnapping);
		bSurfaceKeyHeld = false;
	}
}

void UViewportSnapToggleInteraction::OnCommandChordChanged()
{
	if (UViewportQuickToggleInputBehavior* QuickToggleBehavior = QuickToggleBehaviorWeak.Get())
	{
		QuickToggleBehavior->Initialize(this, GetKeys());
	}
}

TArray<TSharedPtr<FUICommandInfo>> UViewportSnapToggleInteraction::GetCommands() const
{
	if (FEditorViewportCommands::IsRegistered())
	{
		return { FEditorViewportCommands::Get().CurrentTRSSnapToggle, FEditorViewportCommands::Get().SurfaceSnapping };
	}

	return {};
}

TArray<FKey> UViewportSnapToggleInteraction::GetKeys() const
{
	TArray<FKey> Keys;

	for (int32 ChordIndex = 0; ChordIndex < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++ChordIndex)
	{
		const EMultipleKeyBindingIndex BindingIndex = static_cast<EMultipleKeyBindingIndex>(ChordIndex);

		for (const TSharedPtr<FUICommandInfo>& Command : GetCommands())
		{
			Keys.Add(Command->GetActiveChord(BindingIndex)->Key);
		}
	}

	return Keys;
}

void UViewportSnapToggleInteraction::SetTRSSnapping(bool bEnable) const
{
	using namespace UE::Widget;

	if (const ULevelEditorViewportSettings* Settings = GetDefault<ULevelEditorViewportSettings>())
	{
		switch (GetWidgetMode())
		{
		case WM_Translate:
		case WM_2D:
			if (Settings->GridEnabled != bEnable)
			{
				GUnrealEd->Exec(GEditor->GetEditorWorldContext().World(), *FString::Printf(TEXT("MODE GRID=%d"), bEnable));
			}
			break;

		case WM_Rotate:
			if (Settings->RotGridEnabled != bEnable)
			{
				GUnrealEd->Exec(GEditor->GetEditorWorldContext().World(), *FString::Printf(TEXT("MODE ROTGRID=%d"), bEnable));
			}
			break;

		case WM_Scale:
			if (Settings->SnapScaleEnabled != bEnable)
			{
				GUnrealEd->Exec(GEditor->GetEditorWorldContext().World(), *FString::Printf(TEXT("MODE SCALEGRID=%d"), bEnable));
			}
			break;

		default:;
		}
	}
}

void UViewportSnapToggleInteraction::SetSurfaceSnapping(bool bEnable)
{
	ULevelEditorViewportSettings* Settings = GetMutableDefault<ULevelEditorViewportSettings>();

	if (Settings->SnapToSurface.bEnabled != bEnable)
	{
		Settings->SnapToSurface.bEnabled = bEnable;
	}
}

bool UViewportSnapToggleInteraction::CanTemporaryDisableSnapping()
{
	return UE::ViewportSnapToggle::CVarTemporarySnapDisableAvailable.GetValueOnAnyThread();
}

UE::Widget::EWidgetMode UViewportSnapToggleInteraction::GetWidgetMode() const
{
	if (const FEditorViewportClient* Client = GetEditorViewportClient())
	{
		return Client->GetWidgetMode();
	}
	return UE::Widget::WM_None;
}

void UViewportSnapToggleInteraction::CacheTRSSnapping()
{
	const ULevelEditorViewportSettings* LevelViewportSettings = GetDefault<ULevelEditorViewportSettings>();

	switch (GetWidgetMode())
	{
	case UE::Widget::EWidgetMode::WM_Rotate:
		bCachedTRSSnapping = LevelViewportSettings->RotGridEnabled;
		break;
	case UE::Widget::EWidgetMode::WM_Scale:
		bCachedTRSSnapping = LevelViewportSettings->SnapScaleEnabled;
		break;
	default:
		bCachedTRSSnapping = LevelViewportSettings->GridEnabled;
		break;
	}
}

void UViewportSnapToggleInteraction::CacheSurfaceSnapping()
{
	bCachedSurfaceSnapping = GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.bEnabled;
}

