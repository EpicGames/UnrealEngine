// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportViewAngleInteraction.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "EditorViewportClient.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "ViewportClientNavigationHelper.h"
#include "Framework/Application/SlateApplication.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportViewAngleInteraction::UViewportViewAngleInteraction()
{
	InteractionName = UE::Editor::ViewportInteractions::PerspectiveViewAngle;
	Groups = {
		UE::Editor::ViewportInteractions::CameraDrag,
		UE::Editor::ViewportInteractions::CameraFly
	};
	
	ViewportClickDragBehavior->SetBindings({
		EKeys::RightMouseButton
	});
}

bool UViewportViewAngleInteraction::CanBeActivated() const
{
	// TODO: Trackpad view angle adjustment is still driven by `FEditorViewportClient::InputGesture()`
	// Trackpad right click & drag is intended to be a pan and is handled by `UViewportTrackpadPanInteraction`
	if (Super::CanBeActivated() && !FSlateApplication::Get().IsUsingTrackpad())
	{
		if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
		{
			return EditorViewportClient->IsPerspective() && !EditorViewportClient->IsCameraLocked();
		}
	}
	return false;
}

void UViewportViewAngleInteraction::OnDragStart(const FInputDeviceRay& InDragStartPos)
{
	UViewportDragInteraction::OnDragStart(InDragStartPos);

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(true);
	}
}

void UViewportViewAngleInteraction::OnDragEnd(const FInputDeviceRay& InDragEndPos)
{
	UViewportDragInteraction::OnDragEnd(InDragEndPos);

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetIsMouseLooking(false);
	}
}

void UViewportViewAngleInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		// Change viewing angle

		// Inverting orbit axis is handled elsewhere
		const bool bInvertY = !EditorViewportClient->ShouldOrbitCamera()
						   && GetDefault<ULevelEditorViewportSettings>()->bInvertMouseLookYAxis;
		float Direction = bInvertY ? -1.0f : 1.0f;

		const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

		FRotator RotDelta;
		RotDelta.Yaw = InMouseDeltaX * ViewportSettings->MouseSensitivty;
		RotDelta.Pitch = -InMouseDeltaY * ViewportSettings->MouseSensitivty * Direction;
		RotDelta.Roll = 0.0f;

		FRotator& ViewportRotationDelta = EditorViewportClient->GetViewportNavigationHelper()->RotationDelta;
		ViewportRotationDelta += RotDelta;
	}
}
