// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportZoomInteraction.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/KeyInputBehavior.h"
#include "Behaviors/ViewportClickDragBehavior.h"
#include "EditorViewportClient.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Templates/Function.h"
#include "ViewportClientNavigationHelper.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportZoomInteraction::UViewportZoomInteraction()
{
	using namespace UE::Editor::ViewportInteractions;

	InteractionName = Zoom;
	Groups = {
		CameraDrag,
		CameraFly
	};

	// Mousewheel zoom
	UMouseWheelInputBehavior* MouseWheelInputBehavior = NewObject<UMouseWheelInputBehavior>();
	MouseWheelInputBehavior->Initialize(this);
	MouseWheelInputBehavior->SetDefaultPriority(VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY);
	MouseWheelInputBehavior->ModifierCheckFunc = [](const FInputDeviceState& InputDeviceState)
	{
		if (FInputDeviceState::IsAltKeyDown(InputDeviceState) || FInputDeviceState::IsCtrlKeyDown(InputDeviceState)
			|| FInputDeviceState::IsShiftKeyDown(InputDeviceState) || FInputDeviceState::IsCmdKeyDown(InputDeviceState))
		{
			return false;
		}

		return true;
	};

	// Keyboard zoom
	UKeyInputBehavior* KeyInputBehavior = NewObject<UKeyInputBehavior>();
	// Hardcoded
	KeyInputBehavior->Initialize(this, { ZoomIn, ZoomOut });
	KeyInputBehavior->bRequireAllKeys = false;
	KeyInputBehavior->SetDefaultPriority(VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY);

	// Use two different behaviors to manage the two orthographic zoom bindings.
	TArray<ULocalViewportClickDragBehavior*, TInlineAllocator<2>> OrthoZoomBehaviors;
	
	ULocalViewportClickDragBehavior* LmbAndRmbBehavior = NewObject<ULocalViewportClickDragBehavior>();
	LmbAndRmbBehavior->Initialize();
	LmbAndRmbBehavior->SetBindings({
		// The legacy binding
		EKeys::LeftMouseButton,
		EKeys::RightMouseButton
	});
	OrthoZoomBehaviors.Add(LmbAndRmbBehavior);
	
	ULocalViewportClickDragBehavior* AltRmbBehavior = NewObject<ULocalViewportClickDragBehavior>();
	AltRmbBehavior->Initialize();
	AltRmbBehavior->SetBindings({
		// This matches the binding in `UViewportDollyInteraction`
		FButtonBinding(EKeys::RightMouseButton).TriggersStart(),
		FButtonBinding(EKeys::LeftAlt).RequiredToContinue(false)
	});
	OrthoZoomBehaviors.Add(AltRmbBehavior);

	for (ULocalViewportClickDragBehavior* Behavior : OrthoZoomBehaviors)
	{
		Behavior->CanBeginClickDragFunc = [this](const FInputDeviceRay&)
		{
			bool bSuccess = IsEnabled();
			if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
			{
				bSuccess &= EditorViewportClient->IsOrtho();
			}

			return bSuccess ? FInputRayHit(TNumericLimits<float>::Max()) : FInputRayHit();
		};

		Behavior->OnDragStartFunc = [this](const FInputDeviceRay& InDragStartPos)
		{
			OnDragStart();
		};

		Behavior->OnDragEndFunc = [this](const FInputDeviceRay& InDragEndPos)
		{
			OnDragEnd();
		};

		Behavior->OnDragFunc = [this](const IViewportClickDragBehaviorTarget::FDragArgs& InDrag)
		{
			OnClickDrag(InDrag.ScreenDelta);
		};
		
		Behavior->OnEndCaptureFunc = [this](const IViewportClickDragBehaviorTarget::EEndCaptureReason InReason)
		{
			OnForceEndCapture();
		};
	}

	RegisterInputBehaviors({
		MouseWheelInputBehavior,
		KeyInputBehavior,
		LmbAndRmbBehavior,
		AltRmbBehavior
	});
}

FInputRayHit UViewportZoomInteraction::ShouldRespondToMouseWheel(const FInputDeviceRay& InCurrentPos)
{
	return IsEnabled() && !IsAnyMouseButtonDown() ? FInputRayHit(TNumericLimits<float>::Max()) : FInputRayHit();
}

void UViewportZoomInteraction::OnMouseWheelScrollUp(const FInputDeviceRay& InCurrentPos)
{
	if (!IsAnyMouseButtonDown())
	{
		constexpr bool bZoomIn = false;
		MouseWheelZoom(bZoomIn);
	}
}

void UViewportZoomInteraction::OnMouseWheelScrollDown(const FInputDeviceRay& InCurrentPos)
{
	if (!IsAnyMouseButtonDown())
	{
		constexpr bool bZoomIn = true;
		MouseWheelZoom(bZoomIn);
	}
}

void UViewportZoomInteraction::OnKeyPressed(const FKey& InKeyID)
{
	if (InKeyID == ZoomIn)
	{
		ZoomDirection = -1;
	}
	else if (InKeyID == ZoomOut)
	{
		ZoomDirection = 1;
	}
}

void UViewportZoomInteraction::OnKeyReleased(const FKey& InKeyID)
{
	if (InKeyID == EKeys::Add || InKeyID == EKeys::Subtract)
	{
		ZoomDirection = 0;
	}
}

void UViewportZoomInteraction::OnForceEndCapture()
{
	ZoomDirection = 0;
	OnDragEnd();
}

void UViewportZoomInteraction::Tick(float InDeltaTime) const
{
	if (FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
	{
		if (ZoomDirection != 0)
		{
			const bool bZoomIn = ZoomDirection > 0;
			if (EditorViewportClient->IsPerspective())
			{
				PerspectiveZoom(bZoomIn);
			}
			else
			{
				// Slower zoom speed, and always centered when done in orthographic view
				OrthographicZoom(bZoomIn, 0.25f, true);
			}
		}
	}

	UViewportInteraction::Tick(InDeltaTime);
}

void UViewportZoomInteraction::OrthographicZoom(bool bInZoomIn, float InZoomMultiplier, bool bInForceZoomToCenter) const
{
	FEditorViewportClient* EditorViewportClient = GetEditorViewportClient();
	if (!EditorViewportClient)
	{
		return;
	}

	if (const FViewport* const Viewport = EditorViewportClient->Viewport)
	{
		constexpr float Scale = 1.0f;

		// Scrolling the mousewheel up/down zooms the orthogonal viewport in/out.
		float NewOrthoRatio = 1.0f + ((25.0f * Scale) / CAMERA_ZOOM_DAMPEN) * InZoomMultiplier;
		if (!bInZoomIn)
		{
			NewOrthoRatio = 1.0f / NewOrthoRatio;
		}

		// Extract current state
		int32 ViewportWidth = Viewport->GetSizeXY().X;
		int32 ViewportHeight = Viewport->GetSizeXY().Y;

		FVector OldOffsetFromCenter;

		const bool bCenterZoomAroundCursor = GetDefault<ULevelEditorViewportSettings>()->bCenterZoomAroundCursor;

		if (bCenterZoomAroundCursor && !bInForceZoomToCenter)
		{
			// Y is actually backwards, but since we're move the camera opposite the cursor to center, we negate both
			// therefore the x is negated
			// X Is backwards, negate it
			// default to viewport mouse position
			int32 CenterX = Viewport->GetMouseX();
			int32 CenterY = Viewport->GetMouseY();

			//TODO: check this one out
			/*if (EditorViewportClient->ShouldUseMoveCanvasMovement())
			{
				// use virtual mouse while dragging (normal mouse is clamped when invisible)
				CenterX = EditorViewportClient->LastMouseX;
				CenterY = EditorViewportClient->LastMouseY;
			}*/
			int32 DeltaFromCenterX = -(CenterX - (ViewportWidth >> 1));
			int32 DeltaFromCenterY = (CenterY - (ViewportHeight >> 1));

			switch (EditorViewportClient->GetViewportType())
			{
			case LVT_OrthoTop:
				OldOffsetFromCenter.Set(-DeltaFromCenterY, -DeltaFromCenterX, 0.0f);
				break;
			case LVT_OrthoLeft:
				OldOffsetFromCenter.Set(DeltaFromCenterX, 0.0f, DeltaFromCenterY);
				break;
			case LVT_OrthoBack:
				OldOffsetFromCenter.Set(0.0f, DeltaFromCenterX, DeltaFromCenterY);
				break;
			case LVT_OrthoBottom:
				OldOffsetFromCenter.Set(-DeltaFromCenterY, DeltaFromCenterX, 0.0f);
				break;
			case LVT_OrthoRight:
				OldOffsetFromCenter.Set(-DeltaFromCenterX, 0.0f, DeltaFromCenterY);
				break;
			case LVT_OrthoFront:
				OldOffsetFromCenter.Set(0.0f, -DeltaFromCenterX, DeltaFromCenterY);
				break;
			case LVT_OrthoFreelook:
				//@TODO: CAMERA: How to handle this (todo copied from legacy viewport inputs)
				break;
			case LVT_Perspective:
				break;
			}
		}

		// Save off old zoom
		const float OldUnitsPerPixel = EditorViewportClient->GetOrthoUnitsPerPixel(Viewport);

		// Update zoom based on input
		const float Zoom = EditorViewportClient->GetOrthoZoom() * NewOrthoRatio;

		const float MinOrthoZoom = static_cast<float>(
			FMath::Max(GetDefault<ULevelEditorViewportSettings>()->MinimumOrthographicZoom, MIN_ORTHOZOOM)
		);

		EditorViewportClient->SetOrthoZoom(static_cast<float>(FMath::Clamp(Zoom, MinOrthoZoom, MAX_ORTHOZOOM)));

		if (bCenterZoomAroundCursor)
		{
			// This is the equivalent to moving the viewport to center about the cursor, zooming, and moving it back a proportional amount towards the cursor
			const FVector FinalDelta = (EditorViewportClient->GetOrthoUnitsPerPixel(Viewport) - OldUnitsPerPixel)
									 * OldOffsetFromCenter;

			// Now move the view location proportionally
			EditorViewportClient->SetViewLocation(EditorViewportClient->GetViewLocation() + FinalDelta);
		}

		constexpr bool bInvalidateViews = true;

		// Update linked ortho viewport movement based on updated zoom and view location,
		EditorViewportClient->UpdateLinkedOrthoViewports(bInvalidateViews);
		EditorViewportClient->Invalidate(true, true);

		UE_LOGF(LogITFViewportInteractions, Verbose, "ITF CAMERA ORTHO ZOOM");
	}
}

void UViewportZoomInteraction::PerspectiveZoom(bool bInZoomIn) const
{
	FEditorViewportClient* EditorViewportClient = GetEditorViewportClient();
	if (!EditorViewportClient)
	{
		return;
	}

	// Scrolling the mousewheel up/down moves the perspective viewport forwards/backwards.
	FVector Drag;

	const FRotator& ViewRotation = EditorViewportClient->GetViewRotation();
	Drag.X = FMath::Cos(ViewRotation.Yaw * PI / 180.0f) * FMath::Cos(ViewRotation.Pitch * PI / 180.0f);
	Drag.Y = FMath::Sin(ViewRotation.Yaw * PI / 180.0f) * FMath::Cos(ViewRotation.Pitch * PI / 180.0f);
	Drag.Z = FMath::Sin(ViewRotation.Pitch * PI / 180.0f);

	if (bInZoomIn)
	{
		Drag = -Drag;
	}

	const float CameraSpeed = EditorViewportClient->GetCameraSpeedSettings().GetCurrentSpeed();
	Drag *= CameraSpeed * 32.0f;

	constexpr bool bDollyCamera = true;
	EditorViewportClient->MoveViewportCamera(Drag, FRotator::ZeroRotator, bDollyCamera);

	FEditorDelegates::OnDollyPerspectiveCamera.Broadcast(Drag, EditorViewportClient->ViewIndex);

	EditorViewportClient->Invalidate(true, true);

	UE_LOGF(LogITFViewportInteractions, Verbose, "ITF CAMERA PERSPECTIVE ZOOM");
}

void UViewportZoomInteraction::MouseWheelZoom(bool bInZoomIn) const
{
	if (const FEditorViewportClient* EditorViewportClient = GetEditorViewportClient())
	{
		if (EditorViewportClient->IsPerspective())
		{
			PerspectiveZoom(bInZoomIn);
		}
		else
		{
			OrthographicZoom(bInZoomIn);
		}
	}
}

void UViewportZoomInteraction::OnDragStart()
{
	UE_LOGF(LogITFViewportInteractions, Verbose, "ORTHO ZOOM LMB + RMB 1");

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetMouseCursorOverride(EMouseCursor::None);
		BehaviorSource->SetIsMouseLooking(true);
	}
}

void UViewportZoomInteraction::OnDragEnd()
{
	UE_LOGF(LogITFViewportInteractions, Verbose, "ORTHO ZOOM LMB + RMB 0");

	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->ClearMouseCursorOverride();
		BehaviorSource->SetIsMouseLooking(false);
	}
}

void UViewportZoomInteraction::OnClickDrag(const FVector2D& InDelta)
{
	if (FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient())
	{
		const ULevelEditorViewportSettings* Settings = GetDefault<ULevelEditorViewportSettings>(); 
		const float Direction = Settings->bInvertRightMouseDollyYAxis ? -1.0f : 1.0f;
		FVector LocationDelta(0.0f, 0.0f, -InDelta.Y - InDelta.X);
		EditorViewportClient->GetViewportNavigationHelper()->OrbitDelta += LocationDelta * Direction;
	}
}
