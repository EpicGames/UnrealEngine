// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "SCoordinateSystemMask.h"
 
#include "DetailLayoutBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyHandle.h"
#include "UnrealWidgetFwd.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
 
#define LOCTEXT_NAMESPACE "SCoordinateSystemMask"
 
namespace CoordinateSystemMaskLocals
{
    /** Gets the display name for a coordinate system. Copied from UnrealEdViewportToolbar.cpp. */
    FText GetCoordinateSystemDisplayName(const ECoordSystem InCoordinateSystem)
    {
        switch (InCoordinateSystem)
        {
        case COORD_World:
            return LOCTEXT("COORD_World", "World");
        case COORD_Local:
            return LOCTEXT("COORD_Local", "Local");
        case COORD_Parent:
            return LOCTEXT("COORD_Parent", "Parent");
        case COORD_Explicit:
            return LOCTEXT("COORD_Explicit", "Explicit");
        default:
            return FText::GetEmpty();
        }
    }
 
    /** Gets the tooltip text for a coordinate system. */
    FText GetCoordinateSystemTooltipText(const ECoordSystem InCoordinateSystem)
    {
        switch (InCoordinateSystem)
        {
        case COORD_World:
            return LOCTEXT("COORD_World_Tooltip", "Transform relative to the world origin.");
        case COORD_Local:
            return LOCTEXT("COORD_Local_Tooltip", "Transform relative to the selected object's local axes.");
        case COORD_Parent:
            return LOCTEXT("COORD_Parent_Tooltip", "Transform relative to the parent object's axes.");
        case COORD_Explicit:
            return LOCTEXT("COORD_Explicit_Tooltip", "Transform relative to an explicitly set coordinate frame.");
        default:
            return FText::GetEmpty();
        }
    }
 
    FSlateIcon GetIconFromCoordinateSystem(ECoordSystem InCoordinateSystem)
    {
        switch (InCoordinateSystem)
        {
        case COORD_World:
        {
            static FName WorldIcon("EditorViewport.RelativeCoordinateSystem_World");
            return FSlateIcon(FAppStyle::GetAppStyleSetName(), WorldIcon);
        }
  
        case COORD_Parent:
        {
            static const FName ParentIcon("EditorViewport.RelativeCoordinateSystem_Parent");
            return FSlateIcon(FAppStyle::GetAppStyleSetName(), ParentIcon);
        }
 
        case COORD_Explicit:
        {
            static const FName ExplicitIcon("EditorViewport.RelativeCoordinateSystem_Explicit");
            return FSlateIcon(FAppStyle::GetAppStyleSetName(), ExplicitIcon);
        }
 
        case COORD_Local:
        {
            static FName LocalIcon("EditorViewport.RelativeCoordinateSystem_Local");
            return FSlateIcon(FAppStyle::GetAppStyleSetName(), LocalIcon);
        }
            
        default:
            return FSlateIcon();
        }
    }
}
 
ENUM_RANGE_BY_COUNT(
    ECoordSystem,
    ECoordSystem::COORD_Max);
 
SLATE_IMPLEMENT_WIDGET(SCoordinateSystemBitmask)
void SCoordinateSystemBitmask::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
    SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ValueAttribute, EInvalidateWidgetReason::Paint);
}
 
SCoordinateSystemBitmask::SCoordinateSystemBitmask()
    : ValueAttribute(*this, INDEX_NONE)
{
}
 
void SCoordinateSystemBitmask::Construct(const FArguments& InArgs)
{
    ValueAttribute.Assign(*this, InArgs._Value);
    OnSetValue = InArgs._OnSetValue;
 
    auto CreateBitmaskFlagsArray = [ValuesToHide = InArgs._ValuesToHide]()
    {
        TArray<FCoordinateSystemFlagInfo> Flags;
        Flags.Empty(ECoordSystem::COORD_Max);
 
        auto AddFlag = [&Flags](ECoordSystem InCoordSystem, bool bIsEditable)
        {
            using namespace CoordinateSystemMaskLocals;
 
            FCoordinateSystemFlagInfo& Flag = Flags.AddDefaulted_GetRef();
            Flag.Value = (1 << static_cast<int32>(InCoordSystem));
            Flag.DisplayName = GetCoordinateSystemDisplayName(InCoordSystem);
            Flag.ToolTipText = GetCoordinateSystemTooltipText(InCoordSystem);
            Flag.Icon = GetIconFromCoordinateSystem(InCoordSystem);
            Flag.bIsEditable = bIsEditable;
        };
 
        for (ECoordSystem CoordSystemValue : TEnumRange<ECoordSystem>())
        {
            if (ValuesToHide.Contains(CoordSystemValue))
            {
                continue;
            }
 
            AddFlag(CoordSystemValue, true);
        }
 
        // @todo: only allow changing explicit, until EditorTRS -> EditorITF
        if (FCoordinateSystemFlagInfo* ParentFlag = Flags.FindByKey(COORD_Parent))
        {
            ParentFlag->bIsEditable = false;
        }
 
        return Flags;
    };
 
    const FComboBoxStyle& ComboBoxStyle = FCoreStyle::Get().GetWidgetStyle<FComboBoxStyle>("ComboBox");
 
    auto GetComboButtonText = [this, CreateBitmaskFlagsArray]() -> FText
    {
        TOptional<int32> Value = GetValue();
        if (Value.IsSet())
        {
            int32 BitmaskValue = Value.GetValue();
            if (BitmaskValue != 0)
            {
                TArray<FCoordinateSystemFlagInfo> BitmaskFlags = CreateBitmaskFlagsArray();
 
                TArray<FText> SetFlags;
                SetFlags.Reserve(BitmaskFlags.Num());
 
                for (const FCoordinateSystemFlagInfo& FlagInfo : BitmaskFlags)
                {
                    if ((Value.GetValue() & FlagInfo.Value) != 0)
                    {
                        SetFlags.Add(FlagInfo.DisplayName);
                    }
                }
 
                if (SetFlags.Num() > 3)
                {
                    SetFlags.SetNum(3);
                    SetFlags.Add(FText::FromString(TEXT("...")));
                }
 
                return FText::Join(FText::FromString(TEXT(" | ")), SetFlags);
            }
 
            return LOCTEXT("CoordSystemBitmaskNoFlagsSet", "(No Coordinate Systems)");
        }
        else
        {
            return LOCTEXT("CoordSystemBitmaskUndetermined", "Multiple Values");
        }
    };
 
    // Constructs the UI for bitmask property editing.
    SAssignNew(PrimaryWidget, SComboButton)
    .ComboButtonStyle(&ComboBoxStyle.ComboButtonStyle)
    .ToolTipText_Lambda([this, CreateBitmaskFlagsArray]()
    {
        TOptional<int32> Value = GetValue();
        if (Value.IsSet())
        {
            TArray<FCoordinateSystemFlagInfo> BitmaskFlags = CreateBitmaskFlagsArray();
 
            TArray<FText> SetFlags;
            SetFlags.Reserve(BitmaskFlags.Num());
 
            for (const FCoordinateSystemFlagInfo& FlagInfo : BitmaskFlags)
            {
                if ((Value.GetValue() & FlagInfo.Value) != 0)
                {
                    SetFlags.Add(FlagInfo.DisplayName);
                }
            }
 
            return FText::Join(FText::FromString(TEXT(" | ")), SetFlags);
        }
 
        return FText::GetEmpty();
    })
    .ButtonContent()
    [
        SNew(STextBlock)
        .Font(IDetailLayoutBuilder::GetDetailFont())
        .Text_Lambda(GetComboButtonText)
    ]
    .OnGetMenuContent_Lambda([this, CreateBitmaskFlagsArray]()
    {
        FMenuBuilder MenuBuilder(false, nullptr);
 
        TArray<FCoordinateSystemFlagInfo> BitmaskFlags = CreateBitmaskFlagsArray();
        for (int32 FlagIndex = 0; FlagIndex < BitmaskFlags.Num(); ++FlagIndex)
        {
            const FCoordinateSystemFlagInfo FlagInfo = BitmaskFlags[FlagIndex];
            MenuBuilder.AddMenuEntry(
                BitmaskFlags[FlagIndex].DisplayName,
                BitmaskFlags[FlagIndex].ToolTipText,
                BitmaskFlags[FlagIndex].Icon,
                FUIAction
                (
                    FExecuteAction::CreateLambda([this, FlagInfo]()
                    {
                        // Don't allow toggling non-editable flags
                        if (!FlagInfo.bIsEditable)
                        {
                            return;
                        }
 
                        TOptional<int32> Value = GetValue();
                        if (Value.IsSet())
                        {
                            SetValue(Value.GetValue() ^ FlagInfo.Value);
                        }
                    }),
                    FCanExecuteAction::CreateLambda([FlagInfo]()
                    {
                        return FlagInfo.bIsEditable;
                    }),
                    FIsActionChecked::CreateLambda([this, FlagInfo]() -> bool
                    {
                        TOptional<int32> Value = GetValue();
                        if (Value.IsSet()) 
                        {
                            return (Value.GetValue() & FlagInfo.Value) != 0;
                        }
                        return false;
                    })
                ),
                NAME_None,
                EUserInterfaceActionType::Check);
        }
 
        return MenuBuilder.MakeWidget();
    });
 
    ChildSlot
    [
        PrimaryWidget.ToSharedRef()
    ];
}
 
TOptional<int32> SCoordinateSystemBitmask::GetValue() const
{
    return ValueAttribute.Get();
}
 
void SCoordinateSystemBitmask::SetValue(int32 NewValue)
{
    if (NewValue != GetValue())
    {
        OnSetValue.ExecuteIfBound(NewValue);
    }
}
 
#undef LOCTEXT_NAMESPACE
