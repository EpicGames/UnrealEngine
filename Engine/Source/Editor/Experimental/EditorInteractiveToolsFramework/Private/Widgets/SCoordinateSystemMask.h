// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once
 
#include "Textures/SlateIcon.h"
#include "UnrealWidgetFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
 
class SComboButton;
 
template<>
struct TWidgetTypeTraits<class SCoordinateSystemBitmask>
{
    static constexpr bool SupportsInvalidation() { return true; }
};
 
/**
 * A widget that displays ECoordSystem enum values as bitflags.
 * @see: SPropertyEditorNumeric
 */
class SCoordinateSystemBitmask : public SCompoundWidget
{
    SLATE_DECLARE_WIDGET(SCoordinateSystemBitmask, SCompoundWidget)
 
private:
    /** Flag information for coordinate system bitmask UI. */
    struct FCoordinateSystemFlagInfo
    {
        int32 Value = INDEX_NONE;
        FText DisplayName;
        FText ToolTipText;
        FSlateIcon Icon;
        bool bIsEditable = true;
 
        /** Supports find by Key. */
        bool operator==(const ECoordSystem InCoordSystemValue) const
        {
            return Value == (1 << static_cast<int32>(InCoordSystemValue));
        }
    };
 
public:
    SLATE_BEGIN_ARGS(SCoordinateSystemBitmask)
        : _Value(INDEX_NONE)
    {}
    SLATE_ARGUMENT(TSet<ECoordSystem>, ValuesToHide)
    SLATE_ATTRIBUTE(TOptional<int32>, Value)
    SLATE_EVENT(TDelegate<void(int32)>, OnSetValue)
SLATE_END_ARGS()
 
SCoordinateSystemBitmask();
 
    void Construct(const FArguments& InArgs);
 
private:
    /** Gets the current bitmask value from the property. */
    TOptional<int32> GetValue() const;
 
    /** Commits a new bitmask value to the property. */
    void SetValue(int32 NewValue);
 
private:
    TSlateAttribute<TOptional<int32>> ValueAttribute;
    TDelegate<void(int32)> OnSetValue;
 
    TSharedPtr<SComboButton> PrimaryWidget;
};
