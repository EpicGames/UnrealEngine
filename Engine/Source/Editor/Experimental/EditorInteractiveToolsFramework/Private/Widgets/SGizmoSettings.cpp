// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SGizmoSettings.h"

#include "Customizations/GizmoDebugCustomization.h"
#include "Customizations/GizmoInteractionCustomization.h"
#include "Customizations/GizmoStyleCustomization.h"
#include "CVarToggle.h"
#include "DesktopPlatformModule.h"
#include "DetailsDisplayManager.h"
#include "DetailsViewArgs.h"
#include "DetailsViewObjectFilter.h"
#include "EditorDirectories.h"
#include "EditorInteractiveGizmoManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "IDesktopPlatform.h"
#include "IDetailPropertyRow.h"
#include "ISettingsModule.h"
#include "IStructureDetailsView.h"
#include "JsonObjectConverter.h"
#include "Misc/FileHelper.h"
#include "Misc/NotifyHook.h"
#include "Misc/TransactionObjectEvent.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styling/AppStyle.h"
#include "TransformGizmoEditorSettings.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

class UTransformGizmoEditorSettings;

namespace GizmoSettingsLocals
{
	// Allows versioning of settings, for migrating older settings files
	constexpr int32 CurrentSettingsVersion = 3;

	/** @return true if one or more fields removed. */
	template <typename JsonType>
	bool RemoveDefaultJson(const JsonType& InDefaultJson, const JsonType& InOutJson);

	template <>
	bool RemoveDefaultJson(const TSharedPtr<FJsonValue>& InDefaultJson, const TSharedPtr<FJsonValue>& InOutJson);

	template <>
	bool RemoveDefaultJson(const TArray<TSharedPtr<FJsonValue>>& InDefaultJson, const TArray<TSharedPtr<FJsonValue>>& InOutJson);

	template <>
	bool RemoveDefaultJson(const TSharedPtr<FJsonObject>& InDefaultJson, const TSharedPtr<FJsonObject>& InOutJson);

	template <>
	bool RemoveDefaultJson(const TSharedPtr<FJsonValue>& InDefaultJson, const TSharedPtr<FJsonValue>& InOutJson)
	{
		bool bAnyFieldsRemoved = false;
		switch (InDefaultJson->Type)
		{
		case EJson::Array:
			bAnyFieldsRemoved |= RemoveDefaultJson(InDefaultJson->AsArray(), InOutJson->AsArray());
			break;

		case EJson::Object:
			bAnyFieldsRemoved |= RemoveDefaultJson(InDefaultJson->AsObject(), InOutJson->AsObject());
			break;
						
		case EJson::None:
		case EJson::Null:
		case EJson::String:
		case EJson::Number:
		case EJson::Boolean:
			if (!FJsonValue::CompareEqual(*InOutJson, *InDefaultJson))
			{
				bAnyFieldsRemoved = true;
			}
			break;

		default:
			break;
		}

		return bAnyFieldsRemoved;
	}

	template <>
	bool RemoveDefaultJson(const TArray<TSharedPtr<FJsonValue>>& InDefaultJson, const TArray<TSharedPtr<FJsonValue>>& InOutJson)
	{
		bool bAnyFieldsRemoved = false;
		if (InDefaultJson.Num()	!= InOutJson.Num())
		{
			return false;
		}

		for (int32 ValueIndex = 0; ValueIndex < InDefaultJson.Num(); ++ValueIndex)
		{
			const TSharedPtr<FJsonValue>& DefaultJsonFieldValue = InDefaultJson[ValueIndex];
			const TSharedPtr<FJsonValue>& JsonFieldValue = InOutJson[ValueIndex];
			bAnyFieldsRemoved |= RemoveDefaultJson(DefaultJsonFieldValue, JsonFieldValue);

			// We don't do anything here intentionally - if we null it out, it won't be merged correctly on load
		}

		return bAnyFieldsRemoved;
	}

	template <>
	bool RemoveDefaultJson(const TSharedPtr<FJsonObject>& InDefaultJson, const TSharedPtr<FJsonObject>& InOutJson)
	{
		bool bAnyFieldsRemoved = false;
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonFieldPair : InDefaultJson->Values)
		{
			const FJsonObject::FStringType& JsonFieldName = JsonFieldPair.Key;
			const TSharedPtr<FJsonValue>& DefaultJsonFieldValue = JsonFieldPair.Value;
			if (TSharedPtr<FJsonValue> JsonFieldValue = InOutJson->TryGetField(JsonFieldName))
			{
				if (RemoveDefaultJson(DefaultJsonFieldValue, JsonFieldValue))
				{
					if (JsonFieldValue->Type != EJson::Object && JsonFieldValue->Type != EJson::Array)
					{
						InOutJson->RemoveField(JsonFieldName);
					}

					bAnyFieldsRemoved = true;	
				}
			}
		}

		return bAnyFieldsRemoved;
	}
}

#define LOCTEXT_NAMESPACE "SGizmoSettings"

class FGizmoSettingsDisplayManager : public FDetailsDisplayManager
{
public:
	virtual TOptional<bool> OverrideCreateCategoryNodes() const override
	{
		return true;
	}
};

class FGizmoSettingsObjectFilter : public FDetailsViewObjectFilter
{
public:
	FGizmoSettingsObjectFilter()
	{
		DisplayManager = MakeShared<FGizmoSettingsDisplayManager>();
	}

	virtual TArray<FDetailsViewObjectRoot> FilterObjects(const TArray<UObject*>& SourceObjects) override
	{
		TArray<FDetailsViewObjectRoot> Roots;
		Roots.Emplace(SourceObjects);

		return Roots;
	}
};

SGizmoSettings::~SGizmoSettings()
{
	if (StyleDetailsView.IsValid())
	{
		StyleDetailsView->OnFinishedChangingProperties().RemoveAll(this);
		StyleDetailsView.Reset();
	}

	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
}

void SGizmoSettings::Construct(const FArguments& InArgs)
{
	CVarDebugDraw = MakeShared<TCVarToggle<bool>>(TEXT("Gizmos.DebugDraw"));

	auto MakeCVarToggleButton = [](const TSharedPtr<TCVarToggle<bool>>& InCVarToggle, const FString& InLabel)
	{
		return SNew(SCheckBox)
		.Type(ESlateCheckBoxType::Type::ToggleButton)
		.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
		.IsChecked_Lambda([WeakCVarToggle = InCVarToggle.ToWeakPtr()]()
		{
			if (const TSharedPtr<TCVarToggle<bool>> CVarToggle = WeakCVarToggle.Pin())
			{
				return CVarToggle->GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			return ECheckBoxState::Unchecked;
		})
		.OnCheckStateChanged_Lambda([WeakCVarToggle = InCVarToggle.ToWeakPtr()](const ECheckBoxState& InCheckState)
		{
			if (const TSharedPtr<TCVarToggle<bool>> CVarToggle = WeakCVarToggle.Pin())
			{
				return CVarToggle->SetValue(InCheckState == ECheckBoxState::Checked);
			}
		})
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(InLabel))	
			]
		];
	};

	TabSwitcher = SNew(SWidgetSwitcher)
	+ SWidgetSwitcher::Slot()
	[
		MakeStyleDetailsView()
	]

	+ SWidgetSwitcher::Slot()
	[
		MakeInteractionDetailsView()
	]

	+ SWidgetSwitcher::Slot()
	[
		MakeDebugDetailsView()
	];

	// Toggle View — sets both bUseExperimentalGizmo and bUseEditorTRSGizmo together
	// so the UI only offers "New" or "Legacy" (the intermediate state is CVar-only).
	TSharedRef<SWidget> TogglePropertyViewWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([]()
			{
				const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
				return Settings->bUseEditorTRSGizmo ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([](ECheckBoxState InCheckState)
			{
				const bool bEnable = (InCheckState == ECheckBoxState::Checked);
				UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
				Settings->bUseEditorTRSGizmo = bEnable;
				Settings->SetUseExperimentalGizmo(bEnable);
				Settings->SaveConfig();
			})
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("EnableEditorTRSGizmo", "Enable EditorTRS Gizmo (5.8+)"))
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
		];

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.Padding(0.0f, 8.0f)
			[
				TogglePropertyViewWidget
			]

			+ SVerticalBox::Slot()
			.Padding(0.0f, 8.0f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(FText::FromString(TEXT("Editor Settings")))
					.OnClicked_Lambda([]() -> FReply
					{
						const UTransformGizmoEditorSettings* TransformGizmoEditorSettings = GetDefault<UTransformGizmoEditorSettings>();
						FModuleManager::LoadModuleChecked<ISettingsModule>("Settings")
						.ShowViewer(TransformGizmoEditorSettings->GetContainerName(), TransformGizmoEditorSettings->GetCategoryName(), TransformGizmoEditorSettings->GetSectionName());

						return FReply::Handled();
					})
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(2.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					MakeCVarToggleButton(CVarDebugDraw, TEXT("Debug Draw"))
				]

				// + SHorizontalBox::Slot()
				// .AutoWidth()
				// .Padding(FMargin(2.0f, 0.0f))
				// .VAlign(VAlign_Center)
				// [
				// 	MakeCVarToggleButton(CVarDebugHitDraw, TEXT("Debug Hit"))
				// ]
			]

			+ SVerticalBox::Slot()
			.Padding(0.0f, 8.0f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(FText::FromString(TEXT("Export Settings")))
					.OnClicked(this, &SGizmoSettings::SaveSettings)
					.IsEnabled(this, &SGizmoSettings::IsGizmoEnabled)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(FText::FromString(TEXT("Import Settings")))
					.OnClicked(this, &SGizmoSettings::LoadSettings)
					.IsEnabled(this, &SGizmoSettings::IsGizmoEnabled)
				]

				+ SHorizontalBox::Slot()
				.FillContentWidth(1.0f, 0.01f)
				.HAlign(HAlign_Fill)
				[
					SNew(SSpacer)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.Text(FText::FromString(TEXT("Reset All Settings")))
					.OnClicked(this, &SGizmoSettings::ResetSettings)
					.IsEnabled(this, &SGizmoSettings::IsGizmoEnabled)
				]
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			// Details tab switcher, details view stack
			SNew(SVerticalBox)
			.IsEnabled(this, &SGizmoSettings::IsGizmoEnabled)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSegmentedControl<int32>)
				.Value(0)
				.OnValueChanged_Lambda([Switcher = TabSwitcher](int32 InValue)
				{
					Switcher->SetActiveWidgetIndex(InValue);
				})

				+ SSegmentedControl<int32>::Slot(0)
				.Text(LOCTEXT("StyleTab", "Style"))
				.Icon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "MeshPaint.Brush").GetIcon())

				+ SSegmentedControl<int32>::Slot(1)
				.Text(LOCTEXT("InteractionTab", "Interaction"))
				.Icon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.EnableViewportHoverFeedback").GetIcon())

				+ SSegmentedControl<int32>::Slot(2)
				.Text(LOCTEXT("DebugTab", "Debug"))
				.Icon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Debug").GetIcon())
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.HAlign(HAlign_Fill)
			[
				TabSwitcher.ToSharedRef()
			]
		]
	];
}

TSharedRef<SWidget> SGizmoSettings::MakeStyleDetailsView()
{
	return MakeDetailsView<FTransformGizmoStyleCustomization>(
		[](UTransformGizmoEditorSettings* InSettings)
		{
			TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(
				FTransformGizmoStyle::StaticStruct(),
				reinterpret_cast<uint8*>(&const_cast<FTransformGizmoStyle&>(InSettings->GizmosParameters.Style)));
			return StructOnScope;
		})
		.ToSharedRef();
}

TSharedRef<SWidget> SGizmoSettings::MakeInteractionDetailsView()
{
	return MakeDetailsView<FTransformGizmoInteractionCustomization>(
	[](UTransformGizmoEditorSettings* InSettings)
		{
			TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(
				FTransformGizmoInteraction::StaticStruct(),
				reinterpret_cast<uint8*>(&const_cast<FTransformGizmoInteraction&>(InSettings->GizmosParameters.Interaction)));
				return StructOnScope;
		})
		.ToSharedRef();
}

TSharedRef<SWidget> SGizmoSettings::MakeDebugDetailsView()
{
	return MakeDetailsView<FGizmoDebugSettingsCustomization>(
	[](UTransformGizmoEditorSettings* InSettings)
		{
			TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(
				FGizmoDebugSettings::StaticStruct(),
				reinterpret_cast<uint8*>(&const_cast<FGizmoDebugSettings&>(InSettings->GizmosParameters.Debug)));
				return StructOnScope;
		})
		.ToSharedRef();
}

TSharedPtr<IDetailsView> SGizmoSettings::MakeDetailsView(
	TUniqueFunction<TSharedPtr<FStructOnScope>(UTransformGizmoEditorSettings*)>&& InStructProvider,
	FOnGetDetailCustomizationInstance InCustomizationGetter)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bShowSectionSelector = true;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
	DetailsViewArgs.bShowModifiedPropertiesOption = true;
	DetailsViewArgs.bShowKeyablePropertiesOption = false;
	DetailsViewArgs.bShowAnimatedPropertiesOption = false;
	DetailsViewArgs.bShowHiddenPropertiesWhilePlayingOption = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.bShowOptions = true;
	DetailsViewArgs.StyleKey = MakeShared<FDetailsViewStyleKey>(FDetailsViewStyleKeys::Card());
	DetailsViewArgs.ObjectFilter = MakeShared<FGizmoSettingsObjectFilter>();

	if (!FCoreUObjectDelegates::OnObjectTransacted.IsBoundToObject(this))
	{
		FCoreUObjectDelegates::OnObjectTransacted.AddSP(this, &SGizmoSettings::OnObjectTransacted);
	}

	TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	// TSharedPtr<FStructOnScope> StructOnScope = InStructProvider(GetMutableDefault<UTransformGizmoEditorSettings>());
	// TSharedPtr<IStructureDetailsView> StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, { }, StructOnScope);
	//
	// IDetailsView* DetailsView = StructureDetailsView->GetDetailsView();
	DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SGizmoSettings::IsPropertyVisible));

	DetailsView->RegisterInstancedCustomPropertyLayout(
		UTransformGizmoEditorSettings::StaticClass(),
		InCustomizationGetter);

	DetailsView->SetObject(GetMutableDefault<UTransformGizmoEditorSettings>(), true);

	DetailsView->OnFinishedChangingProperties().AddSPLambda(
		DetailsView.Get(),
		[](const FPropertyChangedEvent& PropertyChangedEvent){
			GetMutableDefault<UTransformGizmoEditorSettings>()->SaveConfig();
		});

	return DetailsView;
}

bool SGizmoSettings::IsPropertyVisible(const FPropertyAndParent& InPropertyAndParent)
{
	static TArray<FName> VisibleTopLevelPropertyNames{
		"Style",
		"Interaction",
		"Debug"
	};

	if (VisibleTopLevelPropertyNames.Contains(InPropertyAndParent.Property.GetFName()))
	{
		return true;
	}

	for (const FProperty* ParentProperty : InPropertyAndParent.ParentProperties)
	{
		if (ParentProperty && VisibleTopLevelPropertyNames.Contains(ParentProperty->GetFName()))
		{
			return true;
		}
	}

	return true;
}

void SGizmoSettings::NotifyPreChange(FEditPropertyChain* InPropertyAboutToChange)
{
}

void SGizmoSettings::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
{
	BroadcastGizmoParametersChanged();
}

void SGizmoSettings::OnObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionObjectEvent)
{
	if (InObject && InObject->GetClass() == UTransformGizmoEditorSettings::StaticClass()
		&& InTransactionObjectEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		GetMutableDefault<UTransformGizmoEditorSettings>()->SaveConfig();
		BroadcastGizmoParametersChanged();
	}
}

void SGizmoSettings::BroadcastGizmoParametersChanged() const
{
	UEditorInteractiveGizmoManager::OnGizmosParametersChangedDelegate().Broadcast(
	GetDefault<UTransformGizmoEditorSettings>()->GizmosParameters);
}

bool SGizmoSettings::IsGizmoEnabled() const
{
	return GetDefault<UTransformGizmoEditorSettings>()->bUseEditorTRSGizmo;
}

FReply SGizmoSettings::SetGizmoEnabled(const bool bInEnabled)
{
	UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();
	Settings->bUseEditorTRSGizmo = bInEnabled;
	Settings->SetUseExperimentalGizmo(bInEnabled);
	Settings->SaveConfig();

	return FReply::Handled();
}

FReply SGizmoSettings::SaveSettings()
{
	auto MakeJsonObject = [](const FTransformGizmoStyle& InStyle, const FTransformGizmoInteraction& InInteraction) -> TSharedPtr<FJsonObject>
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();

		const TSharedPtr<FJsonObject> StyleJsonObject = FJsonObjectConverter::UStructToJsonObject(InStyle);
		JsonObject->SetObjectField(TEXT("Style"), StyleJsonObject);

		const TSharedPtr<FJsonObject> InteractionJsonObject = FJsonObjectConverter::UStructToJsonObject(InInteraction);
		JsonObject->SetObjectField(TEXT("Interaction"), InteractionJsonObject);

		return JsonObject;
	};

	TSharedPtr<FJsonObject> DefaultJsonObject = MakeJsonObject(FTransformGizmoStyle(), FTransformGizmoInteraction());

	const UTransformGizmoEditorSettings* Settings = GetDefault<UTransformGizmoEditorSettings>();
	TSharedPtr<FJsonObject> SettingsJsonObject = MakeJsonObject(Settings->GizmosParameters.Style, Settings->GizmosParameters.Interaction);
	SettingsJsonObject->SetNumberField(TEXT("Version"), GizmoSettingsLocals::CurrentSettingsVersion); // Increment and account for if needed

	// GizmoSettingsLocals::RemoveDefaultJson(DefaultJsonObject, SettingsJsonObject);

	if (SettingsJsonObject)
	{
		FString FilePath;

		// File Save Dialog
		{
			IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
			if (DesktopPlatform == nullptr)
			{
				return FReply::Handled();
			}

			const FString DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_SAVE);
			const FString DefaultFilename = TEXT("GizmoSettings.json");
			const FString FileTypes = TEXT("JSON Files (*.json)|*.json");

			// show the file browse dialog
			TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
				? ParentWindow->GetNativeWindow()->GetOSWindowHandle()
				: nullptr;

			TArray<FString> OutFiles;
			if (!DesktopPlatform->SaveFileDialog(
				ParentWindowHandle,
				TEXT("Export Settings to Json"),
				DefaultPath,
				DefaultFilename,
				FileTypes,
				EFileDialogFlags::None,
				OutFiles)
				|| OutFiles.IsEmpty())
			{
				return FReply::Handled();
			}

			FilePath = OutFiles.Last();
		}

		FString JsonString;
		if (!FJsonSerializer::Serialize(SettingsJsonObject.ToSharedRef(), TJsonWriterFactory<>::Create(&JsonString, 0)))
		{
			return FReply::Unhandled();
		}

		if (FFileHelper::SaveStringToFile(JsonString, *FilePath))
		{
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SGizmoSettings::LoadSettings()
{
	FString FilePath;
	
	// File Open Dialog
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (DesktopPlatform == nullptr)
		{
			return FReply::Handled();
		}

		const FString DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN);
		const FString DefaultFilename = TEXT("GizmoSettings.json");
		const FString FileTypes = TEXT("JSON Files (*.json)|*.json");

		// show the file browse dialog
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
			? ParentWindow->GetNativeWindow()->GetOSWindowHandle()
			: nullptr;

		TArray<FString> OutFiles;
		if (!DesktopPlatform->OpenFileDialog(
			ParentWindowHandle,
			TEXT("Import Settings from Json"),
			DefaultPath,
			DefaultFilename,
			FileTypes,
			EFileDialogFlags::None,
			OutFiles)
			|| OutFiles.IsEmpty())
		{
			return FReply::Handled();
		}

		FilePath = OutFiles.Last();
	}
	
	UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();

	FString JsonString;
	if (!FFileHelper::LoadFileToString(JsonString, *FilePath))
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FJsonObject> RootJsonObject;
	if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(JsonString), RootJsonObject))
	{
		return FReply::Unhandled();
	}

	auto BroadcastPropertyChanged = [Settings]()
	{
		FProperty* RootProperty = FindFProperty<FProperty>(FGizmosParameters::StaticStruct(), GET_MEMBER_NAME_CHECKED(FGizmosParameters, Style));

		FEditPropertyChain PropertyChain;
		PropertyChain.AddHead(RootProperty);

		FPropertyChangedEvent ChangedEvent(RootProperty, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(PropertyChain, ChangedEvent);
		Settings->PostEditChangeChainProperty(ChangedChainEvent);
	};

	// Reset to defaults first, so that if the JSON is missing any fields, we won't end up with stale values from a previous import
	ResetSettings();

	int32 Version = INDEX_NONE;
	if (RootJsonObject->TryGetNumberField(TEXT("Version"), Version))
	{
		const TSharedPtr<FJsonObject>* StyleJsonObject = nullptr;
		if (RootJsonObject->TryGetObjectField(TEXT("Style"), StyleJsonObject))
		{
			FJsonObjectConverter::JsonObjectToUStruct(StyleJsonObject->ToSharedRef(), &Settings->GizmosParameters.Style);
		}

		const TSharedPtr<FJsonObject>* InteractionJsonObject = nullptr;
		if (RootJsonObject->TryGetObjectField(TEXT("Interaction"), InteractionJsonObject))
		{
			FJsonObjectConverter::JsonObjectToUStruct(InteractionJsonObject->ToSharedRef(), &Settings->GizmosParameters.Interaction);
		}

		const TSharedPtr<FJsonObject>* DebugJsonObject = nullptr;
		if (RootJsonObject->TryGetObjectField(TEXT("Debug"), DebugJsonObject))
		{
			FJsonObjectConverter::JsonObjectToUStruct(DebugJsonObject->ToSharedRef(), &Settings->GizmosParameters.Debug);
		}

		BroadcastPropertyChanged();
		BroadcastGizmoParametersChanged();

		return FReply::Handled();
	}
	else
	{
		// If Version wasn't found, the root object IS the style
		if (FJsonObjectConverter::JsonObjectToUStruct(RootJsonObject.ToSharedRef(), &Settings->GizmosParameters.Style))
		{
			BroadcastPropertyChanged();

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SGizmoSettings::ResetSettings()
{
	UTransformGizmoEditorSettings* Settings = GetMutableDefault<UTransformGizmoEditorSettings>();

	auto BroadcastPropertyChanged = [Settings](const FName InPropertyName)
	{
		FProperty* RootProperty = FindFProperty<FProperty>(FGizmosParameters::StaticStruct(), InPropertyName);

		FEditPropertyChain PropertyChain;
		PropertyChain.AddHead(RootProperty);

		FPropertyChangedEvent ChangedEvent(RootProperty, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(PropertyChain, ChangedEvent);
		Settings->PostEditChangeChainProperty(ChangedChainEvent);
	};

	Settings->GizmosParameters.Style = FTransformGizmoStyle();
	Settings->GizmosParameters.Interaction = FTransformGizmoInteraction();
	// @note: we explicitly don't reset debug settings

	BroadcastPropertyChanged(GET_MEMBER_NAME_CHECKED(FGizmosParameters, Style));
	BroadcastPropertyChanged(GET_MEMBER_NAME_CHECKED(FGizmosParameters, Interaction));
	BroadcastGizmoParametersChanged();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
