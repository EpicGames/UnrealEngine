// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailsView.h"
#include "Misc/NotifyHook.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FStructOnScope;
class IStructureDetailsView;
class UTransformGizmoEditorSettings;
class SWidgetSwitcher;
template <typename ValueType> class TCVarToggle;

class SGizmoSettings
	: public SCompoundWidget
	, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SGizmoSettings)
	{ }
	SLATE_END_ARGS()

	virtual ~SGizmoSettings() override;

	void Construct(const FArguments& InArgs);

private:
	TSharedRef<SWidget> MakeStyleDetailsView();
	TSharedRef<SWidget> MakeInteractionDetailsView();
	TSharedRef<SWidget> MakeDebugDetailsView();

	TSharedPtr<IDetailsView> MakeDetailsView(TUniqueFunction<TSharedPtr<FStructOnScope>(UTransformGizmoEditorSettings*)>&& InStructProvider, FOnGetDetailCustomizationInstance InCustomizationGetter);

	template <typename CustomizationType>
	TSharedPtr<IDetailsView> MakeDetailsView(TUniqueFunction<TSharedPtr<FStructOnScope>(UTransformGizmoEditorSettings*)>&& InStructProvider)
	{
		return MakeDetailsView(
			MoveTemp(InStructProvider),
			FOnGetDetailCustomizationInstance::CreateLambda([]() -> TSharedRef<IDetailCustomization> { return MakeShared<CustomizationType>();	}));
	}

	bool IsPropertyVisible(const FPropertyAndParent& InPropertyAndParent);

	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FEditPropertyChain* InPropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged) override;
	//~ End FNotifyHook

	void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent);

	void BroadcastGizmoParametersChanged() const;

	bool IsGizmoEnabled() const;
	FReply SetGizmoEnabled(const bool bInEnabled);

	FReply SaveSettings();
	FReply LoadSettings();

	/** Resets style and interaction. */
	FReply ResetSettings();

private:
	/** Switcher that controls which tab widget is currently visible */
	TSharedPtr<SWidgetSwitcher> TabSwitcher;

	TSharedPtr<IDetailsView> StyleDetailsView;
	TSharedPtr<IDetailsView> InteractionDetailsView;

	TSharedPtr<TCVarToggle<bool>> CVarDebugDraw;
	TSharedPtr<TCVarToggle<bool>> CVarDebugHitDraw;
};
