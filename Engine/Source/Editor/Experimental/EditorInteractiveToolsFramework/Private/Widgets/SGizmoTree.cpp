// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SGizmoTree.h"

#include "BaseGizmos/GizmoElementGroup.h"
#include "BaseGizmos/GizmoUtil.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "ContextObjectStore.h"
#include "DetailLayoutBuilder.h"
#include "DetailsViewArgs.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorGizmos/EditorTRSGizmo.h"
#include "EditorGizmos/GizmoElementGimbal.h"
#include "EditorGizmos/TransformGizmo.h"
#include "EditorGizmos/TransformGizmoAccessor.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorModeManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/FileHelper.h"
#include "Misc/NotifyHook.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SGizmoTree"

namespace UE::Editor::GizmoSettings::Private
{
	struct FGizmoTreeColumns
	{
		static const FName& Color()
		{
			static FName Color("Color");
			return Color;
		}

		static const FName& Name()
		{
			static FName Name("Name");
			return Name;
		}

		static const FName& PartId()
		{
			static FName PartId("PartId");
			return PartId;
		}

		static const FName& PreviousState()
		{
			static FName PreviousState("Previous State");
			return PreviousState;
		}

		static const FName& VertexColor()
		{
			static FName FillColor("Fill Color");
			return FillColor;
		}

		static const FName& LineColor()
		{
			static FName LineColor("Line Color");
			return LineColor;
		}

		static const FName& Material()
		{
			static FName Material("Material");
			return Material;
		}

		static const FName& State()
		{
			static FName State("State");
			return State;
		}

		static FName& Visibility()
		{
			static FName Visibility("Visibility");
			return Visibility;
		}

		static FName& Selected()
		{
			static FName Selected("Selected");
			return Selected;
		}
	};

	FName GetTransformPartName(const ETransformGizmoPartIdentifier InPartId)
	{
		static TArray<FName> ValueNames;

		// Init
		if (ValueNames.IsEmpty())
		{
			ValueNames.Reserve(static_cast<int32>(ETransformGizmoPartIdentifier::Max));

			UEnum* Enum = StaticEnum<ETransformGizmoPartIdentifier>();
			if (Enum)
			{
				// First element is "Default", which shouldn't show with any text
				ValueNames.Emplace(NAME_None);
				for (int32 ValueIdx = 1; ValueIdx < static_cast<int32>(ETransformGizmoPartIdentifier::Max); ++ValueIdx)
				{
					ValueNames.Emplace(FName(Enum->GetDisplayNameTextByIndex(ValueIdx).ToString()));	
				}
			}
		}

		return ValueNames[static_cast<int32>(InPartId)];
	}

	FName GetInteractionStateName(const EGizmoElementInteractionState InState)
	{
		static TArray<FName> ValueNames = {
			"None",
			"Hovering",
			"Interacting",
			"Selected",
			"Subdued"
		};

		return ValueNames[static_cast<int32>(InState)];
	}
}

class UTransformGizmoEditorSettings;

TSharedPtr<FGizmoTreeElementViewModel> FGizmoTreeElementViewModel::Construct(const UGizmoElementBase* InGizmoElement, const TFunction<bool(const uint32)>& InGetSelectedFunc, const TSharedPtr<FGizmoTreeElementViewModel>& InParentViewModel)
{
	TSharedPtr<FGizmoTreeElementViewModel> ViewModel = MakeShared<FGizmoTreeElementViewModel>();
	
	ViewModel->WeakGizmoElement = InGizmoElement;
	ViewModel->WeakLineElement = Cast<UGizmoElementLineBase>(InGizmoElement);
	ViewModel->WeakParent = InParentViewModel;
	ViewModel->Level = InParentViewModel.IsValid() ? InParentViewModel->GetLevel() + 1 : 0;
	ViewModel->Color = FLinearColor::Transparent;
	ViewModel->GetSelectedFunc = InGetSelectedFunc;
	
	if (InGizmoElement)
	{
		ViewModel->UniqueId = InGizmoElement->GetUniqueID();

		ViewModel->Color = InGizmoElement->GetVertexColor();
		if (ViewModel->Color == FLinearColor::White)
		{
			if (InParentViewModel.IsValid())
			{
				ViewModel->Color = InParentViewModel->GetColor();
			}

			if (ViewModel->Color == FLinearColor::White)
			{
				ViewModel->Color = FLinearColor::Transparent;	
			}
		}

		ViewModel->OwningPropertyName = ViewModel->ResolveOwningPropertyName(InGizmoElement);
	}

	ViewModel->PopulateChildren();

	return ViewModel;
}

const int32 FGizmoTreeElementViewModel::GetLevel() const
{
	return Level;
}

const FLinearColor FGizmoTreeElementViewModel::GetColor() const
{
	return Color;
}

const FName FGizmoTreeElementViewModel::GetPartName() const
{
	if (const UGizmoElementBase* GizmoElement = WeakGizmoElement.Get())
	{
		using namespace UE::Editor::GizmoSettings::Private;

		const FName PartName = GetTransformPartName(static_cast<ETransformGizmoPartIdentifier>(GizmoElement->GetPartIdentifier()));
		if (PartName == NAME_None)
		{
			if (TSharedPtr<FGizmoTreeElementViewModel> StrongParent = WeakParent.Pin())
			{
				return StrongParent->GetPartName();
			}
		}

		return PartName;
	}

	return NAME_None;
}

const bool FGizmoTreeElementViewModel::IsPartInherited() const
{
	if (const UGizmoElementBase* GizmoElement = WeakGizmoElement.Get())
	{
		using namespace UE::Editor::GizmoSettings::Private;

		const FName PartName = GetTransformPartName(static_cast<ETransformGizmoPartIdentifier>(GizmoElement->GetPartIdentifier()));
		if (PartName == NAME_None)
		{
			if (TSharedPtr<FGizmoTreeElementViewModel> StrongParent = WeakParent.Pin())
			{
				return true;
			}
		}

		return false;
	}

	return false;
}

const FString FGizmoTreeElementViewModel::GetName() const
{
	if (OwningPropertyName.IsEmpty())
	{
		if (const UGizmoElementBase* GizmoElement = WeakGizmoElement.Get())
		{
			return GizmoElement->GetName();
		}	
	}
	else
	{
		return OwningPropertyName;
	}

	return TEXT("");
}

const FName FGizmoTreeElementViewModel::GetStateName() const
{
	if (const UGizmoElementBase* GizmoElement = WeakGizmoElement.Get())
	{
		EGizmoElementInteractionState ElementState = GizmoElement->GetElementInteractionState();
		if (ElementState != CurrentState)
		{
			PreviousState = CurrentState;
			CurrentState = ElementState;
		}

		using namespace UE::Editor::GizmoSettings::Private;
		return GetInteractionStateName(CurrentState);
	}

	return NAME_None;
}

const FLinearColor FGizmoTreeElementViewModel::GetStateColor() const
{
	if (const UGizmoElementBase* GizmoElement = WeakGizmoElement.Get())
	{
		return GetStateColorInternal(GizmoElement->GetElementInteractionState());
	}

	return FLinearColor::Transparent;
}

const FName FGizmoTreeElementViewModel::GetPreviousStateName() const
{
	using namespace UE::Editor::GizmoSettings::Private;
	return GetInteractionStateName(PreviousState);
}

const FLinearColor FGizmoTreeElementViewModel::GetPreviousStateColor() const
{
	return GetStateColorInternal(PreviousState);;
}

const bool FGizmoTreeElementViewModel::IsStateInherited() const
{
	return false;
}

const bool FGizmoTreeElementViewModel::IsVisible() const
{
	bool bIsEnabled = false;
	if (const UGizmoElementBase* GizmoElement = WeakGizmoElement.Get())
	{
		bIsEnabled = GizmoElement->GetVisibleState() && FGizmoElementAccessor::IsEnabledForInteractionState(*GizmoElement);
		if (bIsEnabled)
		{
			if (TSharedPtr<FGizmoTreeElementViewModel> StrongParent = WeakParent.Pin())
			{
				if (!StrongParent->IsVisible())
				{
					bIsEnabled = false;
				}
			}
		}
	}

	return bIsEnabled;
}

const bool FGizmoTreeElementViewModel::IsSelected() const
{
	bool bIsSelected = false;
	if (const UGizmoElementBase* GizmoElement = WeakGizmoElement.Get())
	{
		if (GetSelectedFunc.IsSet())
		{
			bIsSelected = GetSelectedFunc(GizmoElement->GetPartIdentifier());
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "GetSelectedFunc not bound!");
		}
	}

	return bIsSelected;
}

const FLinearColor FGizmoTreeElementViewModel::GetCurrentVertexColor() const
{
	if (const UGizmoElementBase* GizmoElement = WeakGizmoElement.Get())
	{
		const EGizmoElementInteractionState State = GizmoElement->GetElementInteractionState();
		switch (State)
		{
		case EGizmoElementInteractionState::None:
			return GizmoElement->GetVertexColor();

		case EGizmoElementInteractionState::Hovering:
			return GizmoElement->GetHoverVertexColor();

		case EGizmoElementInteractionState::Interacting:
			return GizmoElement->GetInteractVertexColor();

		case EGizmoElementInteractionState::Selected:
			return GizmoElement->GetSelectVertexColor();

		case EGizmoElementInteractionState::Subdued:
			return GizmoElement->GetSubdueVertexColor();

		case EGizmoElementInteractionState::Max:
		default:
			return FLinearColor::Transparent;
		}
	}

	return FLinearColor::Transparent;
}

const FLinearColor FGizmoTreeElementViewModel::GetCurrentLineColor() const
{
	if (const UGizmoElementLineBase* LineElement = WeakLineElement.Get())
	{
		const EGizmoElementInteractionState State = LineElement->GetElementInteractionState();
		switch (State)
		{
		case EGizmoElementInteractionState::None:
			return LineElement->GetLineColor();

		case EGizmoElementInteractionState::Hovering:
			return LineElement->GetHoverLineColor();

		case EGizmoElementInteractionState::Interacting:
			return LineElement->GetInteractLineColor();

		case EGizmoElementInteractionState::Selected:
			return LineElement->GetSelectLineColor();

		case EGizmoElementInteractionState::Subdued:
			return LineElement->GetSubdueLineColor();

		case EGizmoElementInteractionState::Max:
		default:
			return FLinearColor::Transparent;
		}
	}

	return FLinearColor::Transparent;
}

const FString FGizmoTreeElementViewModel::GetCurrentMaterialName() const
{
	if (const UGizmoElementBase* GizmoElement = WeakGizmoElement.Get())
	{
		const EGizmoElementInteractionState State = GizmoElement->GetElementInteractionState();
		const UMaterialInterface* Material = nullptr;
		switch (State)
		{
		case EGizmoElementInteractionState::None:
			Material = GizmoElement->GetMaterial();
			break;

		case EGizmoElementInteractionState::Hovering:
			Material = GizmoElement->GetHoverMaterial();
			break;

		case EGizmoElementInteractionState::Interacting:
			Material = GizmoElement->GetInteractMaterial();
			break;

		case EGizmoElementInteractionState::Selected:
			Material = GizmoElement->GetSelectMaterial();
			break;

		case EGizmoElementInteractionState::Subdued:
			Material = GizmoElement->GetSubdueMaterial();
			break;

		case EGizmoElementInteractionState::Max:
		default:
			break;
		}

		return Material ? Material->GetName() : TEXT("");
	}

	return TEXT("");
}

TConstArrayView<TSharedPtr<FGizmoTreeElementViewModel>> FGizmoTreeElementViewModel::GetChildren() const
{
	return Children;
}

void FGizmoTreeElementViewModel::PopulateChildren()
{
	Children.Reset();
	if (const UGizmoElementBase* GizmoElement = WeakGizmoElement.Get())
	{
		if (const UGizmoElementGroupBase* GroupElement = Cast<UGizmoElementGroupBase>(GizmoElement))
		{
			const TConstArrayView<UGizmoElementBase*> SubElements = FGizmoElementAccessor::GetSubElements(*GroupElement);
			Children.Reserve(SubElements.Num());
			for (UGizmoElementBase* SubGizmoElement : SubElements)
			{
				TSharedPtr<FGizmoTreeElementViewModel> NewItem = FGizmoTreeElementViewModel::Construct(SubGizmoElement, GetSelectedFunc, AsShared());
				Children.Emplace(NewItem);
			}
		}
	}
}

FString FGizmoTreeElementViewModel::ResolveOwningPropertyName(const UGizmoElementBase* InElement)
{
	if (!InElement)
	{
		return TEXT("");
	}

	auto FindPropertyContainingObject = [](const UObject* InObject) -> FProperty*
	{
		if (!InObject || !InObject->GetOuter())
		{
			return nullptr;
		}
    
		// Get the containing object
		UObject* OuterObject = InObject->GetOuter();
		if (OuterObject == GetTransientPackage())
		{
			TArray<UObject*> CandidateOuterObjects;
			for (TObjectIterator<UObject> It; It; ++It)
			{
				UObject* CandidateOuter = *It;
				if (!CandidateOuter || CandidateOuter->IsUnreachable() || CandidateOuter == InObject || CandidateOuter->IsA<UClass>())
				{
					continue;
				}

				CandidateOuterObjects.Add(CandidateOuter);
			}

			for (UObject* CandidateOuter : CandidateOuterObjects)
			{
				if (!IsValid(CandidateOuter))
				{
					continue;
				}

				UClass* OuterClass = CandidateOuter->GetClass();
				if (!OuterClass)
				{
					continue;
				}

				for (TFieldIterator<FProperty> PropIt(OuterClass); PropIt; ++PropIt)
				{
					FProperty* Property = *PropIt;
      
					// Check object properties
					if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
					{
						if (ObjectProperty->ContainerPtrToValuePtr<void>(CandidateOuter))
						{
							UObject* ReferencedObject = ObjectProperty->GetObjectPropertyValue_InContainer(CandidateOuter);
							if (ReferencedObject == InObject)
							{
								return Property;
							}
						}
					}
					// Check array properties
					else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
					{
						if (FObjectProperty* InnerProperty = CastField<FObjectProperty>(ArrayProperty->Inner))
						{
							FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(CandidateOuter));
							for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
							{
								UObject* ElementObject = InnerProperty->GetObjectPropertyValue(ArrayHelper.GetRawPtr(Index));
								if (ElementObject == InObject)
								{
									return Property;
								}
							}
						}
					}
				}
			}
		}
		else
		{
			// Get the object's class
			UClass* OuterClass = OuterObject->GetClass();
			if (!OuterClass)
			{
				return nullptr;
			}
		    
			// Iterate through properties of the outer object
			for (TFieldIterator<FProperty> PropIt(OuterClass); PropIt; ++PropIt)
			{
				FProperty* Property = *PropIt;
		        
				// Handle object property
				if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
				{
					const UObject* ReferencedObject = ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject);
					if (ReferencedObject == InObject)
					{
						return Property;
					}
				}
				// Handle array property of objects
				else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					if (FObjectProperty* ObjProperty = CastField<FObjectProperty>(ArrayProperty->Inner))
					{
						FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(OuterObject));
						for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
						{
							const UObject* ElementObject = ObjProperty->GetObjectPropertyValue(ArrayHelper.GetRawPtr(Index));
							if (ElementObject == InObject)
							{
								return Property;
							}
						}
					}
				}
			}
		}
    
		return nullptr;
	};

	if (FProperty* ContainingProperty = FindPropertyContainingObject(InElement))
	{
		// Get the name of the property
		return ContainingProperty->GetName();
	}

	// Fallback to Object Name
	return InElement->GetName();
}

const FLinearColor FGizmoTreeElementViewModel::GetStateColorInternal(const EGizmoElementInteractionState InState) const
{
	switch (InState)
	{
	case EGizmoElementInteractionState::None:
		return FLinearColor::Transparent;
	case EGizmoElementInteractionState::Hovering:
		return FLinearColor(0.57f, 0.17f, 0.0f);
	case EGizmoElementInteractionState::Interacting:
		return FLinearColor(0.47f, 0.015f, 0.015f);
	case EGizmoElementInteractionState::Selected:
		return FLinearColor(0.17f, 0.315f, 0.015f);
	case EGizmoElementInteractionState::Subdued:
		return FLinearColor::Gray * 0.5f;
	default:
		return FLinearColor::Transparent;		
	}
}

TSharedPtr<FGizmoTreeViewModel::FGizmoTreeContextObjectViewModel> FGizmoTreeViewModel::FGizmoTreeContextObjectViewModel::Construct(const UObject* InContextObject)
{
	TSharedPtr<FGizmoTreeContextObjectViewModel> ViewModel = MakeShared<FGizmoTreeContextObjectViewModel>();

	if (InContextObject)
	{
		if (const UClass* ContextObjectClass = InContextObject->GetClass())
		{
			ViewModel->Name = ContextObjectClass->GetFName();

			for (TFieldIterator<FProperty> PropertyIt(ContextObjectClass, EFieldIterationFlags::IncludeSuper); PropertyIt; ++PropertyIt)
			{
				if (const FProperty* Property = *PropertyIt;
					Property->GetOwnerClass() != UObject::StaticClass())
				{
					ViewModel->PropertyNames.Add(Property->GetFName());
				}
			}

			for (TFieldIterator<UFunction> FunctionIt(ContextObjectClass, EFieldIterationFlags::IncludeSuper); FunctionIt; ++FunctionIt)
			{
				if (const UFunction* Function = *FunctionIt;
					Function->GetOuterUClass() != UObject::StaticClass())
				{
					ViewModel->FunctionNames.Add(Function->GetFName());
				}
			}
		}
	}

	return ViewModel;
}

FGizmoTreeViewModel::~FGizmoTreeViewModel()
{
	if (IConsoleVariable* DebugDrawCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Gizmos.DebugDraw"), false))
	{
		DebugDrawCVar->OnChangedDelegate().RemoveAll(this);
	}
}

TSharedPtr<FGizmoTreeViewModel> FGizmoTreeViewModel::Construct(const UTransformGizmo* InGizmo)
{
	TSharedPtr<FGizmoTreeViewModel> ViewModel = MakeShared<FGizmoTreeViewModel>();
	
	ViewModel->WeakGizmo = InGizmo;

	if (IConsoleVariable* DebugDrawCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Gizmos.DebugDraw"), false))
	{
		ViewModel->bIsDebugDrawing = DebugDrawCVar->GetBool();
		DebugDrawCVar->OnChangedDelegate().AddSP(ViewModel.Get(), &FGizmoTreeViewModel::OnDebugDrawChanged);
	}

	if (const UInteractiveGizmoManager* GizmoManager = InGizmo->GetGizmoManager())
	{
		if (const UContextObjectStore* ContextObjectStore = GizmoManager->GetContextObjectStore())
		{
			const FName ContextObjectsPropertyName = "ContextObjects";
			if (FProperty* ContextObjectsProperty = ContextObjectStore->StaticClass()->FindPropertyByName(ContextObjectsPropertyName))
			{
				if (const FArrayProperty* ContextObjectsArrayProperty = CastField<FArrayProperty>(ContextObjectsProperty))
				{
					if (const FObjectProperty* ContextObjectProperty = CastField<FObjectProperty>(ContextObjectsArrayProperty->Inner))
					{
						FScriptArrayHelper ArrayHelper(ContextObjectsArrayProperty, ContextObjectsProperty->ContainerPtrToValuePtr<void>(ContextObjectStore));
						ViewModel->ContextObjects.Reserve(ArrayHelper.Num());
						for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
						{
							if (const UObject* ContextObject = ContextObjectProperty->GetObjectPropertyValue(ArrayHelper.GetRawPtr(Index)))
							{
								ViewModel->ContextObjects.Emplace(FGizmoTreeContextObjectViewModel::Construct(ContextObject));
							}
						}
					}
				}
			}
		}
	}

	return ViewModel;
}

const FName FGizmoTreeViewModel::GetHitPartName() const
{
	if (const UTransformGizmo* StrongGizmo = WeakGizmo.Get())
	{
		using namespace UE::Editor::InteractiveToolsFramework;
		using namespace UE::Editor::GizmoSettings::Private;

		const FTransformGizmoAccessor Accessor;
		return GetTransformPartName(Accessor.GetLastHitPart(*StrongGizmo));
	}

	return NAME_None;
}

const FName FGizmoTreeViewModel::GetLastSelectedPartName() const
{
	if (const UTransformGizmo* StrongGizmo = WeakGizmo.Get())
	{
		using namespace UE::Editor::InteractiveToolsFramework;
		using namespace UE::Editor::GizmoSettings::Private;

		const FTransformGizmoAccessor Accessor;
		return GetTransformPartName(Accessor.GetLastHitPart(*StrongGizmo));
	}

	return NAME_None;
}

const FName FGizmoTreeViewModel::GetLastHoveredPartName() const
{
	if (const UTransformGizmo* StrongGizmo = WeakGizmo.Get())
	{
		using namespace UE::Editor::InteractiveToolsFramework;
		using namespace UE::Editor::GizmoSettings::Private;

		const FTransformGizmoAccessor Accessor;
		return GetTransformPartName(Accessor.GetLastHitPart(*StrongGizmo));
	}

	return NAME_None;
}

const FName FGizmoTreeViewModel::GetLastInteractedPartName() const
{
	if (const UTransformGizmo* StrongGizmo = WeakGizmo.Get())
	{
		using namespace UE::Editor::InteractiveToolsFramework;
		using namespace UE::Editor::GizmoSettings::Private;

		const FTransformGizmoAccessor Accessor;
		return GetTransformPartName(Accessor.GetLastHitPart(*StrongGizmo));
	}

	return NAME_None;
}

const FName FGizmoTreeViewModel::GetLastSubduedPartName() const
{
	if (const UTransformGizmo* StrongGizmo = WeakGizmo.Get())
	{
		using namespace UE::Editor::InteractiveToolsFramework;
		using namespace UE::Editor::GizmoSettings::Private;

		const FTransformGizmoAccessor Accessor;
		return GetTransformPartName(Accessor.GetLastHitPart(*StrongGizmo));
	}

	return NAME_None;
}

const FText FGizmoTreeViewModel::GetInputRay() const
{
	return FText();
}

const FText FGizmoTreeViewModel::GetInputPosition() const
{
	return FText();
}

const TArray<TSharedPtr<FGizmoTreeViewModel::FGizmoTreeContextObjectViewModel>>& FGizmoTreeViewModel::GetContextObjects() const
{
	return ContextObjects;
}

bool FGizmoTreeViewModel::IsDebugDrawing() const
{
	return bIsDebugDrawing;
}

void FGizmoTreeViewModel::SetDebugDraw(const bool bInValue)
{
	if (IConsoleVariable* DebugDrawCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Gizmos.DebugDraw"), false))
	{
		DebugDrawCVar->Set(bInValue);
	}
}

void FGizmoTreeViewModel::OnDebugDrawChanged(IConsoleVariable* InCVar)
{
	if (InCVar)
	{
		bIsDebugDrawing.store(InCVar->GetBool());
	}
}

void SGizmoTreeTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InTreeView)
{
	Item = InArgs._Item;

	const FSuperRowType::FArguments Args =
		FSuperRowType::FArguments()
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));

	SMultiColumnTableRow<TSharedPtr<FGizmoTreeElementViewModel>>::Construct(Args, InTreeView);
}

TSharedRef<SWidget> SGizmoTreeTableRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	using namespace UE::Editor::GizmoSettings::Private;

	TSharedRef<SWidget> ItemWidget = SNullWidget::NullWidget;

	if (Item.IsValid())
	{
		auto MakeStateWidget = [&](
			TMemFunPtrType<true, FGizmoTreeElementViewModel, const FName()>::Type InNameGetter,
			TMemFunPtrType<true, FGizmoTreeElementViewModel, const FLinearColor()>::Type InColorGetter)
		{
			return SNew(SBox)
			.WidthOverride(128.0f)
			.HeightOverride(16.0f)
			[
				SNew(SBorder)
				.BorderImage(new FSlateRoundedBoxBrush(FStyleColors::White, 4.0f))
				.BorderBackgroundColor_Lambda([WeakItem = Item.ToWeakPtr(), InColorGetter]()
				{
					if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
					{
						return FSlateColor(Invoke(InColorGetter, StrongItem.Get()));
					}
				
					return FSlateColor(FLinearColor::Transparent);
				})
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.ArrowUp"))
						.Visibility_Lambda([WeakItem = Item.ToWeakPtr()]()
						{
							if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
							{
								return StrongItem->IsStateInherited() ? EVisibility::Visible : EVisibility::Collapsed;
							}

							return EVisibility::Collapsed;
						})
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(STextBlock)
						.Text_Lambda([WeakItem = Item.ToWeakPtr(), InNameGetter]()
						{
							if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
							{
								return FText::FromName(Invoke(InNameGetter, StrongItem.Get()));
							}
						
							return FText();
						})
						.ColorAndOpacity_Lambda([WeakItem = Item.ToWeakPtr(), InNameGetter]() -> FSlateColor
						{
							if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
							{
								return Invoke(InNameGetter, StrongItem.Get()) == "Hovering"
									? FLinearColor::Black
									: FLinearColor::White;
							}

							return FLinearColor::White;
						})
					]
				]
			];
		};

		auto MakeColorWidget = [&](TMemFunPtrType<true, FGizmoTreeElementViewModel, const FLinearColor()>::Type InColorGetter)
		{
			return SNew(SBox)
			.WidthOverride(16.0f)
			.HeightOverride(16.0f)
			[
				SNew(SBorder)
				.Padding(1.0f)
				.BorderImage(new FSlateRoundedBoxBrush(FStyleColors::White, 4.0f))
				.BorderBackgroundColor_Lambda([WeakItem = Item.ToWeakPtr(), InColorGetter]()
				{
					if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
					{
						return FSlateColor(Invoke(InColorGetter, StrongItem.Get()));
					}

					return FSlateColor(FLinearColor::Transparent);
				})
			];
		};
		
		if (InColumnName == FGizmoTreeColumns::Color())
		{
			ItemWidget = MakeColorWidget(&FGizmoTreeElementViewModel::GetColor);
		}
		else if (InColumnName == FGizmoTreeColumns::PartId())
		{
			ItemWidget =
				SNew(SBox)
				.Padding_Lambda([WeakItem = Item.ToWeakPtr()]()
				{
					if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
					{
						return FMargin(static_cast<float>(StrongItem->GetLevel()) * 32.0f, 0.0f, 0.0f, 0.0f);
					}

					return FMargin(0);
				})
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.ArrowUp"))
						.Visibility_Lambda([WeakItem = Item.ToWeakPtr()]()
						{
							if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
							{
								return StrongItem->IsPartInherited() ? EVisibility::Visible : EVisibility::Collapsed;
							}

							return EVisibility::Collapsed;
						})
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(STextBlock)
						.Text_Lambda([WeakItem = Item.ToWeakPtr()]()
						{
							if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
							{
								const FName PartName = StrongItem->GetPartName();
								if (PartName.IsNone())
								{
									return FText();
								}

								return FText::FromName(PartName);
							}

							return FText();
						})
					]
				];
		}
		else if (InColumnName == FGizmoTreeColumns::Name())
		{
			ItemWidget = SNew(STextBlock)
				.Text_Lambda([WeakItem = Item.ToWeakPtr()]()
				{
					if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
					{
						return FText::FromString(StrongItem->GetName());
					}

					return FText();
				});
		}
		else if (InColumnName == FGizmoTreeColumns::PreviousState())
		{
			ItemWidget = MakeStateWidget(&FGizmoTreeElementViewModel::GetPreviousStateName, &FGizmoTreeElementViewModel::GetPreviousStateColor);
		}
		else if (InColumnName == FGizmoTreeColumns::State())
		{
			ItemWidget = MakeStateWidget(&FGizmoTreeElementViewModel::GetStateName, &FGizmoTreeElementViewModel::GetStateColor);
		}
		else if (InColumnName == FGizmoTreeColumns::VertexColor())
		{
			ItemWidget = MakeColorWidget(&FGizmoTreeElementViewModel::GetCurrentVertexColor);
		}
		else if (InColumnName == FGizmoTreeColumns::LineColor())
		{
			ItemWidget = MakeColorWidget(&FGizmoTreeElementViewModel::GetCurrentLineColor);
		}
		else if (InColumnName == FGizmoTreeColumns::Material())
		{
			ItemWidget = SNew(STextBlock)
				.Text_Lambda([WeakItem = Item.ToWeakPtr()]()
				{
					if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
					{
						return FText::FromString(StrongItem->GetCurrentMaterialName());
					}

					return FText();
				});
		}
		else if (InColumnName == FGizmoTreeColumns::Visibility())
		{
			ItemWidget = SNew(SCheckBox)
				.IsEnabled(false)
				.IsChecked_Lambda([WeakItem = Item.ToWeakPtr()]()
				{
					if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
					{
						return StrongItem->IsVisible() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}

					return ECheckBoxState::Checked;
				});
		}
		else if (InColumnName == FGizmoTreeColumns::Selected())
		{
			ItemWidget = SNew(SCheckBox)
				.IsEnabled(false)
				.IsChecked_Lambda([WeakItem = Item.ToWeakPtr()]()
				{
					if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
					{
						return StrongItem->IsSelected() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}

					return ECheckBoxState::Checked;
				});
		}
	}

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(2.0f)
		.IsEnabled_Lambda([WeakItem = Item.ToWeakPtr()]()
		{
			if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
			{
				return StrongItem->IsVisible();
			}

			return false;
		})
		[
			ItemWidget
		];
}

void SGizmoTree::Construct(const FArguments& InArgs)
{
	// @note: we just get the default transform gizmo here
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		WeakTransformGizmo = UE::EditorTransformGizmoUtil::FindDefaultTransformGizmo(ToolManager);
	}

	constexpr float ColorColumnWidth = 32.0f;

	SAssignNew(HeaderRowWidget, SHeaderRow)
		+ SHeaderRow::Column(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::Color())
		.DefaultLabel(FText::FromString(TEXT("")))
		.FixedWidth(ColorColumnWidth)

		+ SHeaderRow::Column(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::PartId())
		.DefaultLabel(FText::FromString(TEXT("Part Identifier")))
		.FillWidth(0.3f)
		
		+ SHeaderRow::Column(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::Name())
		.DefaultLabel(FText::FromString(TEXT("Name")))
		.FillWidth(0.3f)

		+ SHeaderRow::Column(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::PreviousState())
		.DefaultLabel(FText::FromString(TEXT("Prev. State")))
		.FillWidth(0.2f)

		+ SHeaderRow::Column(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::State())
		.DefaultLabel(FText::FromString(TEXT("State")))
		.FillWidth(0.2f)

		+ SHeaderRow::Column(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::VertexColor())
		.DefaultLabel(FText::FromString(TEXT("Fill Color")))
		.FixedWidth(ColorColumnWidth)

		+ SHeaderRow::Column(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::LineColor())
		.DefaultLabel(FText::FromString(TEXT("Stroke Color")))
		.FixedWidth(ColorColumnWidth)

		+ SHeaderRow::Column(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::Selected())
		.DefaultLabel(FText::FromString(TEXT("Selected")))
		.FixedWidth(ColorColumnWidth)

		+ SHeaderRow::Column(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::Visibility())
		.DefaultLabel(FText::FromString(TEXT("Visibility")))
		.FixedWidth(ColorColumnWidth)

		+ SHeaderRow::Column(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::Material())
		.DefaultLabel(FText::FromString(TEXT("Material")))
		.FillWidth(0.3f);

	ViewModel = FGizmoTreeViewModel::Construct(WeakTransformGizmo.Get());

	UpdateGizmoChoices();
	UpdateTreeItems();
	OnShowLastStateChanged(bShowLastState ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);

	auto MakeRowWidget = [&](const FText& InLabel, TMemFunPtrType<true, FGizmoTreeViewModel, const FText()>::Type InValueGetter, const FLinearColor& InBackgroundColor = FLinearColor::Transparent) -> TSharedRef<SWidget>
	{
		return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(0.2f)
		[
			SNew(STextBlock)
			.Text(InLabel)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(0.8f)
		[
			SNew(STextBlock)
			.Text_Lambda([WeakViewModel = ViewModel.ToWeakPtr(), InValueGetter]()
			{
				if (TSharedPtr<FGizmoTreeViewModel> StrongViewModel = WeakViewModel.Pin())
				{
					return Invoke(InValueGetter, StrongViewModel.Get());
				}

				return FText();
			})
		];
	};

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.Type(ESlateCheckBoxType::Type::ToggleButton)
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
				.IsChecked_Lambda([WeakViewModel = ViewModel.ToWeakPtr()]()
				{
					if (TSharedPtr<FGizmoTreeViewModel> StrongViewModel = WeakViewModel.Pin())
					{
						return StrongViewModel->IsDebugDrawing() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}

					return ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([WeakViewModel = ViewModel.ToWeakPtr()](const ECheckBoxState& InCheckState)
				{
					if (TSharedPtr<FGizmoTreeViewModel> StrongViewModel = WeakViewModel.Pin())
					{
						StrongViewModel->SetDebugDraw(InCheckState == ECheckBoxState::Checked);
					}
				})
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString(TEXT("Draw Debug")))	
					]
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.Padding(0)
			[
				SNew(SExpandableArea)
				.AreaTitle(LOCTEXT("DebugAreaTitle", "Debug Info"))
				.AreaTitlePadding(8.0f)
				.InitiallyCollapsed(false)
				.Padding(8.0f)
				.BodyContent()
				[
					// Input Stuff
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 8.0f)
					[
						MakeRowWidget(LOCTEXT("InputRay", "Input Ray"), &FGizmoTreeViewModel::GetInputRay)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 8.0f)
					[
						MakeRowWidget(LOCTEXT("InputPosition", "Input Position"), &FGizmoTreeViewModel::GetInputPosition)
					]
				]
			]
		]
		
		+ SVerticalBox::Slot()
		.FillContentHeight(1.0f, 0.01f)
		.Padding(0.0f, 8.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.Padding(0)
			[
				SNew(SExpandableArea)
				.AreaTitle(LOCTEXT("ContextObjectsTitle", "Context Objects"))
				.AreaTitlePadding(8.0f)
				.InitiallyCollapsed(false)
				.Padding(8.0f)
				.BodyContent()
				[
					SNew(SListView<TSharedPtr<FGizmoTreeViewModel::FGizmoTreeContextObjectViewModel>>)
					.SelectionMode(ESelectionMode::None)
					.ListItemsSource(&ViewModel->GetContextObjects())
					.OnGenerateRow(this, &SGizmoTree::GenerateContextObjectRow)
				]
			]
		]

		/*
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f)
		[
			MakeLabelPartWidget(TEXT("Last Hit Part"), &FGizmoTreeViewModel::GetHitPartName)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f)
		[
			MakeLabelPartWidget(TEXT("Last Hovered Part"), &FGizmoTreeViewModel::GetLastHoveredPartName)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f)
		[
			MakeLabelPartWidget(TEXT("Last Interacted Part"), &FGizmoTreeViewModel::GetLastInteractedPartName)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f)
		[
			MakeLabelPartWidget(TEXT("Last Selected Part"), &FGizmoTreeViewModel::GetLastSelectedPartName)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f)
		[
			MakeLabelPartWidget(TEXT("Last Subdued Part"), &FGizmoTreeViewModel::GetLastSubduedPartName)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(0.0f, 8.0f)
			[
				SNew(SComboBox<TSharedPtr<FName>>)
				.OnGenerateWidget(this, &SGizmoTree::GenerateGizmoChoiceWidget)
				.OptionsSource(&GizmoChoiceNames)
				.OnSelectionChanged(this, &SGizmoTree::OnGizmoChoiceChanged)
				.InitiallySelectedItem(GizmoChoiceNames.Last())
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &SGizmoTree::GetGizmoChoiceLabel)
				]
			]
		]
		*/

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SGizmoTree::IsShowingHidden)
				.OnCheckStateChanged(this, &SGizmoTree::OnShowHiddenChanged)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Show Hidden Elements")))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SGizmoTree::IsShowingLastState)
				.OnCheckStateChanged(this, &SGizmoTree::OnShowLastStateChanged)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Show Last State")))
				]
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(TreeViewWidget, STreeView<TSharedPtr<FGizmoTreeElementViewModel>>)
			.SelectionMode(ESelectionMode::None)
			.TreeItemsSource(&TreeItems)
			.OnGetChildren(this, &SGizmoTree::GetTreeChildren)
			.OnGenerateRow(this, &SGizmoTree::GenerateTreeRow)
			.HeaderRow(HeaderRowWidget)
		]
	];
}

void SGizmoTree::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (TryUpdateTreeItems())
	{
		TreeViewWidget->RequestTreeRefresh();

		// Expand all items
		for (const TSharedPtr<FGizmoTreeElementViewModel>& Item : TreeItems)
		{
			TreeViewWidget->SetItemExpansion(Item, true);
		}
	}
	
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

TSharedRef<ITableRow> SGizmoTree::GenerateContextObjectRow(TSharedPtr<FGizmoTreeViewModel::FGizmoTreeContextObjectViewModel> InContextObject, const TSharedRef<STableViewBase>& InOwnerTable)
{
	if (InContextObject->PropertyNames.IsEmpty() && InContextObject->FunctionNames.IsEmpty())
	{
		return SNew(STableRow<TSharedPtr<FGizmoTreeViewModel::FGizmoTreeContextObjectViewModel>>, InOwnerTable)
		.Padding(FMargin(0.0f, 2.0f))
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ExpandableArea.Border"))
			.BorderBackgroundColor(FLinearColor::White)
			.Padding(8.0f)
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("NoBorder"))
				.Padding(FMargin(4.0f, 2.0f))
				[
					SNew(STextBlock)
					.Font(FCoreStyle::Get().GetFontStyle("ExpandableArea.TitleFont"))
					.Text(FText::FromName(InContextObject->Name))
				]
			]
		];
	}

	return SNew(STableRow<TSharedPtr<FGizmoTreeViewModel::FGizmoTreeContextObjectViewModel>>, InOwnerTable)
	.Padding(FMargin(0.0f, 2.0f))
	[
		SNew(SExpandableArea)
		.AreaTitle(FText::FromName(InContextObject->Name))
		.AreaTitlePadding(8.0f)
		.InitiallyCollapsed(false)
		.Padding(8.0f)
		.BodyContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(20.0f)
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SListView<FName>)
					.SelectionMode(ESelectionMode::None)
					.ListItemsSource(&InContextObject->PropertyNames)
					.OnGenerateRow_Lambda([](FName InPropertyName, const TSharedRef<STableViewBase>& OwnerTable)
					{
						return SNew(STableRow<FName>, OwnerTable)
						[
							SNew(STextBlock)
							.Text(FText::FromName(InPropertyName))
						];
					})
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SListView<FName>)
					.SelectionMode(ESelectionMode::None)
					.ListItemsSource(&InContextObject->FunctionNames)
					.OnGenerateRow_Lambda([](FName InFunctionName, const TSharedRef<STableViewBase>& OwnerTable)
					{
						return SNew(STableRow<FName>, OwnerTable)
						[
							SNew(STextBlock)
							.Text(FText::FromName(InFunctionName))
							.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 10, TEXT("Italic")))
						];
					})
				]
			]
		]
	];
}

void SGizmoTree::UpdateGizmoChoices()
{
	const FString GizmoType = TEXT("EditorTransformGizmoBuilder");
	
	GizmoChoiceNames.Reset();

	if (UEditorInteractiveGizmoManager* GizmoManager = GetGizmoManager())
	{
		TArray<UInteractiveGizmo*> GizmosOfType = GizmoManager->FindAllGizmosOfType(GizmoType);
		if (GizmosOfType.IsEmpty())
		{
			return;
		}

		for (const UInteractiveGizmo* Gizmo : GizmosOfType)
		{
			if (Gizmo)
			{
				GizmoChoiceNames.Emplace(MakeShared<FName>(Gizmo->GetFName()));
			}
		}
	}
}

FText SGizmoTree::GetGizmoChoiceLabel() const
{
	if (const UTransformGizmo* TransformGizmo = WeakTransformGizmo.Get())
	{
		return FText::FromName(TransformGizmo->GetFName());
	}

	return FText();
}

TSharedRef<SWidget> SGizmoTree::GenerateGizmoChoiceWidget(TSharedPtr<FName> InName)
{
	return SNew(STextBlock)
		.Text(FText::FromName(*InName))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void SGizmoTree::OnGizmoChoiceChanged(TSharedPtr<FName> InName, ESelectInfo::Type InSelectInfo)
{
	// @note: we just get the default transform gizmo here
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		WeakTransformGizmo = UE::EditorTransformGizmoUtil::FindDefaultTransformGizmo(ToolManager);
	}
}

ECheckBoxState SGizmoTree::IsShowingHidden() const
{
	return bShowHiddenItems ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SGizmoTree::OnShowHiddenChanged(ECheckBoxState CheckBoxState)
{
	if (CheckBoxState == ECheckBoxState::Checked
		&& !bShowHiddenItems)
	{
		bShowHiddenItems = true;
	}
	else if (CheckBoxState == ECheckBoxState::Unchecked
		&& bShowHiddenItems)
	{
		bShowHiddenItems = false;
	}

	if (TreeViewWidget.IsValid())
	{
		TreeViewWidget->RequestTreeRefresh();
	}
}

ECheckBoxState SGizmoTree::IsShowingLastState() const
{
	return bShowLastState ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SGizmoTree::OnShowLastStateChanged(ECheckBoxState CheckBoxState)
{
	if (CheckBoxState == ECheckBoxState::Checked
		&& !bShowLastState)
	{
		bShowLastState = true;
	}
	else if (CheckBoxState == ECheckBoxState::Unchecked
		&& bShowLastState)
	{
		bShowLastState = false;
	}

	if (TreeViewWidget.IsValid())
	{
		HeaderRowWidget->SetShowGeneratedColumn(UE::Editor::GizmoSettings::Private::FGizmoTreeColumns::PreviousState(), bShowLastState);
		TreeViewWidget->RequestTreeRefresh();
	}
}

bool SGizmoTree::TryUpdateTreeItems()
{
	bool bAnyItemChanged = false;

	if (const UTransformGizmo* TransformGizmo = WeakTransformGizmo.Get())
	{
		if (ensure(TransformGizmo->GizmoElementRoot))
		{
			bAnyItemChanged |= UpdateTreeItems();
		}
	}

	return bAnyItemChanged;
}

bool SGizmoTree::UpdateTreeItems()
{
	TArray<TSharedPtr<FGizmoTreeElementViewModel>> NewTreeItems;
	NewTreeItems.Reserve(TreeItems.Num());

	bool bAnyItemChanged = false;

	if (const UTransformGizmo* TransformGizmo = WeakTransformGizmo.Get())
	{
		if (ensure(TransformGizmo->GizmoElementRoot))
		{
			bAnyItemChanged |= UpdateTreeItems(TransformGizmo->GizmoElementRoot, nullptr, NewTreeItems);

			// Special case for Gimbal group
			if (const TObjectPtr<UGizmoElementBase>* FoundElement = TransformGizmo->GizmoElementRoot->GetSubElements().FindByPredicate(
				[&](const UGizmoElementBase* InElement)
				{
					return InElement->GetClass()->IsChildOf(UGizmoElementGimbal::StaticClass());
				}))
			{
				if (const UGizmoElementGimbal* GimbalElement = Cast<UGizmoElementGimbal>(*FoundElement))
				{
					bAnyItemChanged |= UpdateTreeItems(GimbalElement, nullptr, NewTreeItems);
				}
			}
		}
	}

	TreeItems = NewTreeItems;

	return bAnyItemChanged;
}

bool SGizmoTree::UpdateTreeItems(const UGizmoElementGroupBase* InGroupElement, const TSharedPtr<FGizmoTreeElementViewModel>& InParentItem, TArray<TSharedPtr<FGizmoTreeElementViewModel>>& OutChildren)
{
	if (!InGroupElement)
	{
		return false;
	}

	TArray<TSharedPtr<FGizmoTreeElementViewModel>> NewChildren;
	NewChildren.Reserve(OutChildren.Num());

	const TConstArrayView<UGizmoElementBase*> SubElements = FGizmoElementAccessor::GetSubElements(*InGroupElement);
	NewChildren.Reserve(SubElements.Num());

	bool bAnyItemChanged = SubElements.Num() != NewChildren.Num();

	for (UGizmoElementBase* SubElement : SubElements)
	{
		// Check if it already exists, if not, flag as changed
		if (const TSharedPtr<FGizmoTreeElementViewModel>* FoundExistingItem = TreeItems.FindByPredicate(
			[SubElement](const TSharedPtr<FGizmoTreeElementViewModel>& InOldItem)
		{
			return InOldItem->UniqueId == SubElement->GetUniqueID();
		});
		FoundExistingItem && FoundExistingItem->IsValid())
		{
			bAnyItemChanged |= true;
			NewChildren.Emplace(*FoundExistingItem);
		}
		else
		{
			TSharedPtr<FGizmoTreeElementViewModel> NewItem = FGizmoTreeElementViewModel::Construct(
				SubElement,
				[WeakTransformGizmo = WeakTransformGizmo](const uint32 InPartId) -> bool
				{
					using namespace UE::Editor::InteractiveToolsFramework;
					using namespace UE::Editor::GizmoSettings::Private;

					if (UTransformGizmo* StrongTransformGizmo = WeakTransformGizmo.Get())
					{
						FTransformGizmoAccessor Accessor;
						return Accessor.IsPartSelected(*StrongTransformGizmo, static_cast<ETransformGizmoPartIdentifier>(InPartId)); 
					}
					else
					{
						UE_LOGF(LogTemp, Warning, "WeakTransformGizmo invalid!");
					}

					return false;
				},
				InParentItem);

			NewChildren.Emplace(NewItem);
		}
	}

	OutChildren.Append(NewChildren);

	return bAnyItemChanged;
}

void SGizmoTree::GetTreeChildren(TSharedPtr<FGizmoTreeElementViewModel> InViewModel, TArray<TSharedPtr<FGizmoTreeElementViewModel>>& OutChildren)
{
	if (bShowHiddenItems)
	{
		OutChildren.Append(InViewModel->GetChildren());	
	}
	else
	{
		Algo::CopyIf(InViewModel->GetChildren(), OutChildren, [](const TSharedPtr<FGizmoTreeElementViewModel>& InItem)
		{
			return InItem->IsVisible();
		});	
	}
}

TSharedRef<ITableRow> SGizmoTree::GenerateTreeRow(TSharedPtr<FGizmoTreeElementViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	TSharedRef<SGizmoTreeTableRow> Item =
		SNew(SGizmoTreeTableRow, InOwnerTable)
		.Item(InViewModel)
		.Visibility_Lambda([this, WeakItem = InViewModel.ToWeakPtr()]()
		{
			if (TSharedPtr<FGizmoTreeElementViewModel> StrongItem = WeakItem.Pin())
			{
				return bShowHiddenItems || StrongItem->IsVisible()  ? EVisibility::Visible : EVisibility::Collapsed;
			}

			return EVisibility::Visible;
		});

	return Item;
}

UInteractiveToolManager* SGizmoTree::GetToolManager() const
{
	UInteractiveToolManager* ToolManager = Cast<UInteractiveToolManager>(GLevelEditorModeTools().GetInteractiveToolsContext()->ToolManager);
	if (!ToolManager)
	{
		return nullptr;
	}

	return ToolManager;
}

UEditorInteractiveGizmoManager* SGizmoTree::GetGizmoManager() const
{
	UEditorInteractiveGizmoManager* GizmoManager = Cast<UEditorInteractiveGizmoManager>(GLevelEditorModeTools().GetInteractiveToolsContext()->GizmoManager);
	if (!GizmoManager)
	{
		return nullptr;
	}

	return GizmoManager;
}

#undef LOCTEXT_NAMESPACE
