// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoElementGroup.h"
#include "IStructureDetailsView.h"
#include "Misc/NotifyHook.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

class UTransformGizmo;
class UGizmoElementGroupBase;
class UGizmoElementBase;
class UEditorInteractiveGizmoManager;
class SHeaderRow;

class FGizmoTreeElementViewModel : public TSharedFromThis<FGizmoTreeElementViewModel>
{
	friend class SGizmoTree;
	
public:
	explicit FGizmoTreeElementViewModel() = default;

	static TSharedPtr<FGizmoTreeElementViewModel> Construct(const UGizmoElementBase* InGizmoElement, const TFunction<bool(const uint32)>& InGetSelectedFunc, const TSharedPtr<FGizmoTreeElementViewModel>& InParentViewModel = nullptr);

	const int32 GetLevel() const;

	const FLinearColor GetColor() const;

	const FName GetPartName() const;

	const bool IsPartInherited() const;

	const FString GetName() const;

	const FName GetStateName() const;

	const FLinearColor GetStateColor() const;

	const FName GetPreviousStateName() const;

	const FLinearColor GetPreviousStateColor() const;

	const bool IsStateInherited() const;

	const bool IsVisible() const;

	const bool IsSelected() const;

	const FLinearColor GetCurrentVertexColor() const;

	const FLinearColor GetCurrentLineColor() const;

	const FString GetCurrentMaterialName() const;

	TConstArrayView<TSharedPtr<FGizmoTreeElementViewModel>> GetChildren() const;

	TFunction<bool(const uint32)> GetSelectedFunc;

private:
	void PopulateChildren();

	static FString ResolveOwningPropertyName(const UGizmoElementBase* InElement);

	const FLinearColor GetStateColorInternal(const EGizmoElementInteractionState InState) const;

private:
	TWeakObjectPtr<const UGizmoElementBase> WeakGizmoElement;
	TWeakObjectPtr<const UGizmoElementLineBase> WeakLineElement;
	TWeakPtr<FGizmoTreeElementViewModel> WeakParent;
	TArray<TSharedPtr<FGizmoTreeElementViewModel>> Children;

	uint32 UniqueId = 0;
	int32 Level;
	FLinearColor Color;
	FString OwningPropertyName;
	mutable EGizmoElementInteractionState CurrentState = EGizmoElementInteractionState::None;
	mutable EGizmoElementInteractionState PreviousState = EGizmoElementInteractionState::None;
};

class FGizmoTreeViewModel : public TSharedFromThis<FGizmoTreeViewModel>
{
public:
	class FGizmoTreeContextObjectViewModel : public TSharedFromThis<FGizmoTreeContextObjectViewModel>
	{
	public:
		static TSharedPtr<FGizmoTreeContextObjectViewModel> Construct(const UObject* InContextObject);

		FName Name;
		TArray<FName> PropertyNames;
		TArray<FName> FunctionNames;
	};

public:
	explicit FGizmoTreeViewModel() = default;

	~FGizmoTreeViewModel();

	static TSharedPtr<FGizmoTreeViewModel> Construct(const UTransformGizmo* InGizmo);

	const FName GetHitPartName() const;

	const FName GetLastSelectedPartName() const;
	const FName GetLastHoveredPartName() const;
	const FName GetLastInteractedPartName() const;
	const FName GetLastSubduedPartName() const;
	const FText GetInputRay() const;
	const FText GetInputPosition() const;
	const TArray<TSharedPtr<FGizmoTreeContextObjectViewModel>>& GetContextObjects() const;

	bool IsDebugDrawing() const;
	void SetDebugDraw(const bool bInValue);
	void OnDebugDrawChanged(IConsoleVariable* InCVar);

private:
	TWeakObjectPtr<const UTransformGizmo> WeakGizmo;
	std::atomic_bool bIsDebugDrawing;
	TArray<TSharedPtr<FGizmoTreeContextObjectViewModel>> ContextObjects;
};

class SGizmoTreeTableRow : public SMultiColumnTableRow<TSharedPtr<FGizmoTreeElementViewModel>>
{
public:
	SLATE_BEGIN_ARGS(SGizmoTreeTableRow)
	{}
		SLATE_ARGUMENT(TSharedPtr<FGizmoTreeElementViewModel>, Item)
	SLATE_END_ARGS()

	/** Construct function for this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InTreeView);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

private:
	TSharedPtr<FGizmoTreeElementViewModel> Item;
};

class SGizmoTree
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGizmoTree)
	{ }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	TSharedRef<ITableRow> GenerateContextObjectRow(TSharedPtr<FGizmoTreeViewModel::FGizmoTreeContextObjectViewModel> InContextObjectName, const TSharedRef<STableViewBase>& InOwnerTable);

	void UpdateGizmoChoices();
	FText GetGizmoChoiceLabel() const;
	TSharedRef<SWidget> GenerateGizmoChoiceWidget(TSharedPtr<FName> InName);
	void OnGizmoChoiceChanged(TSharedPtr<FName> InName, ESelectInfo::Type InSelectInfo);

	ECheckBoxState IsShowingHidden() const;
	void OnShowHiddenChanged(ECheckBoxState CheckBoxState);

	ECheckBoxState IsShowingLastState() const;
	void OnShowLastStateChanged(ECheckBoxState CheckBoxState);
	
	bool TryUpdateTreeItems();
	bool UpdateTreeItems();
	bool UpdateTreeItems(const UGizmoElementGroupBase* InGroupElement, const TSharedPtr<FGizmoTreeElementViewModel>& InParentItem, TArray<TSharedPtr<FGizmoTreeElementViewModel>>& OutChildren);
	void GetTreeChildren(TSharedPtr<FGizmoTreeElementViewModel> InViewModel, TArray<TSharedPtr<FGizmoTreeElementViewModel>>& OutChildren);
	TSharedRef<ITableRow> GenerateTreeRow(TSharedPtr<FGizmoTreeElementViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	UInteractiveToolManager* GetToolManager() const;
	UEditorInteractiveGizmoManager* GetGizmoManager() const;

private:
	TSharedPtr<SHeaderRow> HeaderRowWidget;
	TSharedPtr<STreeView<TSharedPtr<FGizmoTreeElementViewModel>>> TreeViewWidget;
	TArray<TSharedPtr<FName>> GizmoChoiceNames;
	bool bShowHiddenItems = false;
	bool bShowLastState = false;
	TArray<TSharedPtr<FGizmoTreeElementViewModel>> TreeItems;
	TSharedPtr<FGizmoTreeViewModel> ViewModel;
	TWeakObjectPtr<UTransformGizmo> WeakTransformGizmo;
};
