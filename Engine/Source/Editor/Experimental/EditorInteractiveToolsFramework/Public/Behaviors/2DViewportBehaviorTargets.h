// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "VectorTypes.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

// TODO: Replace with a context object to avoid dependency on FEditorViewportClient (JIRA UE-163677)
class FEditorViewportClient;

/**
 * Allows click-dragging to move the camera in the XY plane.
 */
class FEditor2DScrollBehaviorTarget : public IClickDragBehaviorTarget
{
public:

	UE_API FEditor2DScrollBehaviorTarget(FEditorViewportClient* ViewportClientIn);

	// IClickDragBehaviorTarget
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	UE_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	UE_API virtual void OnTerminateDragSequence() override;

protected:

	// TODO: Replace with a context object to avoid dependency on FEditorViewportClient (JIRA UE-163677)
	FEditorViewportClient* ViewportClient = nullptr;
	FVector3d OriginalCameraLocation;
	FVector3d DragStart;
};

/**
 * Allows the mouse wheel to move the camera forwards/backwards relative to the XY plane,
 * in the direction pointed to by the mouse.
 */
class FEditor2DMouseWheelZoomBehaviorTarget : public IMouseWheelBehaviorTarget
{
public:

	UE_API FEditor2DMouseWheelZoomBehaviorTarget(FEditorViewportClient* ViewportClientIn);

	// IMouseWheelBehaviorTarget
	UE_API virtual FInputRayHit ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos);
	UE_API virtual void OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos);
	UE_API virtual void OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos);
	
	/**
	 * @param PercentZoomIn How much to move forward on each mouse wheel forward scroll. For instance,
	 *  passing 20 here will decrease the distance to the XY plane by 20% each time. The zoom out amount
	 *  will be set in such a way that it undoes the same effect. For instance if 20 was passed here,
	 *  zoom out amount will be 25% since zooming in by 20% and then out by 25% of the result gets you
	 *  back to where you were.
	 */
	UE_API virtual void SetZoomAmount(double PercentZoomIn);
	UE_API void SetZoomLimits(double ZoomInLimitIn, double ZoomOutLimitIn);

	UE_API void SetCameraFarPlaneWorldZ(double CameraFarPlaneWorldZIn);
	UE_API void SetCameraNearPlaneProportionZ(double CameraFarPlaneProportionZIn );

	inline static const double DEFAULT_ZOOM_AMOUNT = 20;
protected:

	// TODO: Replace with a context object to avoid dependency on (JIRA UE-163677)
	FEditorViewportClient* ViewportClient = nullptr;
	double ZoomInProportion;
	double ZoomOutProportion;
	double ZoomInLimit;
	double ZoomOutLimit;
	double CameraFarPlaneWorldZ;
	double CameraNearPlaneProportionZ;
};

#undef UE_API
