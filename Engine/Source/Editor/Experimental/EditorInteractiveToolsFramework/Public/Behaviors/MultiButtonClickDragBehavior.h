// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/ClickDragBehavior.h"

#include "MultiButtonClickDragBehavior.generated.h"

class UObject;

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/**
 * MultiButtonClickDragBehavior is an implementation of UClickDragInputBehavior that can manage several
 * mouse buttons at once and also implements IClickDragBehaviorTarget directly, via a set of local lambda functions.
 * To use/customize this class the client replaces the lambda functions with their own and enables/disables the
 * buttons to capture. 
 */

UCLASS(MinimalAPI)
class UMultiButtonClickDragBehavior : public UClickDragInputBehavior, public IClickDragBehaviorTarget
{
	GENERATED_BODY()
protected:
	using UClickDragInputBehavior::Initialize;
	
public:

	// UClickDragInputBehavior  implementation
	UE_API void Initialize();

	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& InPressPos) override;
	virtual void OnClickPress(const FInputDeviceRay& InPressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& InDragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& InReleasePos) override;
	virtual void OnTerminateDragSequence() override;
	
	// UInputBehavior implementation

	virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
	virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& Input, EInputCaptureSide Side) override;
	virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data) override;

	/** lambda implementation of CanBeginClickDragSequence */
	TUniqueFunction<FInputRayHit(const FInputDeviceRay& PressPos)> CanBeginClickDragFunc = [](const FInputDeviceRay&) { return FInputRayHit(); };

	/** lambda implementation of OnClickPress */
	TUniqueFunction<void(const FInputDeviceRay& PressPos)> OnClickPressFunc = [](const FInputDeviceRay&) {};

	/** lambda implementation of OnClickDrag */
	TUniqueFunction<void(const FInputDeviceRay& PressPos)> OnClickDragFunc = [](const FInputDeviceRay&) {};

	/** lambda implementation of OnClickRelease */
	TUniqueFunction<void(const FInputDeviceRay& ReleasePos)> OnClickReleaseFunc = [](const FInputDeviceRay&) {};

	/** lambda implementation of OnTerminateDragSequence */
	TUniqueFunction<void()> OnTerminateFunc = []() {};
	
	/** lambda implementation of OnStateUpdated */
	TUniqueFunction<void(const FInputDeviceState& Input)> OnStateUpdated = [](const FInputDeviceState&) {};

	/** enable InButton so that in can be captured */
	UE_API void EnableButton(const FKey& InButton);
	
	/** disable InButton so that it's not captured */
	UE_API void DisableButton(const FKey& InButton);

protected:

	/** @return true if any captured button is down. */
	bool IsAnyButtonDown(const FInputDeviceState& InInput);

	/** @return true if any captured button changed its bPressed/bReleased state. */
	bool DidAnyButtonChangeState(const FInputDeviceState& InInput) const;
	
	/** @return true if left mouse button is handled. */
	bool HandlesLeftMouseButton() const;
	
	/** @return true if middle mouse button is handled. */
	bool HandlesMiddleMouseButton() const;
	
	/** @return true if right mouse button is handled. */
	bool HandlesRightMouseButton() const;

	/** mouse buttons that need to be captured. Left=0, Middle=1, Right=2*/
	uint8 CapturedInputs[3] = {0 ,0, 0};
};

#undef UE_API
