// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/AnyButtonInputBehavior.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/InputBehaviorModifierStates.h"

#include "SingleClickAndDragBehavior.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/**
 * Functions required to apply standard "Click" and "Click-Drag" state machines to a target object.
 * Differs to SingleClickOrDragBehavior in that it provides a separate call sequence for click press and click release.
 */
class ISingleClickAndDragBehaviorTarget : public IModifierToggleBehaviorTarget
{
public:
	virtual ~ISingleClickAndDragBehaviorTarget() override = default;

	/**
	 * Test if target can begin click-drag interaction at this point
	 * @param InPressPos device position/ray at click point
	 * @return hit information at this point
	 */
	virtual FInputRayHit CanBeginSingleClickAndDragSequence(const FInputDeviceRay& InPressPos) = 0;

	/**
	 * Notify Target that click press ocurred
	 * @param InPressPos device position/ray at click point
	 */
	virtual void OnClickPress(const FInputDeviceRay& InPressPos) = 0;

	/**
	 * Notify Target that the drag process has started
	 * @param InPressPos device position/ray at move point
	 */
	virtual void OnDragStart(const FInputDeviceRay& InDragPos) = 0;

	/**
	 * Notify Target that input position has changed
	 * @param InPressPos device position/ray at click point
	 */
	virtual void OnClickDrag(const FInputDeviceRay& InDragPos) = 0;

	/**
	 * Notify Target that click release occurred
	 * @param InPressPos device position/ray at click point
	 */
	virtual void OnClickRelease(const FInputDeviceRay& InReleasePos, bool bInIsDragOperation) = 0;

	/**
	 * Notify Target that click-drag sequence has been explicitly terminated (eg by escape key)
	 */
	virtual void OnTerminateSingleClickAndDragSequence() = 0;
};

/**
 * USingleClickAndDragInputBehavior implements a combination of "button-click" and "button-click-drag"-style input behavior.
 * If the mouse is moved away from the original location, and update will occur and a drag operation will start. If the mouse
 * is released without moving too far away, a click event will occur. Once the drag has started, returning to the original
 * location will not produce a click event will not be produced, but the drag operation will continue.
 *
 * NOTE: This differs from USingleClickOrDragInputBehavior in that it provides a separate call sequence for click and drag:
 * Single Clicks:
 *	OnClickPress
 *	OnClickRelease
 *
 * Drag:
 *	OnDragStarted
 *	OnClickDrag
 *
 * An ISingleClickAndDragBehaviorTarget instance must be provided which is manipulated by this behavior.
 *
 * The state machine works as follows:
 *    1) on input-device-button-press, call Target::CanBeginSingleClickDragSequence to determine if capture should begin
 *    2) on input-device-move, call Target::OnClickDrag if drag mode has been started or Target::OnDragStart if it crosses that
 *	       threshold.
 *    3) on input-device-button-release, call Target::OnClickRelease
 *
 * If a ForceEndCapture occurs we call Target::OnTerminateDragClickSequence
 */
UCLASS(MinimalAPI)
class USingleClickAndDragBehavior : public UAnyButtonInputBehavior
{
	GENERATED_BODY()

public:
	/**
	 * Set the targets for Click and Drag interactions
	 */
	UE_API virtual void Initialize(ISingleClickAndDragBehaviorTarget* InTarget);

	/**
	 * WantsCapture() will only return capture request if this function returns true (or is null)
	 */
	TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc = nullptr;

	virtual FInputCapturePriority GetPriority() override;
	virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& InInputState) override;
	virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& InInputState, EInputCaptureSide InCaptureSide) override;
	virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& InInputState, const FInputCaptureData& InCaptureData) override;
	virtual void ForceEndCapture(const FInputCaptureData& InCaptureData) override;
	
	virtual FInputDeviceRay GetDeviceRay(const FInputDeviceState& InInputDeviceState) override;

	UE_API bool UsesUnboundedCursor() const;
	UE_API void SetUsesUnboundedCursor(TAttribute<bool> InUsesUnboundedCursor);
	
	UE_API void SetPendingDragPriority(TOptional<FInputCapturePriority> InPriority);

public:
	/**
	* The modifier set for this behavior
	*/
	FInputBehaviorModifierStates Modifiers;

	/** If true (default), then if the click-mouse-down does not hit a valid click target (determined by IClickBehaviorTarget::IsHitByClick), then the Drag will be initiated */
	UPROPERTY()
	bool bBeginDragIfClickTargetNotHit = true;


	/** If the device moves more than this distance in 2D (pixel?) units, the interaction switches from click to drag */
	UPROPERTY()
	float DragDistanceThreshold = 2.0;

protected:
	/**
	 * Internal function that forwards click evens to Target::OnClickPress, you can customize behavior here
	 */
	virtual void OnClickPressInternal(const FInputDeviceState& InInput, EInputCaptureSide InSide);

	/**
	 * Internal function that forwards click evens to Target::OnDragStarted, you can customize behavior here
	 */
	virtual void OnDragStartedInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData);

	/**
	 * Internal function that forwards click evens to Target::OnClickDrag, you can customize behavior here
	 */
	virtual void OnClickDragInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData);

	/**
	 * Internal function that forwards click evens to Target::OnClickRelease, you can customize behavior here
	 */
	virtual void OnClickReleaseInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData);

private:
	void UpdateDragDistance(const FInputDeviceState& InInputState);

	/** Click and Drag Target object. */
	ISingleClickAndDragBehaviorTarget* Target = nullptr;

	/** The initial mouse down position. */
	TOptional<FInputDeviceRay> InitialMouseDownRay;
	
	/** When set, the drag priority used when mouse motion has not yet moved byond the minimum drag distance. */	
	TOptional<FInputCapturePriority> PendingDragPriority;

	/** Device capture */
	EInputCaptureSide CaptureSide;
	
	/** Updated to mark whether mouse movement has moved beyond drag distance */
	bool bIsBeyondDragDistance = false;

	/** set to true if we are in an active drag capture, eg after rejecting possible click */
	bool bIsDragging = false;

	/** 
	 * set to true between BeginCapture and capture end (release/force-end). Guards WantsCapture from
	 * resetting InitialMouseDownRay when called by CollectWantsCapture during active capture.
	 */
	bool bHasActiveCapture = false;

	// flag used to communicate between WantsCapture and BeginCapture
	bool bImmediatelyBeginDragInBeginCapture = false;

	// flag used to communicate between WantsCapture and BeginCapture
	bool bImmediatelyClickInBeginCapture = false;
	
	TAttribute<bool> UseUnboundedCursor;
};

/**
 * A version of USingleCLickAndDragBehavior that allows for arbitrary function pointers to handle the ISingleClickAndDragBehaviorTarget callbacks.
 */
UCLASS(MinimalAPI)
class ULocalSingleClickAndDragBehavior : public USingleClickAndDragBehavior, public ISingleClickAndDragBehaviorTarget
{
	GENERATED_BODY()

public:
	// ISingleClickAndDragBehaviorTarget
	
	/**
	 * Initializes the behavior with a fallback target. Any null function pointers will fall back to the provided target.
	 */
	virtual void Initialize(ISingleClickAndDragBehaviorTarget* InTarget) override;
	virtual FInputRayHit CanBeginSingleClickAndDragSequence(const FInputDeviceRay& InPressPos) override;
	virtual void OnClickPress(const FInputDeviceRay& InPressPos) override;
	virtual void OnDragStart(const FInputDeviceRay& InDragPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& InDragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& InReleasePos, bool bInIsDragOperation) override;
	virtual void OnTerminateSingleClickAndDragSequence() override;
	// End
	
	TUniqueFunction<FInputRayHit(const FInputDeviceRay& InPressPos)> CanBeginSingleClickAndDragSequenceFunc = nullptr;
	TUniqueFunction<void(const FInputDeviceRay& InDragPos)> OnClickPressFunc = nullptr;
	TUniqueFunction<void(const FInputDeviceRay& InDragPos)> OnDragStartFunc = nullptr;
	TUniqueFunction<void(const FInputDeviceRay& InDragPos)> OnClickDragFunc = nullptr;
	TUniqueFunction<void(const FInputDeviceRay& InDragPos, bool bInIsDragOperation)> OnClickReleaseFunc = nullptr;
	TUniqueFunction<void()> OnTerminateSingleClickAndDragSequenceFunc = nullptr;
	
private:
	ISingleClickAndDragBehaviorTarget* FallbackTarget = nullptr;
};

#undef UE_API
