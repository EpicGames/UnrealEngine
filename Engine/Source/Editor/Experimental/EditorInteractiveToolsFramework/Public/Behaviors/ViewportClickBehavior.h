// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "ViewportInteractions/ViewportInteractionBindings.h"

#include "ViewportClickBehavior.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class IViewportClickBehaviorTarget
{
public:
	virtual ~IViewportClickBehaviorTarget() = default;

	/**
	 * Test if target can begin the click interaction at this point
	 * @param PressPos device position/ray at click point
	 * @return hit information at this point
	 */
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& PressPos) = 0;

	/**
	 * Notify the target that the click has started
	 */
	virtual void OnClickDown(const FInputDeviceRay& InClickPos) = 0;

	/**
	 * Notify the target that the click has completed
	 */
	virtual void OnClickUp(const FInputDeviceRay& InClickPos) = 0;

	/**
	 * Notify Target button state has changed
	 */
	virtual void OnStateUpdated(const FInputDeviceState& InInputDeviceState) = 0;

	/**
	 * Notify Target that capture is ending
	 */
	virtual void OnForceEndCapture()
	{
	}
};

/** A Click behavior designed to compete alongside UViewportClickDragBehaviors */
UCLASS(MinimalAPI)
class UViewportClickBehavior : public UInputBehavior
{
	GENERATED_BODY()
	
public:
	UE_API virtual void Initialize(IViewportClickBehaviorTarget* InTarget);

	UE_API virtual FInputCapturePriority GetPriority() override;
	UE_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& InputState) override;
	UE_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide) override;
	UE_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& InputState, const FInputCaptureData& CaptureData) override;
	
	UE_API virtual void ForceEndCapture(const FInputCaptureData& CaptureData) override;
	
	UE_API void SetBindings(const TArray<UE::Editor::ViewportInteractions::FButtonBinding>& InBindings);
	
	/** @return current 3D world ray and optional 2D position for Target Device */
	UE_API virtual FInputDeviceRay GetDeviceRay(const FInputDeviceState& InInputDeviceState);

	UE_API virtual const FVector2D& GetMousePosition(const FInputDeviceState& InInputDeviceState) const;
	
protected:
	UE_API void OnStateUpdatedInternal(const FInputDeviceState& InInputDeviceState);
	UE_API void OnClickDownInternal(const FInputDeviceState& InputState);
	UE_API void OnClickUpInternal(const FInputDeviceState& InputState);
	
	UE_API virtual TOptional<FInputCapturePriority> GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage Stage, const FInputDeviceState& InputState) const;

	IViewportClickBehaviorTarget* Target = nullptr;
	UE::Editor::ViewportInteractions::FButtonBindings Bindings;
	
	TOptional<FInputCapturePriority> CachedCapturePriority;
	bool bIsClicking = false;
};

#undef UE_API