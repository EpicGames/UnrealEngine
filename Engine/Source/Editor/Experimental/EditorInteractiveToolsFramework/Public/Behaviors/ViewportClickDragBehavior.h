// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MultiButtonClickDragBehavior.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "ViewportInteractions/ViewportInteractionBindings.h"
#include "ViewportClickDragBehavior.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class IViewportClickDragBehaviorTarget
{
public:
	enum class EEndCaptureReason
	{
		/** Ended as part of normal operation, e.g. releasing a mouse button. */
		End,
		/** Canceled by explicit user action, e.g. pressing Escape. */
		Canceled,
		/** Canceled by the system. */
		Forced
	};
	
	struct FDragArgs
	{
		FInputDeviceRay Ray;
		FVector2D ScreenDelta;
	};

	virtual ~IViewportClickDragBehaviorTarget() = default;

	/**
	 * Test if target can begin click-drag interaction at this point
	 * @param PressPos device position/ray at click point
	 * @return hit information at this point
	 */
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) = 0;

	/**
	 * Notify Target that capture has started
	 * @param InClickPressPos device position/ray at click press point
	 */
	virtual void OnBeginCapture(const FInputDeviceRay& InClickPressPos) {}

	/**
	 * Notify Target that drag has started
	 * @param InDragStartPos device position/ray at drag start point
	 */
	virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) = 0;

	/**
	 * Notify Target that input position has changed
	 * @param DragPos device position/ray at click point
	 */
	virtual void OnDrag(const FDragArgs& InDrag) = 0;

	/**
	 * Notify Target that drag has ended
	 * @param InDragEndPos device position/ray at drag end point
	 */
	virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) = 0;

	/**
	 * Notify Target button state has changed
	 * @param InInputDeviceState current device position/ray
	 */
	virtual void OnStateUpdated(const FInputDeviceState& InInputDeviceState) = 0;

	/**
	 * Notify Target that capture has ended.
	 */
	virtual void OnEndCapture(EEndCaptureReason InReason) {}
};

/**
 * A behavior dedicated to Viewport Drag Interactions, directly supporting threshold-based capture.
 * It can be used when capture stealing e.g. a single click behavior. The
 * IViewportClickDragBehaviorTarget of this behavior will receive OnDragStart and OnDragEnd when drag
 * starts/ends, and not on actual button press/release
 */
UCLASS(MinimalAPI)
class UViewportClickDragBehavior : public UInputBehavior
{
	GENERATED_BODY()
public:

	DECLARE_DELEGATE_RetVal(const UE::Editor::ViewportInteractions::FButtonBindings&, FGetBindingsDelegate);

	UE_API void Initialize(IViewportClickDragBehaviorTarget* InTarget);

	/**
	 * Set the priority this Drag Behavior will return from GetPriority once the current interaction is confirmed to be a drag.
	 * Should be higher than its DefaultPriority, and higher than the priority of any other behavior it wants to steal capture from
	 */
	UE_API void SetDragConfirmedPriority(FInputCapturePriority InDragConfirmedPriority);

	UE_API virtual FInputCapturePriority GetPriority() override;

	UE_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& InputState) override;
	UE_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide) override;
	UE_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& InputState, const FInputCaptureData& CaptureData) override;
	UE_API virtual void ForceEndCapture(const FInputCaptureData& CaptureData) override;
	
	UE_API void SetBindings(const TArray<UE::Editor::ViewportInteractions::FButtonBinding>& InBindings);
	UE_API void SetBindings(const FGetBindingsDelegate& InDelegate);

	/** @return current 3D world ray and optional 2D position for Target Device */
	UE_API virtual FInputDeviceRay GetDeviceRay(const FInputDeviceState& InInputDeviceState);

	/** Extract the current mouse position from the Input Device State */
	UE_API virtual const FVector2D& GetMousePosition(const FInputDeviceState& InInputDeviceState) const;

	UE_API virtual bool UsesUnboundedCursor() const;

	/**
	 * Call to set the default unbounded cursor attribute.
	 */
	UE_API void SetUsesUnboundedCursor(TAttribute<bool> InUsesUnboundedCursor);

	/**
	 * Call to set whether drag starts should be contingent on the distance threshold when this behavior has capture.
	 * Default false. When false, this behavior will immediately start the drag operation when it wins capture.
	 */
	UE_API void SetRequireDistanceThresholdOnCapture(TAttribute<bool> InRequireDistanceThresholdOnCapture);

	/**
	 * Reset drag state and tracked drag distance. Calls OnDragEnd() but does not exit the behavior.
	 */
	UE_API void ResetDrag(const FInputDeviceState& InputState);

	bool IsDragging() const { return bDragging; }
	FVector2D GetMouseDownPosition() const { return MouseDownPosition; }
	
protected:
	
	UE_API virtual void OnDragStartInternal(const FInputDeviceState& InputState);
	UE_API virtual void OnClickDragInternal(
		const FInputDeviceState& InputState, const FInputCaptureData& Data
	);
	UE_API virtual void OnDragEndInternal(
		const FInputDeviceState& InputState, const FInputCaptureData& Data
	);

	virtual void OnStateUpdatedInternal(const FInputDeviceState& InInputDeviceState);
	
	bool HasMovedEnoughForDrag() const;
	const UE::Editor::ViewportInteractions::FButtonBindings& GetBindings() const;
	UE_API virtual TOptional<FInputCapturePriority> GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage Stage, const FInputDeviceState& InputState) const;
	
	/** Distance after which a drag is considered such */
	float DragDistanceThreshold = 2.0f;
	
	/** Cached cursor position */
	FVector2D MouseDownPosition;

	TOptional<float> MouseTraveledDistance;

	/**
	 * This is essentially a hack. When this behavior knows it doesn't want an input event, it falls back to this priority.
	 * This works around behavioral awkwardness with how capture stealing is currently implemented.
	 */
	FInputCapturePriority NoCapturePriority = FInputCapturePriority(INT32_MAX);

	/**
	 * The priority this Drag Behavior will return from GetPriority once the current interaction is confirmed to be a drag.
	 * Should be higher than its DefaultPriority, and higher than the priority of any other behavior it wants to steal capture from
	 */
	FInputCapturePriority DragConfirmedPriority;

	IViewportClickDragBehaviorTarget* Target = nullptr;

	UE::Editor::ViewportInteractions::FButtonBindings Bindings;
	FGetBindingsDelegate GetBindingsDelegate;
	
	TOptional<FInputCapturePriority> CachedCapturePriority;

	/** True when capturing and dragging */
	bool bDragging = false;

	TAttribute<bool> UnboundedCursor;
	TAttribute<bool> RequireDistanceThresholdOnCapture;
};

/**
 * An implementation of UViewportClickDragBehavior that also implements IViewportClickDragBehaviorTarget directly, using
 * a set of local lambda functions.
 */
UCLASS(MinimalAPI)
class ULocalViewportClickDragBehavior
	: public UViewportClickDragBehavior
	, public IViewportClickDragBehaviorTarget
{
	GENERATED_BODY()
protected:
	using UViewportClickDragBehavior::Initialize;

public:
	/** Call this to initialize the class */
	virtual void Initialize()
	{
		Initialize(this);
	}

	/** lambda implementation of CanBeginClickDragSequence */
	TUniqueFunction<FInputRayHit(const FInputDeviceRay& PressPos)> CanBeginClickDragFunc = [](const FInputDeviceRay&)
	{
		return FInputRayHit();
	};

	/** lambda implementation of OnDragStart */
	TUniqueFunction<void(const FInputDeviceRay& PressPos)> OnDragStartFunc = [](const FInputDeviceRay&)
	{
	};

	/** lambda implementation of OnClickDrag */
	TUniqueFunction<void(const FDragArgs& PressPos)> OnDragFunc = [](const FDragArgs&)
	{
	};

	/** lambda implementation of OnDragEnd */
	TUniqueFunction<void(const FInputDeviceRay& ReleasePos)> OnDragEndFunc = [](const FInputDeviceRay&)
	{
	};

	/** lambda implementation of OnTerminateDragSequence */
	TUniqueFunction<void(EEndCaptureReason)> OnEndCaptureFunc = [](EEndCaptureReason Reason)
	{
	};

	/** lambda implementation of OnStateUpdated */
	TUniqueFunction<void(const FInputDeviceState& Input)> OnStateUpdatedFunc = [](const FInputDeviceState&)
	{
	};

	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override
	{
		return CanBeginClickDragFunc(PressPos);
	}

	virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) override
	{
		OnDragStartFunc(InDragStartPos);
	}

	virtual void OnDrag(const FDragArgs& DragPos) override
	{
		OnDragFunc(DragPos);
	}

	virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) override
	{
		OnDragEndFunc(InDragEndPos);
	}

	virtual void OnStateUpdated(const FInputDeviceState& InInputDeviceState) override
	{
		OnStateUpdatedFunc(InInputDeviceState);
	}
	
	virtual void OnEndCapture(EEndCaptureReason InReason) override
	{
		OnEndCaptureFunc(InReason);	
	}
};

#undef UE_API
