// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/KeyAsModifierInputBehavior.h"
#include "BaseBehaviors/KeyInputBehavior.h"

#include "ViewportQuickToggleInputBehavior.generated.h"

class IQuickToggleBehaviorTarget : public IKeyInputBehaviorTarget
{
public:
	virtual ~IQuickToggleBehaviorTarget() override = default;

	virtual void OnQuickToggle(const FKey& InKey) {}
};

UCLASS(MinimalAPI)
class UViewportQuickToggleInputBehavior : public UKeyInputBehavior
{
	GENERATED_BODY()

public:
	EDITORINTERACTIVETOOLSFRAMEWORK_API UViewportQuickToggleInputBehavior() {}

	virtual EInputDevices GetSupportedDevices() override
	{
		return EInputDevices::Keyboard;
	}

	EDITORINTERACTIVETOOLSFRAMEWORK_API void Initialize(IQuickToggleBehaviorTarget* InTarget, const FKey& InKey);
	EDITORINTERACTIVETOOLSFRAMEWORK_API void Initialize(IQuickToggleBehaviorTarget* InTarget, const TArray<FKey>& InKeys);

	virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& InputState) override;
	virtual bool WantsForceEndCapture() override;
	virtual void ForceEndCapture(const FInputCaptureData& CaptureData) override;

	virtual void OnKeyPressedInternal(const FInputDeviceState& Input) override;
	virtual void OnKeyReleasedInternal(const FInputDeviceState& Input) override;

	/** Can be used to specify if this behavior target should receive OnQuickToggle events or not */
	TFunction<bool(const FInputDeviceState&)> CanQuickToggleFunc = nullptr;

private:
	virtual void Initialize(IKeyInputBehaviorTarget* InTarget, const FKey& InKey) override {}
	virtual void Initialize(IKeyInputBehaviorTarget* InTarget, const TArray<FKey>& InKeys) override {}

	bool CanQuickToggle(const FInputDeviceState& Input) const;

	IQuickToggleBehaviorTarget* QuickToggleTarget;

	TArray<double> KeyPressedTime;
	double QuickToggleDelay = 0.2f;
};
