// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Misc/CoreMiscDefines.h"

// Moved to UnrealEd module
UE_DEPRECATED_HEADER(5.6, "Use Tools/EditorInteractiveToolsContext.h instead of EdModeInteractiveToolsContext.h.")
#include "Tools/EdModeInteractiveToolsContext.h"
