// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragToolInteraction.h"
#include "MarqueeSelectInteraction.h"

#include "BoxSelectInteraction.generated.h"

class FCanvas;
class UModel;

UCLASS(MinimalAPI, Transient)
class UBoxSelectInteraction : public UMarqueeSelectInteraction
{
	GENERATED_BODY()

public:
	UBoxSelectInteraction();

	//~ Begin IViewportClickDragBehaviorTarget
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& InPressPos) override;
	virtual void OnDragStart(const FInputDeviceRay& InPressPos) override;
	virtual void OnDrag(const FDragArgs& InDrag) override;
	virtual void OnDragEnd(const FInputDeviceRay& InReleasePos) override;
	//~ End IViewportClickDragBehaviorTarget

protected:
	virtual TArray<FEditorModeID> GetUnsupportedModes() const override;

private:
	/**
	 * Returns true if the provided BSP node intersects with the provided frustum
	 *
	 * @param InModel				The model containing BSP nodes to check
	 * @param InNodeIndex				The index to a BSP node in the model.  This node is used for the bounds check.
	 * @param InBox			The frustum to check against.
	 * @param bInUseStrictSelection	true if the node must be entirely within the frustum
	 */
	static bool IntersectsBox(const UModel& InModel, int32 InNodeIndex, const FBox& InBox, bool bInUseStrictSelection);

	/** Adds a hover effect to the passed in actor */
	static void AddHoverEffect(AActor& InActor);

	/** Adds a hover effect to the passed in bsp surface */
	static void AddHoverEffect(UModel& InModel, int32 InSurfIndex);

	/** Removes a hover effect from the passed in actor */
	static void RemoveHoverEffect(AActor& InActor);

	/** Removes a hover effect from the passed in bsp surface */
	static void RemoveHoverEffect(UModel& InModel, int32 InSurfIndex);

	/**
	 * Calculates a box to check actors against
	 *
	 * @param OutBox	The created box.
	 */
	void CalculateBox(FBox& OutBox) const;
	
	/** List of BSP models to check for selection */
	TArray<UModel*> ModelsToCheck;
};
