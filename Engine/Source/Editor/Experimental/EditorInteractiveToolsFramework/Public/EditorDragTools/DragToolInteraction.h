// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor.h"
#include "Behaviors/ViewportClickDragBehavior.h"
#include "ViewportInteractions/ViewportInteraction.h"

#include "DragToolInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class IEditorViewportClientProxy;
struct FWorldSelectionElementArgs;
class FCanvas;
class FEditorModeTools;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FUICommandInfo;

/**
 * ITF related helper functions
 */
namespace UE::Editor::DragTools
{

DECLARE_MULTICAST_DELEGATE(FOnEditorDragToolsToggleDelegate)
DECLARE_MULTICAST_DELEGATE(FOnOnViewportChangeToolToggleDelegate)

/**
 * Returns true if ITF-based drag tools should be used.
 * If an ITF version of a drag tool is not available yet, legacy will be used.
 */
UE_API bool UseEditorDragTools();
UE_API bool IsViewportChangeToolEnabled();

UE_API FOnEditorDragToolsToggleDelegate& OnEditorDragToolsActivated();
UE_API FOnEditorDragToolsToggleDelegate& OnEditorDragToolsDeactivated();

UE_API FOnOnViewportChangeToolToggleDelegate& OnViewportChangeToolActivated();
UE_API FOnOnViewportChangeToolToggleDelegate& OnViewportChangeToolDeactivated();

} // namespace UE::Editor::DragTools

/**
 * The base class that all drag tools inherit from.
 * The drag tools implement special behaviors for the user clicking and dragging in a viewport.
 */
UCLASS(MinimalAPI, Transient)
class UDragToolInteraction : public UViewportInteraction, public IViewportClickDragBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API UDragToolInteraction();

	UE_API virtual void Initialize(UViewportInteractionsBehaviorSource* const InViewportInteractionsBehaviorSource) override;

	/** @return true if we are dragging */
	bool IsDragging() const
	{
		return bIsDragging;
	}

	/** Does this drag tool need to have the mouse movement converted to the viewport orientation? */
	bool bConvertDelta;

	//~ Begin IViewportClickDragBehaviorTarget
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& InPressPos) override
	{
		return FInputRayHit();
	}

	virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) override {}
	virtual void OnDrag(const FDragArgs& InDrag) override {}
	UE_API virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) override;
	UE_API virtual void OnEndCapture(EEndCaptureReason InReason) override;
	UE_API virtual void OnStateUpdated(const FInputDeviceState& InInputDeviceState) override;
	//~ End IViewportClickDragBehaviorTarget

	DECLARE_MULTICAST_DELEGATE(FOnToolStateChange);
	FOnToolStateChange& OnActivateTool()
	{
		return OnToolActivatedDelegate;
	}
	FOnToolStateChange& OnDeactivateTool()
	{
		return OnToolDeactivatedDelegate;
	}

protected:
	UE_API virtual bool ShouldDraw(const EViewInteractionState InInteractionState);

	/** The start/end location of the current drag. */
	FVector Start, End;

	/** If true, the drag tool wants to be passed grid snapped values. */
	bool bUseSnapping;

	FInputDeviceState InputState;
	bool bIsDragging;

	FOnToolStateChange OnToolActivatedDelegate;
	FOnToolStateChange OnToolDeactivatedDelegate;

	IEditorViewportClientProxy* EditorViewportClientProxy;
};

#undef UE_API
