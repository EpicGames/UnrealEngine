// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class FEditorViewportClient;
class FLevelEditorViewportClient;
class UModeManagerInteractiveToolsContext;
struct FWorldSelectionElementArgs;

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class IEditorViewportClientProxy : public TSharedFromThis<IEditorViewportClientProxy>
{
public:
	virtual ~IEditorViewportClientProxy() = default;

	UE_DEPRECATED(5.7, "Please use the version taking UModeManagerInteractiveToolsContext* as argument")
	UE_API static IEditorViewportClientProxy* CreateViewportClientProxy(FEditorViewportClient* InViewportClient);
	UE_API static IEditorViewportClientProxy* CreateViewportClientProxy(UModeManagerInteractiveToolsContext* InInteractiveToolsContext);

	virtual bool IsActorVisible(const AActor* InActor) const = 0;

	virtual const TArray<FName> GetHiddenLayers() const = 0;

	virtual FEditorViewportClient* GetEditorViewportClient() const = 0;
};

class FEditorViewportClientProxy : public IEditorViewportClientProxy
{
public:
	FEditorViewportClientProxy(UModeManagerInteractiveToolsContext* InInteractiveToolsContext);

	virtual bool IsActorVisible(const AActor* InActor) const override;
	virtual const TArray<FName> GetHiddenLayers() const override;
	virtual FEditorViewportClient* GetEditorViewportClient() const override;

private:
	TWeakObjectPtr<UModeManagerInteractiveToolsContext> InteractiveToolsContextWeak;
};

class FLevelEditorViewportClientProxy : public IEditorViewportClientProxy
{
public:
	FLevelEditorViewportClientProxy(UModeManagerInteractiveToolsContext* InInteractiveToolsContext);

	virtual bool IsActorVisible(const AActor* InActor) const override;
	virtual const TArray<FName> GetHiddenLayers() const override;
	virtual FEditorViewportClient* GetEditorViewportClient() const override;
	virtual FLevelEditorViewportClient* GetLevelEditorViewportClient() const;

private:
	TWeakObjectPtr<UModeManagerInteractiveToolsContext> InteractiveToolsContextWeak;
};

#undef UE_API
