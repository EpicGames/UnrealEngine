// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragToolInteraction.h"
#include "MarqueeSelectInteraction.h"

#include "FrustumSelectInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API


class FCanvas;
class FEditorViewportClient;
class FSceneView;
class UModel;
struct FConvexVolume;

/**
 * Draws a box in the current viewport and when the mouse button is released,
 * it selects/unselects everything inside of it.
 */
UCLASS(MinimalAPI, Transient)
class UFrustumSelectInteraction : public UMarqueeSelectInteraction
{
	GENERATED_BODY()

public:
	UE_API UFrustumSelectInteraction();

	//~ Begin IViewportClickDragBehaviorTarget
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& InPressPos) override;
	UE_API virtual void OnDragStart(const FInputDeviceRay& InPressPos) override;
	UE_API virtual void OnDrag(const FDragArgs& InDrag) override;
	UE_API virtual void OnDragEnd(const FInputDeviceRay& InReleasePos) override;
	//~ End IViewportClickDragBehaviorTarget

protected:
	UE_API virtual TArray<FEditorModeID> GetUnsupportedModes() const override;

private:
	/**
	 * Returns true if the provided BSP node intersects with the provided frustum
	 *
	 * @param InModel				The model containing BSP nodes to check
	 * @param NodeIndex				The index to a BSP node in the model.  This node is used for the bounds check.
	 * @param InFrustum				The frustum to check against.
	 * @param bUseStrictSelection	true if the node must be entirely within the frustum
	 */
	static bool IntersectsFrustum(const UModel& InModel, int32 NodeIndex, const FConvexVolume& InFrustum, bool bUseStrictSelection);

	/**
	 * Calculates a frustum to check actors against
	 *
	 * @param InView			Information about scene projection
	 * @param OutFrustum		The created frustum
	 * @param bUseBoxFrustum	If true a frustum out of the current dragged box will be created.  false will use the view frustum.
	 */
	void CalculateFrustum(const FSceneView* InView, FConvexVolume& OutFrustum, bool bUseBoxFrustum) const;
};

namespace UE::EditorDragTools
{

/**
 * FViewportFrustum calculates a frustum box from 2D start and end positions in a viewport.  
 */

struct FViewportFrustum
{
public:
	/** 
	 * @param InViewportClient	Information about the viewport client (view location)
	 * @param InView			Information about scene projection
	 * @param InStart			The 2D screen space start position of the dragged box
	 * @param InEnd				The 2D screen space end position of the dragged box
	 * @param InUseBoxFrustum	If true a frustum out of the current dragged box will be created (false will use the view frustum)
	 */
	UE_API FViewportFrustum(const FEditorViewportClient& InViewportClient, const FSceneView& InView,
		const FVector& InStart, const FVector& InEnd, const bool InUseBoxFrustum);

	/** 
	 * @param OutFrustum		The created frustum
	 */
	UE_API void Calculate(FConvexVolume& OutFrustum) const;

private:

	const FEditorViewportClient& ViewportClient;
	const FSceneView& View;
	const FVector& Start;
	const FVector& End;
	bool bUseBoxFrustum = false;
};

}

#undef UE_API
