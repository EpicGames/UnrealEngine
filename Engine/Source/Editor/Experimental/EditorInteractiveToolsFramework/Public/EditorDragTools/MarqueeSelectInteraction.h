// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DragToolInteraction.h"

#include "MarqueeSelectInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class FCanvas;
class FEditorViewportClient;
class FSceneView;
class UModel;

/**
 * Base class for tools that use a marquee selection behavior
 */
UCLASS(MinimalAPI, Transient)
class UMarqueeSelectInteraction : public UDragToolInteraction
{
	GENERATED_BODY()

public:
	UMarqueeSelectInteraction();

	UE_API virtual void Draw(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) override;
	
protected:

	/**
	 * @return Whether selection should only include items completely contained by the box.  
	 */
	bool IsWindowSelection() const;

	TWeakObjectPtr<UViewportClickDragBehavior> ClickDragInputBehavior;
};

#undef UE_API
