// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragToolInteraction.h"

#include "MeasureToolInteraction.generated.h"

class FCanvas;

UCLASS(MinimalAPI, Transient)
class UMeasureToolInteraction : public UDragToolInteraction
{
	GENERATED_BODY()

public:
	UMeasureToolInteraction();

	virtual void Draw(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) override;

	//~ Begin IViewportClickDragBehaviorTarget
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& InPressPos) override;
	virtual void OnDragStart(const FInputDeviceRay& InPressPos) override;
	virtual void OnDrag(const FDragArgs& InDrag) override;
	//~ End IViewportClickDragBehaviorTarget

private:
	/**
	 * Gets the world-space snapped position of the specified pixel position
	 */
	FVector2D GetSnappedPixelPos(FVector2D InPixelPos) const;

	/** Pixel-space positions for start and end */
	FVector2D PixelStart, PixelEnd;
};
