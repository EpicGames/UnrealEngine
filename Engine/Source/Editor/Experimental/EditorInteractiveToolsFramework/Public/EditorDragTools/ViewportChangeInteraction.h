// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/UnrealEdTypes.h"
#include "DragToolInteraction.h"

#include "ViewportChangeInteraction.generated.h"

class FCanvas;
class FEditorViewportClient;
class FLevelEditorViewportClient;
class FSceneView;

UCLASS(MinimalAPI, Transient)
class UViewportChangeInteraction : public UDragToolInteraction
{
	GENERATED_BODY()

public:
	UViewportChangeInteraction();

	virtual void Draw(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) override;

	//~ Begin IViewportClickDragBehaviorTarget
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& InPressPos) override;
	virtual void OnBeginCapture(const FInputDeviceRay& InClickPressPos) override;
	virtual void OnDragStart(const FInputDeviceRay& InPressPos) override;
	virtual void OnDrag(const FDragArgs& InDrag) override;
	virtual void OnDragEnd(const FInputDeviceRay& InReleasePos) override;
	//~ End IViewportClickDragBehaviorTarget

private:
	ELevelViewportType GetDesiredViewportType() const;
	FText GetDesiredViewportTypeText() const;

	FVector2D ViewOptionOffset;
};
