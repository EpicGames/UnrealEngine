// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoElementShared.h"
#include "CoreTypes.h"

#include "EditorGizmoElementShared.generated.h"

class UGizmoElementLineBase;
class UGizmoElementBase;
class UMaterialInterface;
struct FGizmoPerStateValueLinearColor;
struct FGizmoPerStateValueMaterialVariant;
struct FGizmoMaterialVariant;

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

namespace UE::Editor::InteractiveToolsFramework
{
	UE_API void ApplyMaterialsToElement(UGizmoElementBase* InElement, const FGizmoPerStateValueMaterialVariant& InMaterials);

	UE_API void ApplyMaterialOverrides(FGizmoMaterialVariant& InMaterialToOverride, const FGizmoMaterialVariant& InMaterialToOverrideFrom);

	/** Overrides materials in MaterialsToOverride with those in MaterialsToOverrideFrom, if set. */
	UE_API void ApplyMaterialOverrides(FGizmoPerStateValueMaterialVariant& InMaterialsToOverride, const FGizmoPerStateValueMaterialVariant& InMaterialsToOverrideFrom);

	UE_API void ApplyColorsToElement(UGizmoElementBase* InElement, const FGizmoPerStateValueLinearColor& InColors);

	UE_API void ApplyColorsToElement(UGizmoElementLineBase* InElement, const FGizmoPerStateValueLinearColor& InColors);

	UE_API void ApplyColorOverrides(FGizmoPerStateValueLinearColor& InColorsToOverride, const FGizmoPerStateValueLinearColor& InColorsToOverrideFrom);
}

USTRUCT(MinimalAPI)
struct FGizmoDebugSettings
{
	GENERATED_BODY()

public:
	/** Corresponds with the CVar. Not user-exposed, only used for persitence. */
	UPROPERTY(meta = (AllowPrivateAccess = "true"))
	bool bDrawDebug = false;

	/** Set to false to disable rendering of gizmos while still drawing any selected debug options. */
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawGizmos = true;

	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawTiming = true;

	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawSnapping = true;

	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawHitTarget = false;

	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawInteractionPlane = true;

	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawAlignedInteractionPlane = true;

	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawScreenPlane = true;

	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawInputDelta = false;

	/** Transform Axes */
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawLocalTransform = true;

	/** Element Axes */
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawElementTransform = true;

	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawCursor = false;

	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawCursorRay = false;

	/** Shows visuals to assist QA in debugging the relationship between input, and the resulting transform applied. */
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawInputCorrespondence = false;

	/** Freezes view-dependent transforms, so that you can navigate around and see how elements are positioned in screen space. */
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bFreezeCamera = false;

	/** Shows the last delta visuals. */
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bFreezeDelta = false;
};

USTRUCT()
struct FGizmoMaterialVariant
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMaterialInterface> Solid;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> Translucent;

	bool operator==(const FGizmoMaterialVariant& InOther) const
	{
		return Solid == InOther.Solid
			&& Translucent == InOther.Translucent;
	}
};

/**
 * Used to store per-state (GizmoMaterialVariant) values for gizmo elements.
 * ie. vertex color.
 */
USTRUCT(MinimalAPI, meta = (DisplayName = "Per-State Value (GizmoMaterialVariant)"))
struct FGizmoPerStateValueMaterialVariant
{
	GENERATED_BODY()

	using FValueType = FGizmoMaterialVariant;

	/**
	 * Default value, used when the Interaction State is "None".
	 * Optional to allow explicit un-setting, implying inheritance or some other value source.
	 */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<FGizmoMaterialVariant> Default;

	/** Value used when hovering. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<FGizmoMaterialVariant> Hover;

	/** Value used when interacting. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<FGizmoMaterialVariant> Interact;

	/** Value used when selected. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<FGizmoMaterialVariant> Select;

	/** Value used when subdued. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<FGizmoMaterialVariant> Subdue;

	/** Get the value for the given Interaction State, or Default if not set. */
	UE_API const FGizmoMaterialVariant& GetValueForState(const EGizmoElementInteractionState InState) const;

	/** Get the Default value if set, otherwise FLinearColor::Transparent. */
	UE_API const FGizmoMaterialVariant& GetDefaultValue() const;

	/** Get the Hover value, or Default if not set. */
	const FGizmoMaterialVariant& GetHoverValue() const { return Hover.Get(GetDefaultValue()); }

	/** Get the Interact value, or Default if not set. */
	const FGizmoMaterialVariant& GetInteractValue() const { return Interact.Get(GetDefaultValue()); }

	/** Get the Select value, or Default if not set. */
	const FGizmoMaterialVariant& GetSelectValue() const { return Select.Get(GetDefaultValue()); }

	/** Get the Subdue value, or Default if not set. */
	const FGizmoMaterialVariant& GetSubdueValue() const { return Subdue.Get(GetDefaultValue()); }

	friend bool operator==(const FGizmoPerStateValueMaterialVariant& InLeft, const FGizmoPerStateValueMaterialVariant& InRight)
	{
		return InLeft.Default == InRight.Default
			&& InLeft.Hover == InRight.Hover
			&& InLeft.Interact == InRight.Interact
			&& InLeft.Select == InRight.Select
			&& InLeft.Subdue == InRight.Subdue;
	}

	friend bool operator!=(const FGizmoPerStateValueMaterialVariant& InLeft, const FGizmoPerStateValueMaterialVariant& InRight)
	{
		return !(InLeft == InRight);
	}
};

/** Contains Style/Visual properties common to all gizmo elements, ie. PixelHitDistanceThreshold. */
USTRUCT(MinimalAPI)
struct FGizmoStyleBase
{
	GENERATED_BODY()

	/** Default values, required when applying one style to another - where they'll only override destination values that are Default. */
	static const FGizmoPerStateValueMaterialVariant DefaultMaterials;
	static const FGizmoPerStateValueLinearColor DefaultColors;
	static constexpr float DefaultMinLineThickness = 1.0f;
	static constexpr float DefaultLineThicknessMultiplier = 1.0f;
	static constexpr float DefaultHoverLineThicknessMultiplier = 1.5f;
	static const FGizmoPerStateValueMaterialVariant DefaultLineMaterials;
	static const FGizmoPerStateValueLinearColor DefaultLineColors;
	static constexpr float DefaultPixelHitDistanceThreshold = 7.0f;
	static constexpr float DefaultMinimumPixelHitDistanceThreshold = 0.0f;

	UPROPERTY()
	FGizmoPerStateValueMaterialVariant Materials = DefaultMaterials;

	UPROPERTY(EditAnywhere, Category = "Style", meta = (DisplayPriority = -100))
	FGizmoPerStateValueLinearColor Colors = DefaultColors;

	UPROPERTY()
	FGizmoPerStateValueMaterialVariant LineMaterials = DefaultLineMaterials;

	UPROPERTY(EditAnywhere, Category = "Style", meta = (DisplayPriority = -100))
	FGizmoPerStateValueLinearColor LineColors = DefaultLineColors;

	/** Used to prevent sub-pixel line sizes. */
	UPROPERTY()
	float MinLineThickness = DefaultMinLineThickness;

	UPROPERTY()
	float LineThicknessMultiplier = DefaultLineThicknessMultiplier;

	UPROPERTY()
	float HoverLineThicknessMultiplier = DefaultHoverLineThicknessMultiplier;

	/* Pixel hit distance threshold - where the default is generally 7px. */
	UPROPERTY(EditAnywhere, Category = "Style", AdvancedDisplay, meta = (ClampMin = 0, UIMin = 0, UIMax = 50))
	float PixelHitDistanceThreshold = DefaultPixelHitDistanceThreshold;

	/* Minimum pixel hit distance threshold. */
	UPROPERTY(EditAnywhere, Category = "Style", AdvancedDisplay, meta = (ClampMin = 0, UIMin = 0, UIMax = 50))
	float MinimumPixelHitDistanceThreshold = DefaultMinimumPixelHitDistanceThreshold;

	UPROPERTY()
	TOptional<float> SizeCoefficient;

	/** Applies values to the given Style, if those corresponding values are Defaults. */
	template <typename FromType, typename ToType
		UE_REQUIRES(std::is_base_of_v<FGizmoStyleBase, FromType>
			&& std::is_base_of_v<FGizmoStyleBase, ToType>)>
	static void ApplyTo(const FromType& InFrom, ToType& InTo)
	{
		InTo.Materials = InTo.Materials == InTo.DefaultMaterials ? InFrom.Materials : InTo.Materials;
		InTo.Colors = InTo.Colors == InTo.DefaultColors ? InFrom.Colors : InTo.Colors;
		InTo.LineMaterials = InTo.LineMaterials == InTo.DefaultLineMaterials ? InFrom.LineMaterials : InTo.LineMaterials;
		InTo.LineColors = InTo.LineColors == InTo.DefaultLineColors ? InFrom.LineColors : InTo.LineColors;
		InTo.MinLineThickness = InTo.MinLineThickness == InTo.DefaultMinLineThickness ? InFrom.MinLineThickness : InTo.MinLineThickness;
		InTo.LineThicknessMultiplier = InTo.LineThicknessMultiplier == InTo.DefaultLineThicknessMultiplier ? InFrom.LineThicknessMultiplier : InTo.LineThicknessMultiplier;
		InTo.HoverLineThicknessMultiplier = InTo.HoverLineThicknessMultiplier == InTo.DefaultHoverLineThicknessMultiplier ? InFrom.HoverLineThicknessMultiplier : InTo.HoverLineThicknessMultiplier;
		InTo.PixelHitDistanceThreshold = InTo.PixelHitDistanceThreshold == InTo.DefaultPixelHitDistanceThreshold ? InFrom.PixelHitDistanceThreshold : InTo.PixelHitDistanceThreshold;
		InTo.MinimumPixelHitDistanceThreshold = InTo.MinimumPixelHitDistanceThreshold == InTo.DefaultMinimumPixelHitDistanceThreshold ? InFrom.MinimumPixelHitDistanceThreshold : InTo.MinimumPixelHitDistanceThreshold;
	}

	friend bool operator==(const FGizmoStyleBase& InLeft, const FGizmoStyleBase& InRight)
	{
		return InLeft.Materials == InRight.Materials
			&& InLeft.Colors == InRight.Colors
			&& InLeft.LineMaterials == InRight.LineMaterials
			&& InLeft.LineColors == InRight.LineColors
			&& InLeft.MinLineThickness == InRight.MinLineThickness
			&& InLeft.LineThicknessMultiplier == InRight.LineThicknessMultiplier
			&& InLeft.HoverLineThicknessMultiplier == InRight.HoverLineThicknessMultiplier
			&& InLeft.PixelHitDistanceThreshold == InRight.PixelHitDistanceThreshold
			&& InLeft.MinimumPixelHitDistanceThreshold == InRight.MinimumPixelHitDistanceThreshold;
	}

	friend bool operator!=(const FGizmoStyleBase& InLeft, const FGizmoStyleBase& InRight)
	{
		return !(InLeft == InRight);
	}
};

#undef UE_API
