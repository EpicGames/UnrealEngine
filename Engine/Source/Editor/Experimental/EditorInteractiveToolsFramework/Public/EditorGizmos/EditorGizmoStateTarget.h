// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/StateTargets.h"

#include "Templates/Function.h"
#include "Internationalization/Text.h"
#include "TransformGizmoInterfaces.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "EditorGizmoStateTarget.generated.h"

class FEditorModeTools;
class IToolContextTransactionProvider;
class UTransformGizmo;

/**
 * UEditorGizmoStateTarget
 */
UCLASS(MinimalAPI)
class UEditorGizmoStateTarget : public UObject, public IGizmoStateTarget
{
	GENERATED_BODY()
public:

	/**
	 * BeginUpdate is called before the gizmo starts modifying the transform.  
	 */
	virtual void BeginUpdate() override;

	/**
	 * EndUpdate is called after the gizmo finished modifying the transform.
	 */
	virtual void EndUpdate() override;
	
	/*
	 * CancelUpdate is called if a gizmo cancels a modification.
	 */
	virtual void CancelUpdate() override;

	/**
	 * TWeakPtr to mode manager to interface with
	 */
	TWeakPtr<FEditorModeTools> WeakModeTools;

	/**
	* Localized text description of the transaction (used if TransactionManager is not null) 
	*/
	FText TransactionDescription;
	
	/**
	 * Pointer to the GizmoManager or ToolManager that is used to open/close the transaction (if no Begin/End functions is provided by BeginUpdate)
	 */
	UPROPERTY()
	TScriptInterface<IToolContextTransactionProvider> TransactionManager;

	/**
	 * Create and initialize an standard instance of UGizmoEditorStateTarget
	 * @param InModeManager the context object this StateTarget will retrieve the mode manager from
	 * @param InDescription Localized text description of the transaction (if no Begin/End functions is provided by BeginUpdate)
	 * @param TransactionManagerIn pointer to the object to manage transactions (if no Begin/End functions is provided by BeginUpdate)
	 */
	EDITORINTERACTIVETOOLSFRAMEWORK_API static UEditorGizmoStateTarget* Construct(
		FEditorModeTools* InModeManager,
		const FText& InDescription,
		IToolContextTransactionProvider* TransactionManagerIn,
		UObject* Outer = (UObject*)GetTransientPackage());

	/**
	 * Sets the transform gizmo from which we can get data if required.
	 */
	void SetTransformGizmo(UTransformGizmo* InGizmo);
	
protected:
	/** Gets the current TransformMode, from the associated TransformGizmo (if available). */
	EGizmoTransformMode GetTransformMode() const;
	
protected:
	/**
	 * The transform gizmo from which we can get data if required.
	 */
	TWeakObjectPtr<UTransformGizmo> TransformGizmo;
};
