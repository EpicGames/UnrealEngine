// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseGizmos/GizmoElementGroup.h"
#include "BaseGizmos/GizmoElementHitTargets.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/TransformProxy.h"
#include "Behaviors/SingleClickAndDragBehavior.h"
#include "Containers/EnumAsByte.h"
#include "EditorGizmos/EditorGizmoElementShared.h"
#include "EditorGizmos/EditorTransformGizmo.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"
#include "FrameTypes.h"
#include "Framework/Commands/InputChord.h"
#include "InputState.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Math/Axis.h"
#include "Math/Quat.h"
#include "Math/Ray.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "EditorTRSGizmo.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class FEditorViewportClient;
class ULocalViewportClickDragBehavior;
class IGizmoAxisSource;
class IGizmoStateTarget;
class IGizmoTransformSource;
class IToolContextTransactionProvider;
class IToolsContextRenderAPI;
class UGizmoConstantFrameAxisSource;
class UGizmoElementArc;
class UGizmoElementArrow;
class UGizmoElementBase;
class UGizmoElementBox;
class UGizmoElementCircle;
class UGizmoElementCone;
class UGizmoElementCylinder;
class UGizmoDebugProvider;
class UGizmoElementGimbal;
class UGizmoElementGroup;
class UGizmoElementHitMultiTarget;
class UGizmoElementRay;
class UGizmoElementRoot;
class UGizmoElementRotateAxis;
class UGizmoElementRotateAxisSet;
class UGizmoElementScaleGroup;
class UGizmoElementTranslateGroup;
class UGizmoElementSphere;
class UGizmoElementTorus;
class UGizmoElementWidget;
class UInteractiveGizmoManager;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UMultiButtonClickOrDragBehavior;
class UObject;
class UTransformProxy;

class UEditorTRSGizmo;

namespace UE::Editor::InteractiveToolsFramework
{
	class FTransformGizmoProfiler;

	namespace Private
	{
		/** Friended class to access UEditorTRSGizmo internals, used internally. */
		struct FTransformGizmoAccessorPrivate;
	}
	
	template<typename DeltaType>
	struct TNudgeDelta
	{
		DeltaType Delta;
		EAxisList::Type Axis = EAxisList::None;
	};
}

/**
 */
UCLASS(MinimalAPI)
class UEditorTRSGizmo
	: public UEditorTransformGizmo
	, public ISingleClickAndDragBehaviorTarget
	, public IKeyInputBehaviorTarget
{
	GENERATED_BODY()

	friend class UEditorTRSGizmoBuilder;
	friend UE::Editor::InteractiveToolsFramework::FTransformGizmoAccessor;
	friend UE::Editor::InteractiveToolsFramework::Private::FTransformGizmoAccessorPrivate;
	friend class UTransformGizmoDebug;

public:

	// UInteractiveGizmo overrides
	virtual void Setup() override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

	//~ Begin ISingleClickAndDragBehaviorTarget
	UE_API virtual FInputRayHit CanBeginSingleClickAndDragSequence(const FInputDeviceRay& InPressPos) override;
	UE_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnDragStart(const FInputDeviceRay& InDragPos) override;
	UE_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	UE_API virtual void OnClickRelease(const FInputDeviceRay& InReleasePos, bool bInIsDragOperation) override;
	UE_API virtual void OnTerminateSingleClickAndDragSequence() override;
	//~ End ISingleClickAndDragBehaviorTarget

	//~ Begin IKeyInputBehaviorTarget interface
	UE_API virtual void OnKeyPressed(const FKey& KeyID) override;
	UE_API virtual void OnKeyReleased(const FKey& KeyID) override;
	UE_API virtual void OnForceEndCapture() override;
	//~ End IKeyInputBehaviorTarget interface

	//~ Begin IModifierToggleBehaviorTarget
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;
	//~ End IModifierToggleBehaviorTarget

private:
	// The IClickDragBehaviorTarget functions are not used 
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	virtual void OnTerminateDragSequence() override;

public:

	virtual void BeginTransformEditSequence() override;
	virtual void EndTransformEditSequence() override;

	const FTransformGizmoStyle& GetStyle() const;

	/** Sets and applies the given Style. */
	void SetStyle(const FTransformGizmoStyle& InStyle);

	/** Sets the Gizmo view context, needed for screen space interactions */
	UE_API void SetGizmoViewContext(UGizmoViewContext* InGizmoViewContext);

	/**
	 * Handle widget mode changed.
	 */
	UE_API virtual void HandleWidgetModeChanged(UE::Widget::EWidgetMode InWidgetMode) override;

	/**
	 * Handle user parameters changes
	 */
	UE_API virtual void OnParametersChanged(const FGizmosParameters& InParameters) override;

public:

	/** Style contains various uniform and varying properties that affect the appearance of this gizmo. */
	UPROPERTY()
	FTransformGizmoStyle Style;

	/** Interaction contains various user-defined options for gizmo interaction. */
	UPROPERTY()
	FTransformGizmoInteraction Interaction;

	/** Debug Info provider */
	UPROPERTY()
	TObjectPtr<UGizmoDebugProvider> DebugProvider;

	/**  */
	UPROPERTY()
	FVector2D InteractionPlanarStartPoint2D;

	/**  */
	UPROPERTY()
	FVector2D InteractionPlanarCurrPoint2D;

	/**  */
	UPROPERTY()
	FVector2D InteractionScreenObjectPos2D;

	UPROPERTY()
	double InteractionPlanarStartDistance = 0.0;

	UPROPERTY()
	double InteractionPlanarCurrDistance = 0.0;

	/**  */
	UPROPERTY()
	double InteractionDeltaDivisor;

	UPROPERTY()
	double InteractionReferencePointOffsetDistance = 0.0;

	/** The difference between the angle in screen-space vs. world-space, if applicable. */
	UPROPERTY()
	double AlignedInteractionAngleOffset = 0.0;

	/** The normal direction of the current axis, which may or may not be the same as InteractionPlanarNormal. */
	UPROPERTY()
	FVector AlignedInteractionPlanarNormal = FVector::ZeroVector;

	/** Store accumulatively applied delta. */
	UPROPERTY()
	FVector InteractionDelta = FVector::ZeroVector;

	UPROPERTY()
	FVector InteractionBidirection = FVector::ZeroVector;

	/** Used to translate interaction direction to aligned (accounts for flipping, etc.) */
	UE::Math::TIntVector3<int8> ViewToAlignedSign = UE::Math::TIntVector3<int8>(1);

	UPROPERTY()
	FVector2D InteractionBidirection2D = FVector2D::ZeroVector;

	/** Initial sign of the interaction, if applicable. */
	UPROPERTY()
	double InteractionStartSign = 1.0;

	enum ENudgeDirection
	{
		None,
		Horizontal,
		Vertical,
		Secondary // A modifier can be used to enter a secondary nudging mode
	};

protected:

	UPROPERTY()
	TObjectPtr<ULocalViewportClickDragBehavior> IndirectClickDragBehavior;

	//
	// Gizmo Objects, used for rendering and hit testing
	//

	/** Translate Axis, Plane and ScreenSpace Group. */
	UPROPERTY()
	TObjectPtr<UGizmoElementTranslateGroup> TranslateGroupElement;

	/** Rotate Axis Group. */
	UPROPERTY()
	TObjectPtr<UGizmoElementRotateAxisSet> RotateAxisSetElement;

	/** Gimbal Rotate X Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementRotateAxis> RotateXGimbalElement2;

	/** Gimbal Rotate Y Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementRotateAxis> RotateYGimbalElement2;
	
	/** Gimbal Rotate Z Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementRotateAxis> RotateZGimbalElement2;

	/** Rotate screen space circle */
	UPROPERTY()
	TObjectPtr<UGizmoElementRotateAxis> RotateScreenSpaceElement2;

	/** Scale Axis & Plane Group. */
	UPROPERTY()
	TObjectPtr<UGizmoElementScaleGroup> ScaleGroupElement;


protected:

	TSharedPtr<SWidget> GetCursorWidget(const ETransformGizmoPartIdentifier InHitPartId) const;

	UE_API float GetCursorRotation() const;

	UE_API void UpdateCursorRotation(const FInputDeviceRay& InPressPos);
	UE_API void UpdateCursor(const EGizmoElementInteractionState InState, const ETransformGizmoPartIdentifier InHitPartId, const bool bIsExitingState = false);

	/** Renders various debug visualizations, if enabled. Canvas is only valid when called from DrawHUD. */
	void DrawDebug(FCanvas* InCanvas, IToolsContextRenderAPI* RenderAPI, const double InPixelToWorldScale);

	/** Setup behaviors */
	UE_API virtual void SetupBehaviors() override;

	/** Setup indirect behaviors */
	UE_API virtual void SetupIndirectBehaviors() override;

	/** Setup materials */
	UE_API virtual void SetupMaterials() override;

	/** Setup on click functions */
	UE_API virtual void SetupOnClickFunctions() override;

	/** Update current gizmo mode based on transform source */
	UE_API virtual void UpdateMode() override;

	/** Update current gizmo rotation mode (default or gimbal)*/
	UE_API virtual void UpdateRotationMode() override;

	/** Enable the given mode with the specified axes, EAxisList::Type::None will hide objects associated with mode */
	UE_API virtual void EnableMode(EGizmoTransformMode InGizmoMode, EAxisList::Type InAxisListToDraw) override;

	/** Enable translate using specified axis list */
	UE_API virtual void EnableTranslate(EAxisList::Type InAxisListToDraw) override;

	/** Enable rotate using specified axis list */
	UE_API virtual void EnableRotate(EAxisList::Type InAxisListToDraw) override;

	/** Enable scale using specified axis list */
	UE_API virtual void EnableScale(EAxisList::Type InAxisListToDraw) override;

	/** Construct translate group. AxisList should be XYZ or LUF, not the subset of axis' being currently edited. */
	UE_API virtual UGizmoElementTranslateGroup* MakeTranslateGroup(const EAxisList::Type InAxisList);

	/** Updates the given translate group with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateTranslateGroup(UGizmoElementTranslateGroup* InElement);

	/** Construct scale group. AxisList should be XYZ or LUF, not the subset of axis' being currently edited. */
	UE_API virtual UGizmoElementScaleGroup* MakeScaleGroup(const EAxisList::Type InAxisList);

	/** Updates the given scale group with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateScaleGroup(UGizmoElementScaleGroup* InElement);

	/** Construct rotate axis handle */
	UE_API virtual UGizmoElementTorus* MakeRotateAxis(ETransformGizmoPartIdentifier InPartId, const FVector& TorusAxis0, const FVector& TorusAxis1, UMaterialInterface* InMaterial);

	/** Updates the given rotate axis handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateRotateAxis(UGizmoElementTorus* InElement) override;

	/** Construct rotate axis set. AxisList should be XYZ or LUF, not the subset of axis' being currently edited. */
	UE_API virtual UGizmoElementRotateAxisSet* MakeRotateAxisSet(const EAxisList::Type InAxisList);

	/** Updates the given rotate axis set with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateRotateAxisSet(UGizmoElementRotateAxisSet* InElement);

	/** Construct rotate axis. */
	UE_API UGizmoElementRotateAxis* MakeRotateAxis(const ETransformGizmoPartIdentifier InPartId, const EAxis::Type InAxis);

	/** Construct rotate axis. */
	UE_API virtual UGizmoElementTorus* MakeRotateAxis(ETransformGizmoPartIdentifier InPartId, const FVector& TorusAxis0, const FVector& TorusAxis1,
		UMaterialInterface* InMaterial, UMaterialInterface* InCurrentMaterial) override;

	/** Updates the given rotate axis with the current parameters (ie. size coefficient). */
	UE_API virtual void UpdateRotateAxis(UGizmoElementRotateAxis* InElement, const EAxis::Type InAxis);

	/** Construct rotate axis arc handle */
	UE_API virtual UGizmoElementArc* MakeRotateAxisArc(ETransformGizmoPartIdentifier InPartId, const FVector& InAxis0, const FVector& InAxis1, UMaterialInterface* InMaterial);

	/** Updates the given rotate axis arc handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateRotateAxisArc(UGizmoElementArc* InElement, UMaterialInterface* InMaterial);

	/** Construct arcball screen space handle */
	UE_API virtual UGizmoElementCircle* MakeArcballCircleHandle(ETransformGizmoPartIdentifier InPartId, float InRadius, const FLinearColor& InColor) override;

	/** Updates the given arcball handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateArcballCircleHandle(UGizmoElementCircle* InElement);

	/** Construct rotate screen space handle */
	UE_API virtual UGizmoElementRotateAxis* MakeRotateScreenSpaceHandle();

	/** Updates the given rotate screen space handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateRotateScreenSpaceHandle(UGizmoElementRotateAxis* InElement);

	/** Updates the given element with properties common to all. */
	UE_API void UpdateElement(UGizmoElementBase* InElement, const TOptional<FGizmoStyleBase>& InStyle = TOptional<FGizmoStyleBase>());

	/** Updates all gizmo elements based on current parameters */
	UE_API virtual void UpdateElements() override;

	/** Get gizmo transform based on cached current transform. */
	UE_API virtual FTransform GetGizmoTransform() const override;

	/** Updated referenced materials according to current parameters. */
	UE_API virtual void UpdateMaterials();

	/** Determine hit part and update hover state based on current input ray */
	UE_API virtual FInputRayHit UpdateHoveredPart(const FInputDeviceRay& DevicePos) override;

	/** Return true if input ray intersects current interaction plane and return param along ray in OutHitParam */
	UE_API virtual bool GetRayParamIntersectionWithInteractionPlane(const FInputDeviceRay& InRay, FVector::FReal& OutHitParam) override;

	/** Return true if input ray intersects the given plane and return param along ray in OutHitParam */
	UE_API virtual bool GetRayParamIntersectionWithPlane(const FRay& InRay, const FVector& InPlaneOrigin, const FVector& InPlaneNormal, FVector::FReal& OutHitParam);

	/** Update hover state for given part id */
	UE_API virtual void UpdateHoverState(const bool bInHover, const ETransformGizmoPartIdentifier InPartId) override;

	/** Update interacting state for given part id */
	UE_API virtual void UpdateInteractingState(const bool bInInteracting, const ETransformGizmoPartIdentifier InPartId, const bool bIdOnly = false) override;

	/** Reset all interacting states related to the transform mode to false */
	UE_API virtual void ResetInteractingStates(const EGizmoTransformMode InMode) override;

	/** Update selected state for given part id */
	UE_API virtual void UpdateSelectedState(const bool bInSelected, const ETransformGizmoPartIdentifier InPartId, const bool bIdOnly = false);

	/** Reset all selected states related to the transform mode to false */
	UE_API void ResetSelectedStates(const EGizmoTransformMode InMode);

	/** Update subdued state for given part id */
	UE_API virtual void UpdateSubdueState(const bool bInSubdued, const ETransformGizmoPartIdentifier InPartId, const bool bIdOnly = false);

	/** Reset all subdued states related to the transform mode to false */
	UE_API void ResetSubdueStates(const EGizmoTransformMode InMode);

	/** Ends all Delta states/visuals. */
	UE_API void EndDeltas();

	/**
	 * Translate axis click-drag handling methods 
	 */ 

	/** Handle click press for translate axes */
	UE_API virtual void OnClickPressAxis(const FInputDeviceRay& PressPos) override;

	/** Handle drag start for translate axes */
	UE_API void OnDragStartTranslateAxis(const FInputDeviceRay& PressPos);

	/** Handle click drag for translate axes */
	UE_API virtual void OnClickDragTranslateAxis(const FInputDeviceRay& DragPos) override;

	/** Handle click release for translate axes */
	UE_API virtual void OnClickReleaseTranslateAxis(const FInputDeviceRay& ReleasePos) override;

	/**
	 * Translate and scale planar click-drag handling methods
	 */

	UE_API virtual void OnClickPressTranslatePlanar(const FInputDeviceRay& PressPos) override;

	/** Handle click drag for translate planar */
	UE_API virtual void OnClickDragTranslatePlanar(const FInputDeviceRay& DragPos) override;

	/** Handle click release for translate planar */
	UE_API virtual void OnClickReleaseTranslatePlanar(const FInputDeviceRay& ReleasePos) override;

	/**
	 * Screen-space translate interaction methods
	 */

	 /** Handle click press for screen-space translate */
	UE_API virtual void OnClickPressScreenSpaceTranslate(const FInputDeviceRay& PressPos) override;

	/** Handle click drag for screen-space translate */
	UE_API virtual void OnClickDragScreenSpaceTranslate(const FInputDeviceRay& DragPos) override;

	/** Handle click release for screen-space translate */
	UE_API virtual void OnClickReleaseScreenSpaceTranslate(const FInputDeviceRay& ReleasePos) override;

	/**
	 * Rotate interaction methods
	 */

	/** Handle click press for any rotate axis */
	UE_API virtual void OnClickPressRotateAxis(const FInputDeviceRay& InPressPos) override;

	/** Handle click drag for rotate axis */
	UE_API virtual void OnClickDragRotateAxis(const FInputDeviceRay& InDragPos) override;

	/** Handle click release for rotate axes */
	UE_API virtual void OnClickReleaseRotateAxis(const FInputDeviceRay& InReleasePos) override;

	/** Prepares data for arc rotation. This will return false if this is not possible (the rotate handle is perpendicular to the view) */
	UE_API bool OnClickPressRotateArc( const FInputDeviceRay& InPressPos,
		const FVector& InPlaneNormal, const FVector& InPlaneAxis1, const FVector& InPlaneAxis2, const EAxis::Type InAxis = EAxis::None);

	/** Compute angle (degrees) delta based on screen-space start/end positions. Intended to be used in combination with ComputeAxisRotateDelta */
	UE_API virtual double ComputeAxisRotateDeltaAngle(const FVector2D& InStartPos, const FInputDeviceRay& InDragPos) override;

	/** Compute gimbal rotate delta based on screen-space start/end positions */
	UE_API virtual FQuat ComputeGimbalRotateDelta(const FVector2D& InStartPos, const FVector2D& InEndPos, const double InAngleSign = 1.0) override;

	UE_API virtual FVector GetGimbalRotationAxis(const int32 InAxis) const override;

	// @TODO: Overwrite GetScreenGimbalRotateAxisDir
	/** Get screen-space axis for gimbal rotation drag */
	UE_API virtual FVector2D GetScreenGimbalRotateAxisDir(const FInputDeviceRay& InPressPos) override;

	// @TODO: Overwrite GetScreenRotateAxisDir
	/** Get screen-space axis for rotation drag */
	UE_API virtual FVector2D GetScreenRotateAxisDir(const FInputDeviceRay& InPressPos) override;

	/**
	 * Gimbal rotate interaction methods
	 * note that gimbal rotation release functions use OnClickReleaseRotateAxis
	 */

	/** Handle click press for any gimbal rotate axis */
	UE_API virtual void OnClickPressGimbalRotateAxis(const FInputDeviceRay& InPressPos) override;

	/** Handle click drag for gimbal rotate axis */
	UE_API virtual void OnClickDragGimbalRotateAxis(const FInputDeviceRay& InDragPos) override;
	
	/**
	 * Screen-space rotate interaction methods
	 */

	 /** Handle click press for screen-space rotate */
	UE_API virtual void OnClickPressScreenSpaceRotate(const FInputDeviceRay& PressPos) override;

	/** Handle click drag for screen-space rotate */
	UE_API virtual void OnClickDragScreenSpaceRotate(const FInputDeviceRay& InDragPos) override;

	/** Handle click release for screen-space rotate */
	UE_API virtual void OnClickReleaseScreenSpaceRotate(const FInputDeviceRay& ReleasePos) override;

	/**
	 * Arc ball rotate interaction methods
	 */

	/** Handle click press for arc ball rotate */
	UE_API virtual void OnClickPressArcBallRotate(const FInputDeviceRay& PressPos) override;

	/** Handle click drag for arc ball rotate */
	UE_API virtual void OnClickDragArcBallRotate(const FInputDeviceRay& DragPos) override;

	/**
	* Scale click-drag handling methods
	*/

	/** Handle click press for scale X axis */
	UE_API virtual void OnClickPressScaleXAxis(const FInputDeviceRay& PressPos) override;

	/** Handle click press for scale Y axis */
	UE_API virtual void OnClickPressScaleYAxis(const FInputDeviceRay& PressPos) override;

	/** Handle click press for scale Z axis */
	UE_API virtual void OnClickPressScaleZAxis(const FInputDeviceRay& PressPos) override;

	/** Handle click press for scale XY planar */
	UE_API virtual void OnClickPressScaleXYPlanar(const FInputDeviceRay& PressPos) override;

	/** Handle click press for scale YZ planar */
	UE_API virtual void OnClickPressScaleYZPlanar(const FInputDeviceRay& PressPos) override;

	/** Handle click press for scale XZ planar */
	UE_API virtual void OnClickPressScaleXZPlanar(const FInputDeviceRay& PressPos) override;

	/** Handle click press for scale planar */
	UE_API virtual void OnClickPressScalePlanar(const FInputDeviceRay& PressPos);

	/** Handle click press for all scale methods */
	UE_API virtual void OnClickPressScale(const FInputDeviceRay& PressPos) override;

	/** Handle click drag for all scale */
	UE_API virtual void OnClickDragScale(const FInputDeviceRay& DragPos) override;

	/** Handle click release for scale axes */
	UE_API virtual void OnClickReleaseScaleAxis(const FInputDeviceRay& ReleasePos) override;

	/** Handle click release for scale planar */
	UE_API virtual void OnClickReleaseScalePlanar(const FInputDeviceRay& ReleasePos) override;

	/** Handle click release for uniform scale */
	UE_API virtual void OnClickReleaseScaleXYZ(const FInputDeviceRay& ReleasePos) override;

	/** Compute scale delta based on screen space start/end positions */
	UE_API virtual FVector ComputeScaleDelta(const FVector2D& InStartPos, const FVector2D& InEndPos, FVector2D& OutScreenDelta) override;

	UE_API virtual void ApplyScaleDelta(const FVector& InScaleDelta) override;

	/**
	 * Screen-space helper method
	 */

	/**
	 * Apply transform methods
	 */

	UE_API virtual void ApplyTranslate(const FVector& InTranslate);

	/**
	 * Apply transform delta methods
	 */

	UE_API void SnapTranslate(FVector& InOutWorldDelta, const EAxisList::Type InAxisList, const FRay& InRay = FRay()) const;
	UE_API void SnapRotateDelta(FQuat& InOutWorldDelta, const EAxisList::Type InAxisList) const;
	UE_API void SnapRotateAngleDelta(double& InOutAngleDelta, const EAxisList::Type InAxisList) const;
	UE_API void SnapScaleDelta(FVector& InOutLocalScaleDelta, const EAxisList::Type InAxisList, const FRay& InRay = FRay()) const;

	UE_API void OnSelectionChanged(const UTypedElementSelectionSet* InElementSelectionSet);

	FTransform GetActiveTransform() const;

	void CacheCursorPosition();
	void RestoreCursorPosition() const;

	/** Used when TRS Snapping flag has changed, and gizmo needs to adjust accordingly */
	void ApplySnappingUpdateDeltas(const FInputDeviceRay& DragPos);

	/** Used when Surface Snapping flag has changed, and gizmo needs to adjust accordingly */
	void ApplySurfaceSnappingUpdateDeltas(const FInputDeviceRay& DragPos);

	void CacheSnappingStates();

	bool HasCurrentTRSSnappingChanged() const;
	bool HasSurfaceSnappingChanged() const;
	void HandleSnappingChanges(const FInputDeviceRay& DragPos);

	/** Returns true if the gizmo should be rendered. */
	UE_API virtual bool CanRender(IToolsContextRenderAPI* InRenderAPI) const;

	/** Returns true if the gizmo can be interacted with. */
	UE_API virtual bool CanInteract(const EViewportContext InViewportContext = EViewportContext::Focused) const override;

private:

	/** Get's the coordinate system for the current part, in case of override - ie. uniform translate is always in screen-space. */
	EToolContextCoordinateSystem GetCurrentPartCoordinateSystem() const;

	EToolContextCoordinateSystem GetCoordinateSystem() const;
	EGizmoTransformScaleType GetScaleType() const;

	FGizmoPerStateValueMaterialVariant GetAxisMaterials(const EAxis::Type InAxis);
	FGizmoPerStateValueLinearColor GetAxisColors(const EAxis::Type InAxis) const;
	FGizmoPerStateValueLinearColor GetAxisColors(const EAxisList::Type InAxis) const;
	UMaterialInterface* GetAxisMaterialSolid(const EAxis::Type InAxis);
	UMaterialInterface* GetAxisMaterialTranslucent(const EAxis::Type InAxis, const bool bInSubdued = false);

	/** Setup on hover functions */
	void SetupOnHoverFunctions();

	/**
	 * Get a material for the given static parameters (that require a separate material, not just different material parameters).
	 * Note that you should create an MID from this, rather than use it directly.
	 */
	UMaterialInterface* GetMaterialPermutation(const bool bInUseShading, const bool bInTranslucent, const bool bInVertexColored) const;

	// Returns whether the given part can interact, which can differ depending on the context (ie. arcball rotation is not interactable in gimbal rotation mode).
	UE_API bool CanInteractWithPart(const ETransformGizmoPartIdentifier InPartId) const;

	[[maybe_unused]] ETransformGizmoPartIdentifier GetHitPartInternal(const FInputDeviceRay& InDeviceRay, FInputRayHit& OutRayHit) const;

	/** Initializes the interaction plane according to the current state. */
	void InitializeInteractionPlane(const EAxisList::Type InAxisList);

	/** Initializes the interaction plane according to the current state. */
	void InitializeInteractionPlane(const UE::Geometry::FFrame3d& InFrame);

	/** Initializes both the interaction and aligned planes to the input value. */
	void InitializeInteractionPlanes(const UE::Geometry::FFrame3d& InFrame);

	/** Initializes screen-space interaction according to the input ray and current state. */
	void InitializeScreenInteraction(const FInputDeviceRay& InDeviceRay);

	/** Initializes the interaction sign such that positive movements in the interaction plane appear to be applied in the correct direction on the object. */
	void InitializeInteractionSign();

	void ApplyStyle();

	/** Shared axis and element accessors */

	EAxis::Type GetAxisForIndex(const int32 InIndex) const;
	EAxis::Type GetAxisForPart(const ETransformGizmoPartIdentifier InPartId) const;
	EAxisList::Type GetAxisListForPart(const ETransformGizmoPartIdentifier InPartId) const;
	
	ETransformGizmoPartIdentifier GetPartForAxisList(const EAxisList::Type InAxisList) const;

	void InitializeScreenSpaceTranslate(const FInputDeviceRay& InPressPos);
	void ResetDragTranslate(const FInputDeviceRay& DragPos);
	bool CameraFollowsMovement() const;

	void DuplicateSelection(bool bDroppingDuplicate = false) const;
	
	enum class EDragDuplicateContext
	{
		/** Whether we can capture the duplication gesture */
		Capture,
		/** Whether we can execute the duplication gesture */
		Action
	};
	bool AllowsDragDuplicate(EDragDuplicateContext Context) const;

	bool IsRotationPrecisionMode() const;
	double GetRotationPrecisionModeMultiplier() const;

	FEditorViewportClient* GetEditorViewportClient() const;

#pragma region Translation

	static int32 GetTranslateAxisIndexForPart(const ETransformGizmoPartIdentifier InPartId);

	void BeginTranslateDelta() const;
	void UpdateTranslateDelta() const;

#pragma endregion Translation

#pragma region Rotation

	/** Allows interception for debugging purposes, etc. */
	void SetRotateMode(const TEnumAsByte<EAxisRotateMode::Type> InRotateMode);

	template <typename MakePullQuatFunc, typename MakeArcQuatFunc>
	void OnClickDragRotateAxisInternal(
		const FInputDeviceRay& InDragPos,
		UGizmoElementRotateAxis* InElement,
		MakePullQuatFunc&& InMakePullQuat,
		MakeArcQuatFunc&& InMakeArcQuat);

	/** Gets a corresponding axis vector in world space for the given PartId.*/
	bool GetRotateAxisVectorForPart(const ETransformGizmoPartIdentifier InPartId, FVector& OutAxisVector) const;

	static int32 GetRotateAxisIndexForPart(const ETransformGizmoPartIdentifier InPartId);

	void BeginRotateAxisDelta(UGizmoElementRotateAxis* InElement) const;
	void UpdateRotateAxisDelta(UGizmoElementRotateAxis* InElement) const;

#pragma endregion Rotation

#pragma region Scale

	static int32 GetScaleAxisIndexForPart(const ETransformGizmoPartIdentifier InPartId);

	double GetScreenSign(const FVector2D& InScreenPosition);
	double GetScreenSign(const FVector2D& InScreenPosition, const FVector2D& InSignAxisOrigin);

	void BeginScaleDelta() const;
	void UpdateScaleDelta() const;

	/** Overload with a user-specified delta scale, which can differ to the actual one applied (ie. to match the cursor). */
	void UpdateScaleDelta(const FVector& InDeltaScale) const;

	/** Optionally modify the provided scale delta according to current settings, ie. percentage vs. offset. */
	void ModifyScaleDeltaForScaleType(const EGizmoTransformScaleType InScaleType, const FVector& InPreviousDelta, FVector& InOutScaleDelta);

#pragma endregion Scale

	void NudgeSelection(const FKey& InCurrentNudgeKey);

	UE::Editor::InteractiveToolsFramework::TNudgeDelta<FVector> GetTranslationNudge() const;
	UE::Editor::InteractiveToolsFramework::TNudgeDelta<FQuat> GetRotationNudge() const;
	UE::Editor::InteractiveToolsFramework::TNudgeDelta<FVector> GetScaleNudge() const;
	ENudgeDirection GetNudgeDirection(const FKey& InCurrentNudgeKey) const;

protected:

	// @todo: re-add this deprecation message in UTransformGizmo when ready
	// UE_DEPRECATED(5.6, "Use HoverAxisMaterial (etc) instead.")
	// UPROPERTY()
	// TObjectPtr<UMaterialInstanceDynamic> CurrentAxisMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> HoverAxisMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> SubdueAxisMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> SelectAxisMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> InteractAxisMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialXTranslucent;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialYTranslucent;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialZTranslucent;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialScreenSpaceTranslucent;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialXSubdued;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialYSubdued;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialZSubdued;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> UniformScaleMaterial;

	TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> TransformAdjuster = nullptr;

	/** Start transform */
	UPROPERTY()
	FTransform StartTransform = FTransform::Identity;

	// Interaction, best fit for viewport interaction (could be in screen space rather than world)
	UE::Geometry::FFrame3d InteractionPlane;

	// Aligns with the actual element being manipulated, may be the same or different to the interaction plane
	UE::Geometry::FFrame3d AlignedInteractionPlane;

	// View-facing
	UE::Geometry::FFrame3d ScreenPlane;

	// Somewhere between the interaction and screen plane, used for cursor
	UE::Geometry::FFrame3d CursorPlane;

	TArray<IHoverBehaviorTarget*> HoverElements;
	TArray<IClickDragBehaviorTarget*> ClickDragElements;

	/** Array of function pointers, indexed by gizmo part id, to handle click press behavior */
	TArray<TMemFunPtrType<false, UEditorTRSGizmo, void(const FInputDeviceRay& PressPos)>::Type> OnClickPressFunctions;
	
	/** Array of function pointers, indexed by gizmo part id, to handle drag start behavior.
	 * This is often fired after some cursor movement test, so the input cursor position will be different than OnClickPress. */
	TArray<TMemFunPtrType<false, UEditorTRSGizmo, void(const FInputDeviceRay& DragPos)>::Type> OnDragStartFunctions;

	/** Array of function pointers, indexed by gizmo part id, to handle click drag behavior */
	TArray<TMemFunPtrType<false, UEditorTRSGizmo, void(const FInputDeviceRay& DragPos)>::Type> OnClickDragFunctions;

	/** Array of function pointers, indexed by gizmo part id, to handle click release behavior */
	TArray<TMemFunPtrType<false, UEditorTRSGizmo, void(const FInputDeviceRay& ReleasePos)>::Type> OnClickReleaseFunctions;

	/** Customization function (to override default material or increment gizmo size for example) */
	TFunction<const FGizmoCustomization()> CustomizationFunction;

	float CursorRotation = 0.0f;

	TSharedPtr<class SGizmoCursor> DirectionalCursorWidget;

	//
	// The values below are used in the context of a single click-drag interaction, ie if bInInteraction = true
	// They otherwise should be considered uninitialized
	//

	/** Arc ball current rotation */
	FQuat CurrentRotation = FQuat::Identity;

	/** Actual rotate mode used (based on view dependant information). */
	TEnumAsByte<EAxisRotateMode::Type> RotateMode = EAxisRotateMode::Arc;

	/** Corrective sign so that an intended positive angular change maps to a clockwise rotation. Only apply once when needed (some operations already account for it). */
	int8 ClockwiseCorrectionSignForAxis = 1;

	/** Switch from tangential to normal projection based on the first mouse drag. */
	bool bTrySwitchingToNormalPull = false;

	/** Used to check if the gimbal mode is currently active (this is updated when ticking the gizmo) */
	bool bGimbalRotationMode = false;
	
	bool bAdditiveIndirectAxes = false;
	
	/** Set to true to only update the pivot point */
	bool bOnlyUpdatePivot = false;
	
	/** Set to true when an attempt to scale a target has no result */
	bool bDetectedNonStandardScaling = false;
	
	bool bCameraFollowsMovement = false;
	bool bResetDragTranslate = false;

	bool bCtrlKeyDown = false;
	bool bAltKeyDown = false;
	bool bShiftKeyDown = false;

	bool bRotationPrecisionModeDirty = false;

	/** Screen space precision rotation requires to keep track of rotation amount, to prevent UI inconsistencies */
	double ScreenSpacePrecisionVisualOffset = 0.0;

	/** Used when hiding the cursor */
	FIntPoint CachedCursorPosition;
	
	FVector CachedViewLocation;

	bool bTRSSnapDirty = false;
	bool bSurfaceSnapDirty = false;

	bool bCachedTranslationSnap = false;
	bool bCachedRotationSnap = false;
	bool bCachedScaleSnap = false;
	bool bCachedSurfaceSnap = false;

	/** Used when temporary rotation snapping is turned on, to reset rotation */
	FQuat CumulativeRotationDelta;
	FVector CumulativeDragDelta;

private:
	TWeakObjectPtr<USingleClickAndDragBehavior> ClickAndDragBehaviorWeak = nullptr;

	TSharedPtr<UE::Editor::InteractiveToolsFramework::FTransformGizmoProfiler> Profiler;
	
	/** Defines various interaction state behavior. */
	struct FGizmoElementInteraction
	{
	public:
		/** A set of parts. */
		struct FPartSet
		{
			struct FPartRange
			{
				ETransformGizmoPartIdentifier Start;
				ETransformGizmoPartIdentifier End;
				TOptional<ETransformGizmoPartIdentifier> Exclude;
			};
			
			FPartSet(const std::initializer_list<ETransformGizmoPartIdentifier> InParts)
				: Parts(InParts)
			{ }

			explicit FPartSet(const FPartRange& InRange)
				: PartRange(InRange)
			{ }
			
			TArray<ETransformGizmoPartIdentifier> Parts;

			/**
			 * If specified, the given part range is iterated, excluding the "Exclude" value (if not none).
			 * Parts are also iterated over.
			 */
			TOptional<FPartRange> PartRange;

			void ForEachPart(const TFunctionRef<void(const ETransformGizmoPartIdentifier)>& InFunc);
		};

	public:
		/** Parts to also Hover (see HoverElements) when the given part is Hovered. This applies to behaviors/input, not InteractionState. */
		static TMap<ETransformGizmoPartIdentifier, FPartSet> HoverParts;

		/** Parts to also Click/Drag (see ClickDragElements) when the given part is Hovered. This applies to behaviors/input, not InteractionState. */
		static TMap<ETransformGizmoPartIdentifier, FPartSet> ClickDragParts;
		
		/** Parts to Subdue when the Part specified by Key is Interacted with. */
		static TMap<ETransformGizmoPartIdentifier, FPartSet> SubdueOnInteractParts;

		/** Parts to Hide when the Part specified by Key is Interacted with. */
		static TMap<ETransformGizmoPartIdentifier, FPartSet> HideOnInteractParts;

		/** Parts to (also) Interact when the Part specified by Key is Interacted with. */
		static TMap<ETransformGizmoPartIdentifier, FPartSet> InteractGroupParts;
	} GizmoElementInteraction;

	mutable struct FGizmoDebugData
	{
		/** Determines whether certain data is displayed, ie. drag operation deltas. */
		bool bIsEditing = false;

		FRay InteractionRay;
		FVector2D InteractionScreenPos;

		FTransform TransformStart;
		FTransform TransformCurrent;

		FTransform InteractionStart;
		FTransform InteractionCurrent;
		FVector InteractionPlaneNormal;

		TSet<ETransformGizmoPartIdentifier> LastHitDeltaParts;

		/** Can indicate a 2D drag direction, etc. */
		FVector2D InteractionScreenDirection;

		FVector CursorDirectionWS;
		FVector2D CursorDirectionSS;

		FInputDeviceRay LastDeviceRay;
		FInputRayHit LastRayHit;

		/** Debug attributes to display the pull direction */
		bool bDebugRotate = false;
		FVector DebugDirection = FVector::ZeroVector;
		FVector DebugClosest = FVector::ZeroVector;
		FVector DebugNormalRemoved = FVector::ZeroVector;
		FVector DebugNormalSkip = FVector::ZeroVector;
		double InteractionAngleStart;
		double InteractionAngleCurrent;
		double InteractionRadius;
		double HitAngle;
		FVector PointOnCircleDirection = FVector::ZeroVector;

		FVector ReferencePoint = FVector::ZeroVector;

		FVector Test = FVector::ZeroVector;
		FVector2D Test2 = FVector2D::ZeroVector;
		FVector Test3 = FVector::ZeroVector;

		FVector2D Test2D1 = FVector2D::ZeroVector;
		FVector2D Test2D2 = FVector2D::ZeroVector;
		FVector2D Test2D3 = FVector2D::ZeroVector;

		FString DebugString; // Can be current rotation mode, etc.

		// Utility to get hit position from LastDeviceRay, LastRayHit pair
		FVector GetHitPosition() const
		{
			return LastDeviceRay.WorldRay.PointAt(LastRayHit.HitDepth);
		}
	} DebugData;


	struct FGizmoNudgeData
	{
		/** List of currently pressed nudge keys */
		TArray<FKey> PressedKeys;

		/** Elapsed time since last nudge */
		float TimeSinceLastNudge = 0.0f;

		FKey GetCurrentKey() const
		{
			if (PressedKeys.IsEmpty())
			{
				return EKeys::AnyKey;
			}

			return PressedKeys.Top();
		}

		void Reset()
		{
			PressedKeys.Empty();
			TimeSinceLastNudge = 0.0f;
		}
	};

	FGizmoNudgeData NudgeData;
	
	struct FModeAxisOverride
	{
		EGizmoTransformMode Mode;
		EAxisList::Type AxisToDraw;
	};
	TOptional<FModeAxisOverride> ModeAxisOverride;

	TWeakObjectPtr<const UTypedElementSelectionSet> SelectionSetWeak;
};

#undef UE_API
