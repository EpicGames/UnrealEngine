// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorGizmos/TransformGizmo.h"
#include "Math/MathFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorTransformGizmo.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UObject;

/**
 * UEditorTransformGizmo handles Editor-specific functionality for the TransformGizmo,
 * applied to a UEditorTransformProxy target object.
 */
UCLASS(MinimalAPI)
class UEditorTransformGizmo : public UTransformGizmo
{
	GENERATED_BODY()

	/**  UTransformGizmo override */
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	
protected:

	/** Apply translate delta to transform proxy */
	UE_API virtual void ApplyTranslateDelta(const FVector& InTranslateDelta) override;

	/** Apply rotate delta to transform proxy */
	UE_API virtual void ApplyRotateDelta(const FQuat& InRotateDelta) override;

	/** Apply scale delta to transform proxy */
	UE_API virtual void ApplyScaleDelta(const FVector& InScaleDelta) override;

	/**  UTransformGizmo override */
	UE_API virtual void SetActiveTarget(
		UTransformProxy* Target,
		IToolContextTransactionProvider* TransactionProvider = nullptr,
		IGizmoStateTarget* InStateTarget = nullptr) override;

	/**
	 * Functions to listen to gizmo transform begin/end.
	 * They are currently used to set data on the legacy widget as some viewport clients rely on current axis to be set.
	 */
	UE_API void OnGizmoTransformBegin(UTransformProxy* TransformProxy) const;
	UE_API void OnGizmoTransformEnd(UTransformProxy* TransformProxy) const;;
};

#undef UE_API
