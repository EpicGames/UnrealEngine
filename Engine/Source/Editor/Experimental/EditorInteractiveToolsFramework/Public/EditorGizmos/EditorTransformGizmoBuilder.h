// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorInteractiveGizmoSelectionBuilder.h"
#include "InteractiveGizmoBuilder.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"

#include "EditorTransformGizmoBuilder.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UInteractiveGizmo;
class UObject;
class UEditorTransformGizmoContextObject;
struct FToolBuilderState;

UCLASS(MinimalAPI)
class UEditorTransformGizmoBuilder : public UInteractiveGizmoBuilder, public IEditorInteractiveGizmoSelectionBuilder
{
	GENERATED_BODY()

public:

	// UEditorInteractiveGizmoSelectionBuilder interface 
	UE_API virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
	UE_API virtual void UpdateGizmoForSelection(UInteractiveGizmo* Gizmo, const FToolBuilderState& SceneState) override;

	// If set, this function will be passed to UTransformGizmo instances to override the default material and display size.
	TFunction<const FGizmoCustomization()> CustomizationFunction;

protected:
	static UE_API const UEditorTransformGizmoContextObject* GetTransformGizmoContext(const FToolBuilderState& InSceneState);
};


#undef UE_API
