// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Containers/Set.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UGizmoEditorStateTarget;
class UEditorTransformGizmoContextObject;
class UTransformGizmo;
class UTransformProxy;
class FEditorViewportClient;

/**
 * FEditorTransformGizmoDataBinder is a helper class for binding a UTransformGizmo to a FEditorModeTools
 */

class FEditorTransformGizmoDataBinder : public TSharedFromThis<FEditorTransformGizmoDataBinder>
{
public:

	UE_API virtual ~FEditorTransformGizmoDataBinder();
	
	/**
	 * Makes it so that the gizmo binder attaches to any gizmos created by the context object
	 * in the future. The binding is automatically removed if FEditorTransformGizmoDataBinder is
	 * destroyed.
	 */
	UE_API void BindToGizmoContextObject(UEditorTransformGizmoContextObject* InContextObject);

	/** Used for binding to context object, called from the OnGizmoCreated delegate. */
	UE_API void BindToUninitializedGizmo(UTransformGizmo* Gizmo);
	
	/** Binds to a specific gizmo for tracking. Requires ActiveTarget to be set. */
	UE_API void BindToInitializedGizmo(UTransformGizmo* InGizmo, UTransformProxy* InProxy);
	
	/** Unbinds from a given gizmo. Done automatically for gizmos when their ActiveTarget is cleared. */
	UE_API void UnbindFromGizmo(UTransformGizmo* InGizmo, UTransformProxy* InProxy);

private:

	/** List of the gizmos currently bound to this so that we ensure that they are actually unbound on destruction.
	 * Note that this should not happen and bound gizmos should have called their Shutdown function before this being destroyed.
	 */
	TSet<TWeakObjectPtr<UTransformGizmo>> BoundGizmos;

	/** Weak ptr to the context so we can interface with the current mode manager */
	TWeakObjectPtr<UEditorTransformGizmoContextObject> WeakContext;

	UE_API FEditorViewportClient* GetViewportClient() const;

	UE_API void OnProxyBeginTransformEdit(UTransformProxy* InTransformProxy);
	UE_API void OnProxyTransformChanged(UTransformProxy* InTransformProxy, FTransform InTransform);
	UE_API void OnProxyEndTransformEdit(UTransformProxy* InTransformProxy);
	
	UE_API void OnProxyBeginPivotEdit(UTransformProxy* InTransformProxy);
	UE_API void OnProxyPivotChanged(UTransformProxy* InTransformProxy, FTransform InTransform);
	UE_API void OnProxyEndPivotEdit(UTransformProxy* InTransformProxy);

	bool bHasTransformChanged = false;
};

#undef UE_API
