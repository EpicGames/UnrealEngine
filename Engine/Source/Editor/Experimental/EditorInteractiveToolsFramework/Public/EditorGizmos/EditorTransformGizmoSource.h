// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "EditorGizmos/TransformGizmoInterfaces.h"
#include "Math/Axis.h"
#include "ToolContextInterfaces.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UnrealWidgetFwd.h"

#include "EditorTransformGizmoSource.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UEditorTransformGizmoContextObject;
class FEditorModeTools;
class FEditorViewportClient;
class FSceneView;

namespace FEditorTransformGizmoUtil
{
	/** Convert UE::Widget::EWidgetMode to ETransformGizmoMode*/
	EGizmoTransformMode GetGizmoMode(UE::Widget::EWidgetMode InWidgetMode);

	/** Convert EEditorGizmoMode to UE::Widget::EWidgetMode*/
	UE::Widget::EWidgetMode GetWidgetMode(EGizmoTransformMode InGizmoMode);
};

/**
 * UEditorTransformGizmoSource is an ITransformGizmoSource implementation that provides
 * current state information used to configure the Editor transform gizmo.
 */
UCLASS(MinimalAPI)
class UEditorTransformGizmoSource : public UObject, public ITransformGizmoSource
{
	GENERATED_BODY()
public:
	
	/**
	 * @return The current display mode for the Editor transform gizmo
	 */
	UE_API virtual EGizmoTransformMode GetGizmoMode() const override;

	/**
	 * @return The current axes to draw for the specified mode
	 */
	UE_API virtual EAxisList::Type GetGizmoAxisToDraw(EGizmoTransformMode InWidgetMode) const override;

	/**
	 * @return The coordinate system space (world or local) to display the widget in
	 */
	UE_API virtual EToolContextCoordinateSystem GetGizmoCoordSystemSpace() const override;

	/**
	 * Returns a scale factor for the gizmo
	 */
	UE_API virtual float GetGizmoScale() const override;

	/**
	 * Whether the gizmo is visible
	 */
	UE_API virtual bool GetVisible(const EViewportContext InViewportContext = EViewportContext::Focused) const override;

	/* 
	 * Returns whether the gizmo can interact.
	 * Note that this can be true even if the gizmo is hidden to support indirect manipulation in game mode.
	 */
	UE_API virtual bool CanInteract(const EViewportContext InViewportContext = EViewportContext::Focused) const override;

	/**
 	 * Get current scale type
	 */
	UE_API virtual EGizmoTransformScaleType GetScaleType() const override;

	/** Get rotation context */
	UE_API virtual const FRotationContext& GetRotationContext() const override;
	
	static UE_API UEditorTransformGizmoSource* CreateNew(
		UObject* Outer = (UObject*)GetTransientPackage(),
		const UEditorTransformGizmoContextObject* InContext = nullptr);

protected:

	UE_API const FEditorModeTools& GetModeTools() const;

	UE_API const FEditorViewportClient* GetViewportClient(const EViewportContext InViewportContext = EViewportContext::Focused) const;
	
	TWeakObjectPtr<const UEditorTransformGizmoContextObject> WeakContext = nullptr;
};

#undef UE_API
