// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TransformGizmoInterfaces.h"
#include "UObject/Object.h"
#include "Containers/UnrealString.h"

#include "EditorTransformGizmoUtil.generated.h"

struct FGizmosParameters;
class UInteractiveToolsContext;
class UInteractiveToolManager;
class UTransformGizmo;
class FEditorModeTools;
class FEditorViewportClient;
class UEditorInteractiveGizmoManager;
class UEditorTransformGizmoContextObject;
class UTransformProxy;
class FEditorTransformGizmoDataBinder;

namespace UE
{
	
namespace EditorTransformGizmoUtil
{
	/**
	 * The functions below are helper functions that simplify usage of a UEditorTransformGizmoContextObject
	 * that is registered as a ContextStoreObject in an InteractiveToolsContext
	 */

	/**
	 * If one does not already exist, create a new instance of UEditorTransformGizmoContextObject and add it to the
	 * ToolsContext's ContextObjectStore
	 * @return true if the ContextObjectStore now has a UEditorTransformGizmoContextObject (whether it already existed, or was created)
	 */
	EDITORINTERACTIVETOOLSFRAMEWORK_API bool RegisterTransformGizmoContextObject(FEditorModeTools* InModeTools);

	/**
	 * Remove any existing UEditorTransformGizmoContextObject from the ToolsContext's ContextObjectStore
	 * @return true if the ContextObjectStore no longer has a UEditorTransformGizmoContextObject (whether it was removed, or did not exist)
	 */
	EDITORINTERACTIVETOOLSFRAMEWORK_API bool UnregisterTransformGizmoContextObject(FEditorModeTools* InModeTools);

	/**
	 * Spawn an editor like Transform Gizmo. ToolManager's ToolsContext must have a UEditorTransformGizmoContextObject registered 
	 */
	EDITORINTERACTIVETOOLSFRAMEWORK_API UTransformGizmo* CreateTransformGizmo(
		UInteractiveToolManager* InToolManager, const FString& InInstanceIdentifier = FString(), void* InOwner = nullptr);
	
	/**
	 * Spawn an editor like Transform Gizmo. ToolManager's ToolsContext must have a UEditorTransformGizmoContextObject registered 
	 */
	EDITORINTERACTIVETOOLSFRAMEWORK_API UTransformGizmo* GetDefaultTransformGizmo(UInteractiveToolManager* InToolManager);

	/**
	 * Returns the existing default Transform Gizmo if any. 
	 */
	EDITORINTERACTIVETOOLSFRAMEWORK_API UTransformGizmo* FindDefaultTransformGizmo(UInteractiveToolManager* InToolManager);

	/**
     * Removes the existing default Transform Gizmo if any. 
     */
	EDITORINTERACTIVETOOLSFRAMEWORK_API void RemoveDefaultTransformGizmo(UInteractiveToolManager* InToolManager);
}
	
}

/**
 * UEditorTransformGizmoContextObject is a utility object that registers a Gizmo Builder for UTransformGizmo.
 * (see UCombinedTransformGizmoContextObject for more details)
 */

UCLASS(Transient, MinimalAPI)
class UEditorTransformGizmoContextObject : public UObject
{
	GENERATED_BODY()

public:
	
	EDITORINTERACTIVETOOLSFRAMEWORK_API void RegisterGizmosWithManager(UInteractiveToolManager* InToolManager);
	EDITORINTERACTIVETOOLSFRAMEWORK_API void UnregisterGizmosWithManager(UInteractiveToolManager* InToolManager);

	EDITORINTERACTIVETOOLSFRAMEWORK_API void Initialize(FEditorModeTools* InModeTools);
	EDITORINTERACTIVETOOLSFRAMEWORK_API void Shutdown();
	
	EDITORINTERACTIVETOOLSFRAMEWORK_API void SetGizmosEnabled(bool bInEnabled);

	EDITORINTERACTIVETOOLSFRAMEWORK_API UTransformGizmo* CreateTransformGizmo(
		UEditorInteractiveGizmoManager* InGizmoManager, const FString& InInstanceIdentifier, void* InOwner) const;
	
	EDITORINTERACTIVETOOLSFRAMEWORK_API FEditorModeTools* GetModeTools() const;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGizmoCreated, UTransformGizmo*);
	EDITORINTERACTIVETOOLSFRAMEWORK_API FOnGizmoCreated& OnGizmoCreatedDelegate();

	UPROPERTY(Transient)
	FRotationContext RotationContext;
	
	/** Temporary exposure for the purposes of flowing into legacy pivot-only manipulations */
	UPROPERTY(Transient)
	bool bIsManipulatingPivot = false;
	
private:

	EDITORINTERACTIVETOOLSFRAMEWORK_API void UpdateGizmo(const TArray<FEditorViewportClient*>& InViewportClients) const;
	
	EDITORINTERACTIVETOOLSFRAMEWORK_API void InitializeGizmoManagerBinding();
	EDITORINTERACTIVETOOLSFRAMEWORK_API void InitializeViewportsBinding();
	
	EDITORINTERACTIVETOOLSFRAMEWORK_API void RemoveGizmoManagerBinding();
	EDITORINTERACTIVETOOLSFRAMEWORK_API void RemoveViewportsBinding();

	FOnGizmoCreated OnGizmoCreated;
	
	FEditorModeTools* ModeTools = nullptr;
	FDelegateHandle ViewportClientsChangedHandle;
	FDelegateHandle LevelViewportClientsChangedHandle;
	FDelegateHandle UseNewGizmosChangedHandled;
	FDelegateHandle GizmosParametersChangedHandle;
	bool bGizmosRegistered = false;

	TSharedPtr<FEditorTransformGizmoDataBinder> DataBinder;
};
