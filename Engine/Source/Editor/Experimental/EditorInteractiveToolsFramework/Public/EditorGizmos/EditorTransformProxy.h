// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/TransformProxy.h"
#include "CoreMinimal.h"
#include "Math/Axis.h"
#include "Math/MathFwd.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Misc/AssertionMacros.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakInterfacePtr.h"

#include "EditorTransformProxy.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class IAssetEditorContextInterface;
class UEditorTransformGizmoContextObject;
class UTypedElementViewportInteraction;
class FEditorViewportClient;
class UObject;

/**
 * UEditorTransformProxy is a derivation of UTransformProxy that
 * returns the transform that defines the current space of the default
 * Editor transform gizmo for a given mode manager / viewport.
 * 
 * This defaults internally to GLevelEditorModeManager() if none provided when constructing the proxy.
 */
UCLASS(MinimalAPI, Transient)
class UEditorTransformProxy : public UTransformProxy
{
	GENERATED_BODY()

public:

	UE_API UEditorTransformProxy();

	/**
	 * @return the stored transform for currently selected objects.
	 */
	UE_API virtual FTransform GetTransform() const override;

	/**
	 * Unimplemented - all updates to the Editor transform proxy MUST be made by calling the Input delta methods.
	 */
	virtual void SetTransform(const FTransform& Transform) override 
	{
		check(false);
	}

	UE_API virtual void BeginTransformEditSequence() override;

	/** Input translate delta to be applied in world space of the current transform. */
	UE_API virtual void InputTranslateDelta(const FVector& InDeltaTranslate, EAxisList::Type InAxisList);

	/** Input rotation delta to be applied in local space of the current transform. */
	UE_API virtual void InputRotateDelta(const FRotator& InDeltaRotate, EAxisList::Type InAxisList);

	/** Input scale delta to be applied in local space of the current transform. */
	UE_API virtual void InputScaleDelta(const FVector& InDeltaScale, EAxisList::Type InAxisList);

	/** Set legacy widget axis temporarily because FEditorViewportClient overrides may expect it */
	UE_API void SetCurrentAxis(const EAxisList::Type InAxisList) const;

	/** Set whether to use FEditorModeTools::GetWidgetScale as the Transform scale. */
	UE_API void SetUseLegacyWidgetScale(const bool bInUseLegacyWidgetScale);

	static UE_API UEditorTransformProxy* CreateNew(const UEditorTransformGizmoContextObject* InContext = nullptr, IAssetEditorContextInterface* InAssetEditorContext = nullptr);

	UE_DEPRECATED(5.7, "Use CreateNew that accepts an AssetEditorContext instead")
	static UE_API UEditorTransformProxy* CreateNew(const UEditorTransformGizmoContextObject* InContext);

private:

	/** Recalculate main SharedTransform based on the current selection set. */
	UE_API virtual void UpdateSharedTransform() override;

	UE_API FTypedElementListConstRef GetElementsToManipulate() const;

	UE_API FEditorViewportClient* GetViewportClient() const;

private:

	/** Whether to use FEditorModeTools::GetWidgetScale as the Transform scale. If false, scale is derived from the underlying objects. */
	UPROPERTY()
	bool bUseLegacyWidgetScale = true;

	/** The viewport interaction instance, providing handlers for selected Typed Elements. */
	UPROPERTY()
	TObjectPtr<UTypedElementViewportInteraction> ViewportInteraction;

	TWeakObjectPtr<const UEditorTransformGizmoContextObject> WeakContext;
	TWeakInterfacePtr<IAssetEditorContextInterface> WeakAssetEditorContext;
};

#undef UE_API
