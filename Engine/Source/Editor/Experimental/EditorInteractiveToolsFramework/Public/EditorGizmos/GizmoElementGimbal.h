// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoElementGroup.h"
#include "TransformGizmoInterfaces.h"

#include "GizmoElementGimbal.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/**
 * Gimbal rotation group object intended to be used as part of 3D Rotation Gizmos
 * This group expects three rotation sub-elements
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementGimbal : public UGizmoElementGroup
{
	GENERATED_BODY()

public:

	//~ Begin UGizmoElementBase Interface.
	UE_API virtual void Render(IToolsContextRenderAPI* InRenderAPI, const FRenderTraversalState& InRenderState) override;
	UE_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Add object to group.
	UE_API virtual void Add(UGizmoElementBase* InElement) override;

	template <class ElementType UE_REQUIRES(std::is_base_of_v<UGizmoElementBase, ElementType>)>
	ElementType* GetAxisElement(const EAxis::Type InAxis) const
	{
		// @note: this makes the huge assumption that the elements are in X, Y, Z order.
		const int32 AxisIndex = static_cast<int32>(InAxis) - 1;
		if (ensure(Elements.IsValidIndex(AxisIndex)))
		{
			return Cast<ElementType>(Elements[AxisIndex]);
		}

		return nullptr;
	}

	UPROPERTY()
	FRotationContext RotationContext;
};

#undef UE_API
