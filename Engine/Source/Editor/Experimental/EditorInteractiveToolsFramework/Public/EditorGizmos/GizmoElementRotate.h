// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorGizmoElementShared.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"

#include "GizmoElementRotate.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/** Bitflags controlling visibility of rotation delta visual elements during interaction. */
UENUM(meta = (Bitflags, ShowFlags))
enum class EGizmoElementRotateShowFlags : uint32
{
	/** No flags set. */
	None = 0 UMETA(Hidden),
	/** Show the filled arc indicating the rotation delta. */
	DeltaArc = 1 << 0,
	/** Show the numeric angle label. */
	DeltaLabel = 1 << 1,

	/** All delta visual elements enabled. */
	All = DeltaArc | DeltaLabel
};
ENUM_CLASS_FLAGS(EGizmoElementRotateShowFlags);

/** Visual style parameters for a single rotation axis element (ring + delta arc + origin marker + value label). */
USTRUCT(MinimalAPI)
struct FGizmoElementRotateAxisStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	UE_API FGizmoElementRotateAxisStyle();

	/** Controls which delta visual elements are shown during interaction. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (Bitmask))
	EGizmoElementRotateShowFlags ShowFlags = EGizmoElementRotateShowFlags::All;

	/** Radius of the rotation ring in world units (before multiplier). */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 1.0f))
	float Radius = 70.0f;

	/** Multiplier applied to Radius, typically driven by a user setting. */
	UPROPERTY()
	float RadiusMultiplier = 1.0f;

	/** Thickness of the rotation ring line in world units. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.1f))
	float LineThickness = 1.5f;

	/** Radius of the origin sphere drawn at the rotation center during interaction, in world units. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.1f))
	float OriginRadius = 3.0f;

	/** Outline thickness of the origin sphere, in world units. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.1f))
	float OriginLineThickness = 0.25f;

	/** Thickness of the line connecting the Origin to cursor location. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.1f))
	float OriginToCursorLineThickness = 0.25f;

	/** Dash length of the line connecting the Origin to cursor location. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.1f))
	float OriginToCursorLineDashSpacing = 2.0f;

	/** Dash gap length of the line connecting the Origin to cursor location. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.1f))
	float OriginToCursorLineDashGapSpacing = 2.0f;

	/** Color of the line connecting the Origin to cursor location. */
	UPROPERTY(EditAnywhere, Category = "Rotate")
	FLinearColor OriginToCursorLineColor = FLinearColor::Black;

	/** Fill opacity of the delta arc. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.0f, ClampMax = 1.0f))
	float DeltaFillOpacity = 0.4f;

	/** Thickness of the delta arc lines. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.1f))
	float DeltaStrokeThickness = .625f;

	/** Stroke/line opacity of the delta arc. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.0f, ClampMax = 1.0f))
	float DeltaStrokeOpacity = 1.0f;

	/** HSV modifier for the delta arc fill color. R = hue offset, G = saturation multiplier, B = value multiplier. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (HideAlphaChannel))
	FLinearColor DeltaFillHSVModifier = FLinearColor(0.0f, 0.5f, 2.0f, 1.0f);

	/** Background color/alpha of the text box. */
	UPROPERTY(EditAnywhere, Category = "Rotate")
	FLinearColor DeltaTextBackgroundColor = FLinearColor(0.141f, 0.141f, 0.141f, 0.65f);

	/** Cursor color. */
	UPROPERTY(EditAnywhere, Category = "Rotate")
	FLinearColor CursorColor = FLinearColor::Black;

	/** Number of segments used when generating the rotation ring geometry. */
	UPROPERTY()
	int32 NumSegments = 64;

	/** Number of inner slices used for the delta arc fill geometry. */
	UPROPERTY()
	int32 NumInnerSlices = 8;

	/** Material used for rendering the delta arc fill. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DeltaMaterial;

	/** Material used for vertex-color-based rendering of axis elements. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> VertexColorMaterial;

	/** Returns true if all style properties are equal. */
	friend bool operator==(const FGizmoElementRotateAxisStyle& InLeft, const FGizmoElementRotateAxisStyle& InRight)
	{
		return static_cast<const FGizmoStyleBase&>(InLeft) == static_cast<const FGizmoStyleBase&>(InRight)
			&& InLeft.ShowFlags == InRight.ShowFlags
			&& InLeft.Radius == InRight.Radius
			&& InLeft.RadiusMultiplier == InRight.RadiusMultiplier
			&& InLeft.LineThickness == InRight.LineThickness
			&& InLeft.OriginRadius == InRight.OriginRadius
			&& InLeft.OriginLineThickness == InRight.OriginLineThickness
			&& InLeft.OriginToCursorLineThickness == InRight.OriginToCursorLineThickness
			&& InLeft.OriginToCursorLineDashSpacing == InRight.OriginToCursorLineDashSpacing
			&& InLeft.OriginToCursorLineDashGapSpacing == InRight.OriginToCursorLineDashGapSpacing
			&& InLeft.OriginToCursorLineColor == InRight.OriginToCursorLineColor
			&& InLeft.DeltaFillOpacity == InRight.DeltaFillOpacity
			&& InLeft.DeltaStrokeThickness == InRight.DeltaStrokeThickness
			&& InLeft.DeltaStrokeOpacity == InRight.DeltaStrokeOpacity
			&& InLeft.DeltaFillHSVModifier == InRight.DeltaFillHSVModifier
			&& InLeft.DeltaTextBackgroundColor == InRight.DeltaTextBackgroundColor
			&& InLeft.CursorColor == InRight.CursorColor
			&& InLeft.NumSegments == InRight.NumSegments
			&& InLeft.NumInnerSlices == InRight.NumInnerSlices
			&& InLeft.DeltaMaterial == InRight.DeltaMaterial
			&& InLeft.VertexColorMaterial == InRight.VertexColorMaterial;
	}

	/** Returns true if any style property differs. */
	friend bool operator!=(const FGizmoElementRotateAxisStyle& InLeft, const FGizmoElementRotateAxisStyle& InRight)
	{
		return !(InLeft == InRight);
	}
};

/** Visual style parameters for the arcball (trackball) rotation element. */
USTRUCT(MinimalAPI)
struct FGizmoElementRotateArcballStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	UE_API FGizmoElementRotateArcballStyle();

	/** Radius of the arcball sphere in world units. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 1.0f))
	float Radius = 70.0f;
};

/** Composite style for the entire rotation gizmo, combining per-axis and arcball sub-styles with screen-space circle settings. */
USTRUCT(MinimalAPI)
struct FGizmoElementRotateStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	/** Shared style for all single-axis rotation ring elements. */
	UPROPERTY(EditAnywhere, Category = "Rotate")
	FGizmoElementRotateAxisStyle AxisStyle;

	/** Style for the arcball (trackball) rotation element. */
	UPROPERTY(EditAnywhere, Category = "Rotate")
	FGizmoElementRotateArcballStyle ArcballStyle;

	/** Controls which delta visual elements are shown during interaction at the group level. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (Bitmask))
	EGizmoElementRotateShowFlags ShowFlags = EGizmoElementRotateShowFlags::All;

	/** Distance offset from the rotation ring radius for the screen-space circle, in world units. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.01f))
	float ScreenSpaceRadiusOffset = 12.6f;

	/** The thickness of the screen space rotation handle arc. */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ClampMin = 0.01f))
	float ScreenSpaceLineThickness = 0.625f;

	/** Color of the screen-space rotation circle (outer ring). */
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (HideAlphaChannel))
	FLinearColor ScreenSpaceCircleColor = FLinearColor(0.9f, 0.9f, 0.9f);
};

/** Interaction options for rotation gizmo elements, controlling how user input maps to rotation deltas. */
USTRUCT(MinimalAPI, Category = "Rotate")
struct FGizmoElementRotateInteraction
{
	GENERATED_BODY()

	/** The default rotation mode to use with the Gizmo. */
	static constexpr EAxisRotateMode::Type GetDefaultRotateMode() { return EAxisRotateMode::ScreenArc; }

	/** Optionally override the project-level default rotation mode (Arc or Pull). */
	UPROPERTY(EditAnywhere, Category = "Input")
	TOptional<TEnumAsByte<EAxisRotateMode::Type>> RotateMode;

	/**
	 * The explicit drag direction corresponding with view-relative rotation. (1.0, 0.0) means left-to-right movement rotates clockwise, right-to-left moves counter-clockwise. Vertical movement wouldn't do anything.
	 * Only applicable to Pull rotation.
	 * If not set, uses a "pull-string" approach where the direction is determined by the cursor position relative to the origin.
	 */
	UPROPERTY(EditAnywhere, Category = "Input", meta = (DisplayName = "Drag Direction"))
	TOptional<FVector2D> Direction;

	/** Sensitivity multiplier for Pull rotation mode. */
	UPROPERTY(EditAnywhere, Category = "Input")
	float PullMultiplier = 0.25f;

	/** Sensitivity multiplier for Arc rotation mode. */
	UPROPERTY(EditAnywhere, Category = "Input")
	float ArcMultiplier = 1.0f;
	
	/** Returns the overridden RotateMode if set, otherwise use the Editor Setting. */
	UE_API EAxisRotateMode::Type GetRotateMode() const;

	/** Arc rotation damping factor (precision mode) */
	UPROPERTY(EditAnywhere, Category = "Precision Mode", meta = (ClampMin = 0.01f))
	float ArcballRotationPrecisionDamping = 0.05f;

	/** Axis rotation damping factor (precision mode) */
	UPROPERTY(EditAnywhere, Category = "Precision Mode", meta = (ClampMin = 0.01f))
	float AxisRotationPrecisionDamping = 0.025f;

	/** Screen Space rotation damping factor (precision mode) */
	UPROPERTY(EditAnywhere, Category = "Precision Mode", meta = (ClampMin = 0.01f))
	float ScreenSpaceRotationPrecisionDamping = 0.025f;

	/** Arc rotation boost factor (precision mode) */
	UPROPERTY(EditAnywhere, Category = "Precision Mode", meta = (ClampMin = 0.01f))
	float ArcballRotationPrecisionBoost = 2.0f;

	/** Axis rotation boost factor (precision mode) */
	UPROPERTY(EditAnywhere, Category = "Precision Mode", meta = (ClampMin = 0.01f))
	float AxisRotationPrecisionBoost = 4.0f;

	/** Screen Space rotation boost factor (precision mode) */
	UPROPERTY(EditAnywhere, Category = "Precision Mode", meta = (ClampMin = 0.01f))
	float ScreenSpaceRotationPrecisionBoost = 4.0f;
};

#undef UE_API
