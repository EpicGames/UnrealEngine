// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorGizmoElementShared.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"

#include "GizmoElementScale.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/** Bitflags controlling visibility of scale delta visual elements during interaction. */
UENUM(meta = (Bitflags, ShowFlags))
enum class EGizmoElementScaleShowFlags : uint32
{
	/** No flags set. */
	None = 0 UMETA(Hidden),
	/** Show the delta line from the axis handle to the scaled position. */
	DeltaLine = 1 << 0,
	/** Show the numeric scale label. */
	DeltaLabel = 1 << 1,

	/** All delta visual elements enabled. */
	All = DeltaLine | DeltaLabel
};
ENUM_CLASS_FLAGS(EGizmoElementScaleShowFlags);

/** Visual style parameters for a single scale axis element (cylinder body + arrow head handle + delta visuals). */
USTRUCT(MinimalAPI)
struct FGizmoElementScaleAxisStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	UE_API FGizmoElementScaleAxisStyle();

	/** Length of the axis body in world units (before multiplier). */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 1.0f))
	float AxisLength = 62.0f;

	/** Multiplier applied to AxisLength, typically driven by a user setting. */
	UPROPERTY()
	float AxisLengthMultiplier = 1.0f;

	/** Gap between the gizmo origin and the start of the axis body, in world units. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 1.0f))
	float AxisOffsetFromCenter = 20.0f;

	/** Multiplier for the arrow head size, typically driven by a user setting. */
	UPROPERTY()
	float HandleSizeMultiplier = 1.0f;

	/** Thickness of the axis line in world units. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	float LineThickness = 1.5f;

	/** Size of the cube arrow head at the end of the axis, in world units. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	float HeadSize = 8.0f;

	/** Size of the uniform (center) scale handle, in world units. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	float UniformSize = 12.0f;

	/** Controls which delta visual elements are shown during interaction. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (Bitmask))
	EGizmoElementScaleShowFlags ShowFlags = EGizmoElementScaleShowFlags::All;

	/** Optional override for the delta line thickness. Falls back to LineThickness if not set. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	TOptional<float> DeltaLineThickness;

	/** Length of each dash segment in the delta line, in world units. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	float DeltaLineDashSpacing = 4.0f;

	/** Length of each gap between dashes in the delta line, in world units. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	float DeltaLineDashGapSpacing = 4.0f;

	/** Per-state colors for the uniform (center) scale handle. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	FGizmoPerStateValueLinearColor UniformColors;

	/** Overrides the default prefix text for offset-based scale labels, if set. */
	UPROPERTY(EditAnywhere, Category = "Scale")
	TOptional<FText> DeltaOffsetTextPrefix;

	/** Overrides the default prefix text for percentage-based scale labels, if set. */
	UPROPERTY(EditAnywhere, Category = "Scale")
	TOptional<FText> DeltaPercentageTextPrefix;

	/** Overrides the default suffix text for offset-based scale labels, if set. */
	UPROPERTY(EditAnywhere, Category = "Scale")
	TOptional<FText> DeltaOffsetTextSuffix;

	/** Overrides the default suffix text for percentage-based scale labels, if set. */
	UPROPERTY(EditAnywhere, Category = "Scale")
	TOptional<FText> DeltaPercentageTextSuffix;

	/** Background color/alpha of the text box. */
	UPROPERTY(EditAnywhere, Category = "Scale")
	FLinearColor DeltaTextBackgroundColor = FLinearColor(0.141f, 0.141f, 0.141f, 0.65f);

	/** Material used for vertex-color-based rendering of axis elements. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> VertexColorMaterial;

	/** Number of segments used when generating cylindrical geometry (axis body, etc.). */
	UPROPERTY()
	int32 NumSegments = 32;

	/** Gets the overridden prefix, or the default if not overriden. */
	UE_API const FText& GetDeltaTextPrefixForScaleType(const EGizmoTransformScaleType InScaleType) const;

	/** Gets the overridden suffix, or the default if not overriden. */
	UE_API const FText& GetDeltaTextSuffixForScaleType(const EGizmoTransformScaleType InScaleType) const;

	/** Returns true if all style properties are equal. */
	friend bool operator==(const FGizmoElementScaleAxisStyle& InLeft, const FGizmoElementScaleAxisStyle& InRight)
	{
		return static_cast<const FGizmoStyleBase&>(InLeft) == static_cast<const FGizmoStyleBase&>(InRight)
			&& InLeft.AxisLength == InRight.AxisLength
			&& InLeft.AxisLengthMultiplier == InRight.AxisLengthMultiplier
			&& InLeft.AxisOffsetFromCenter == InRight.AxisOffsetFromCenter
			&& InLeft.LineThickness == InRight.LineThickness
			&& InLeft.HeadSize == InRight.HeadSize
			&& InLeft.UniformSize == InRight.UniformSize
			&& InLeft.ShowFlags == InRight.ShowFlags
			&& InLeft.DeltaLineThickness == InRight.DeltaLineThickness
			&& InLeft.DeltaLineDashSpacing == InRight.DeltaLineDashSpacing
			&& InLeft.DeltaLineDashGapSpacing == InRight.DeltaLineDashGapSpacing
			&& InLeft.UniformColors == InRight.UniformColors
			&& InLeft.DeltaTextBackgroundColor == InRight.DeltaTextBackgroundColor
			&& InLeft.VertexColorMaterial == InRight.VertexColorMaterial
			&& InLeft.NumSegments == InRight.NumSegments
			&& InLeft.HandleSizeMultiplier == InRight.HandleSizeMultiplier
			&& InLeft.DeltaOffsetTextPrefix.Get(FText::GetEmpty()).EqualTo(InRight.DeltaOffsetTextPrefix.Get(FText::GetEmpty()))
			&& InLeft.DeltaPercentageTextPrefix.Get(FText::GetEmpty()).EqualTo(InRight.DeltaPercentageTextPrefix.Get(FText::GetEmpty()))
			&& InLeft.DeltaOffsetTextSuffix.Get(FText::GetEmpty()).EqualTo(InRight.DeltaOffsetTextSuffix.Get(FText::GetEmpty()))
			&& InLeft.DeltaPercentageTextSuffix.Get(FText::GetEmpty()).EqualTo(InRight.DeltaPercentageTextSuffix.Get(FText::GetEmpty()));
	}

	/** Returns true if any style property differs. */
	friend bool operator!=(const FGizmoElementScaleAxisStyle& Lhs, const FGizmoElementScaleAxisStyle& RHS)
	{
		return !(Lhs == RHS);
	}
};

/** Visual style parameters for a planar scale handle (two-axis rectangle). */
USTRUCT(MinimalAPI)
struct FGizmoElementScalePlanarStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	UE_API FGizmoElementScalePlanarStyle();

	/** Distance from the gizmo origin to the center of the planar handle, in world units. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	float OffsetFromOrigin = 50.0f;

	/** Width and height of the planar handle square, in world units. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	float Size = 10.0f;

	/** Multiplier applied to Size, typically driven by a user setting. */
	UPROPERTY()
	float SizeMultiplier = 1.0f;

	/** Thickness (depth) of the planar handle geometry, in world units. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	float Thickness = 0.75f;

	/** Material used for vertex-color-based rendering of the planar handle. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> VertexColorMaterial;
};

/** Visual style parameters for the uniform (screen-space) scale handle. */
USTRUCT(MinimalAPI)
struct FGizmoElementScaleUniformStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	UE_API FGizmoElementScaleUniformStyle();

	/** Size of the uniform scale handle element, in world units. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ClampMin = 0.1f))
	float Size = 10.0f;

	/** Multiplier applied to Size, typically driven by a user setting. */
	UPROPERTY()
	float SizeMultiplier = 1.0f;
};

/** Composite style for the entire scale gizmo, combining axis, planar, and uniform sub-styles. */
USTRUCT(MinimalAPI)
struct FGizmoElementScaleStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	/** Shared style for all single-axis scale elements. */
	UPROPERTY(EditAnywhere, Category = "Scale")
	FGizmoElementScaleAxisStyle AxisStyle;

	/** Shared style for all two-axis planar scale handles. */
	UPROPERTY(EditAnywhere, Category = "Scale")
	FGizmoElementScalePlanarStyle PlanarStyle;

	/** Style for the uniform (screen-space) scale handle. */
	UPROPERTY(EditAnywhere, Category = "Scale")
	FGizmoElementScaleUniformStyle UniformStyle;

	/** Controls which delta visual elements are shown during interaction at the group level. */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (Bitmask))
	EGizmoElementScaleShowFlags ShowFlags = EGizmoElementScaleShowFlags::All;
};

/** Determines the reference point from which scale distance is measured. */
UENUM()
enum class EGizmoElementScaleDistanceSource : uint8
{
	/** Measure distance from the object being scaled. */
	FromObject = 0,
	/** Measure distance from the starting input/cursor position. */
	FromStart = 1,
};

/** Determines how the scale magnitude is computed from cursor movement. */
UENUM()
enum class EGizmoElementScaleDistanceType : uint8
{
	/** Based on the linear (Euclidean) distance from the distance source. */
	Linear = 0,
	/** Based on the projected distance along a specific screen-space direction. */
	Directional = 1,
};

/** Interaction options for scale gizmo elements, controlling how user input maps to scale deltas. */
USTRUCT(MinimalAPI, Category = "Scale")
struct FGizmoElementScaleInteraction
{
	GENERATED_BODY()

	/** The default scale-type to use with the Gizmo. */
	static constexpr EGizmoTransformScaleType GetDefaultScaleType() { return EGizmoTransformScaleType::PercentageBased; }

	/** The amount is based on the distance to the reference point specified here. */
	UPROPERTY(EditAnywhere, Category = "Input")
	EGizmoElementScaleDistanceSource DistanceSource = EGizmoElementScaleDistanceSource::FromStart;

	/** The direction of the scale (+/-) is based on the metric below. */
	UPROPERTY(EditAnywhere, Category = "Input")
	EGizmoElementScaleDistanceType DistanceType = EGizmoElementScaleDistanceType::Directional;

	/** The drag direction corresponding with a scale increase. (1.0, 0.0) means left-to-right movement scales up, right-to-left scales down. Vertical movement wouldn't do anything.  */
	UPROPERTY(EditAnywhere, Category = "Input", meta = (DisplayName = "Drag Direction", EditCondition = "DistanceType == EGizmoElementScaleDistanceType::Directional"))
	FVector2D Direction = FVector2d(1.0f, 1.0f);

	/** Overall multiplier applied to the computed scale delta. */
	UPROPERTY(EditAnywhere, Category = "Input")
	float Multiplier = 1.0f;

	/** Minimum scale value clamp. Prevents the scale from reaching zero or extremely small values. */
	UPROPERTY(EditAnywhere, Category = "Input", AdvancedDisplay)
	TOptional<float> MinimumScale = 0.0001f;
};

#undef UE_API
