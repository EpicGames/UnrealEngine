// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorGizmos/EditorGizmoElementShared.h"
#include "UObject/ObjectPtr.h"

#include "GizmoElementTranslate.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/** Bitflags controlling visibility of translation delta visual elements during interaction. */
UENUM(meta = (Bitflags, ShowFlags))
enum class EGizmoElementTranslateShowFlags : uint32
{
	/** No flags set. */
	None = 0 UMETA(Hidden),
	/** Show the dashed line from origin to current position. */
	DeltaLine = 1 << 0,
	/** Show the numeric distance label. */
	DeltaLabel = 1 << 1,
	/** Show the origin marker at the drag start point. */
	DeltaOrigin = 1 << 2,

	/** All delta visual elements enabled. */
	All = DeltaLine | DeltaLabel | DeltaOrigin
};
ENUM_CLASS_FLAGS(EGizmoElementTranslateShowFlags);

/** Visual style parameters for a single translation axis element (arrow handle + delta visuals). */
USTRUCT(MinimalAPI)
struct FGizmoElementTranslateAxisStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	UE_API FGizmoElementTranslateAxisStyle();

	/** Length of the axis arrow body in world units (before multiplier). */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 1.0f))
	float AxisLength = 49.0f;

	/** Multiplier applied to AxisLength, typically driven by a user setting. */
	UPROPERTY()
	float AxisLengthMultiplier = 1.0f;

	/** Gap between the gizmo origin and the start of the axis arrow, in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 1.0f))
	float AxisOffsetFromCenter = 20.0f;

	/** Multiplier for the arrow head size, typically driven by a user setting. */
	UPROPERTY()
	float HandleSizeMultiplier = 1.0f;

	/** Thickness of the axis line in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float LineThickness = 1.5f;

	/** Overall scale multiplier for the arrow head geometry. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float ArrowSizeMultiplier = 1.0f;

	/** Height of the arrow head cone in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float ArrowHeight = 20.0f;

	/** Base radius of the arrow head cone in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float ArrowRadius = 5.0f;

	/** Controls which delta visual elements are shown during interaction. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (Bitmask))
	EGizmoElementTranslateShowFlags ShowFlags = EGizmoElementTranslateShowFlags::All;

	/** Background color/alpha of the text box. */
	UPROPERTY(EditAnywhere, Category = "Translate")
	FLinearColor DeltaTextBackgroundColor = FLinearColor(0.141f, 0.141f, 0.141f, 0.65f);

	/** Material used for vertex-color-based rendering of axis elements. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> VertexColorMaterial;

	/** Number of segments used when generating cylindrical geometry (arrow body, etc.). */
	UPROPERTY()
	int32 NumSegments = 32;

	/** Returns true if all style properties are equal. */
	friend bool operator==(const FGizmoElementTranslateAxisStyle& InLeft, const FGizmoElementTranslateAxisStyle& InRight)
	{
		return static_cast<const FGizmoStyleBase&>(InLeft) == static_cast<const FGizmoStyleBase&>(InRight)
			&& InLeft.AxisLength == InRight.AxisLength
			&& InLeft.AxisLengthMultiplier == InRight.AxisLengthMultiplier
			&& InLeft.AxisOffsetFromCenter == InRight.AxisOffsetFromCenter
			&& InLeft.HandleSizeMultiplier == InRight.HandleSizeMultiplier
			&& InLeft.LineThickness == InRight.LineThickness
			&& InLeft.ArrowSizeMultiplier == InRight.ArrowSizeMultiplier
			&& InLeft.ArrowHeight == InRight.ArrowHeight
			&& InLeft.ArrowRadius == InRight.ArrowRadius
			&& InLeft.ShowFlags == InRight.ShowFlags
			&& InLeft.DeltaTextBackgroundColor == InRight.DeltaTextBackgroundColor
			&& InLeft.VertexColorMaterial == InRight.VertexColorMaterial
			&& InLeft.NumSegments == InRight.NumSegments;
	}

	/** Returns true if any style property differs. */
	friend bool operator!=(const FGizmoElementTranslateAxisStyle& InLeft, const FGizmoElementTranslateAxisStyle& InRight)
	{
		return !(InLeft == InRight);
	}
};

/** Visual style parameters for a planar translation handle (two-axis rectangle). */
USTRUCT(MinimalAPI)
struct FGizmoElementTranslatePlanarStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	UE_API FGizmoElementTranslatePlanarStyle();

	/** Distance from the gizmo origin to the center of the planar handle, in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float OffsetFromOrigin = 50.0f;

	/** Width and height of the planar handle square, in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float Size = 10.0f;

	/** Multiplier applied to Size, typically driven by a user setting. */
	UPROPERTY()
	float SizeMultiplier = 1.0f;

	/** Thickness (depth) of the planar handle geometry, in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float Thickness = 0.75f;

	/** Material used for vertex-color-based rendering of the planar handle. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> VertexColorMaterial;
};

/** Visual style parameters for the uniform (screen-space) translation handle. */
USTRUCT(MinimalAPI)
struct FGizmoElementTranslateUniformStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	UE_API FGizmoElementTranslateUniformStyle();

	/** Size of the uniform handle element, in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float Size = 12.0f;

	/** Multiplier applied to Size, typically driven by a user setting. */
	UPROPERTY()
	float SizeMultiplier = 1.0f;

	/** Line thickness for the uniform handle outline, in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float LineThickness = 1.5f;
};

/** Composite style for the entire translation gizmo, combining axis, planar, and uniform sub-styles with shared delta visual settings. */
USTRUCT(MinimalAPI)
struct FGizmoElementTranslateStyle : public FGizmoStyleBase
{
	GENERATED_BODY()

	/** Shared style for all single-axis translation arrow elements. */
	UPROPERTY(EditAnywhere, Category = "Translate")
	FGizmoElementTranslateAxisStyle AxisStyle;

	/** Shared style for all two-axis planar translation handles. */
	UPROPERTY(EditAnywhere, Category = "Translate")
	FGizmoElementTranslatePlanarStyle PlanarStyle;

	/** Style for the uniform (screen-space) translation handle. */
	UPROPERTY(EditAnywhere, Category = "Translate")
	FGizmoElementTranslateUniformStyle UniformStyle;

	/** Controls which delta visual elements are shown during interaction at the group level. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (Bitmask))
	EGizmoElementTranslateShowFlags ShowFlags = EGizmoElementTranslateShowFlags::All;

	/** Radius of the origin sphere drawn at the drag start point, in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float OriginRadius = 1.5f;

	/** Outline thickness of the origin sphere, in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float OriginLineThickness = 0.5f;

	/** Optional override for the delta dashed line thickness. Falls back to the axis LineThickness if not set. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	TOptional<float> DeltaLineThickness;

	/** Length of each dash segment in the delta line, in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float DeltaLineDashSpacing = 4.0f;

	/** Length of each gap between dashes in the delta line, in world units. */
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ClampMin = 0.1f))
	float DeltaLineDashGapSpacing = 4.0f;
};

/** Interaction options for translation gizmo elements, controlling how user input maps to translation deltas. */
USTRUCT(MinimalAPI, Category = "Translate")
struct FGizmoElementTranslateInteraction
{
	GENERATED_BODY()

public:
	/**
	 * The explicit drag direction corresponding with view-relative translation. (1.0, 0.0) means left-to-right movement moves positively, right-to-left moves negatively. Vertical movement wouldn't do anything.
	 * Only applicable to indirect manipulation.
	 * If not set, the screen-space direction of the selected axis is used.
	 */
	UPROPERTY(EditAnywhere, Category = "Input", meta = (DisplayName = "Drag Direction"))
	TOptional<FVector2D> Direction;
};

#undef UE_API
