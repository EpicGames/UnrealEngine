// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Transform.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

struct FRotationContext;
struct FEulerTransform;

namespace UE::GizmoRotationUtil
{
	/**
	 * FRotationDecomposition is a data storage structure representing the rotations around the X, Y and Z axis  
	 */	
	struct FRotationDecomposition
	{
		FRotationDecomposition()
		{
			R[0] = R[1] = R[2] = FQuat::Identity;
		}
		FQuat R[3];
	};

	/**
	 * Decomposes InRotationContext.Rotation in respect to the rotation order to output the explicit per axis rotations.
	 */
	UE_API void DecomposeRotations(const FTransform& InTransform, const FRotationContext& InRotationContext, FRotationDecomposition& OutDecomposition);

	/**
	 * Returns the explicit rotation axis in respect to InRotationContext.Rotation & InRotationContext.RotationOrder
	 */
	UE_API FVector GetRotationAxis(const FTransform& InTransform, const FRotationContext& InRotationContext, const int32 InAxis);
}

namespace UE::GizmoRotationUtil
{
	/**
	 * IRelativeTransformInterface
	 */
	struct IRelativeTransformInterface
	{
		virtual ~IRelativeTransformInterface() = default;

		virtual bool GetRelativeTransform(const FTypedElementHandle& InElementHandle, FEulerTransform& OutRelativeTransform) = 0;
		virtual bool SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FEulerTransform& InRelativeTransform) = 0;
	};

	/**
	 * FRelativeTransformInterfaceRegistry
	 */
	class FRelativeTransformInterfaceRegistry
	{
	public:
		~FRelativeTransformInterfaceRegistry() = default;

		/** Get the singleton registry object */
		static EDITORINTERACTIVETOOLSFRAMEWORK_API FRelativeTransformInterfaceRegistry& Get();

		/** Register an interface for the TWorldInterface static class */
		template<typename TWorldInterface>
		void RegisterRelativeTransformInterface(TUniquePtr<IRelativeTransformInterface>&& InInterface)
		{
			UClass* WorldInterfaceClass = TWorldInterface::StaticClass();
			check(!WorldInterfaceToRelativeTransformInterface.Contains(WorldInterfaceClass));
			WorldInterfaceToRelativeTransformInterface.Add(WorldInterfaceClass, MoveTemp(InInterface));
		}

		static void RegisterDefaultInterfaces();

		/** Find the registered interface from the given class. Returns a nullptr if noting registered for that class. */
		IRelativeTransformInterface* FindRelativeTransformInterface(const TTypedElement<ITypedElementWorldInterface>& InElement) const;
		IRelativeTransformInterface* FindRelativeTransformInterface(const UClass* InClass) const;

	private:
		FRelativeTransformInterfaceRegistry() = default;
		
		TMap<UClass*, TUniquePtr<IRelativeTransformInterface>> WorldInterfaceToRelativeTransformInterface;

		mutable TSet<const UClass*> QueriedUnregisteredClass;
		void LogOnce(const UClass* InClass) const;
	};

	/**
	* Handles get / set relative transform operations UActorElementWorldInterface & UActorElementEditorWorldInterface
	*/
	struct FActorRelativeTransformInterface : IRelativeTransformInterface
	{
		virtual ~FActorRelativeTransformInterface() override = default;

		virtual bool GetRelativeTransform(const FTypedElementHandle& InElementHandle, FEulerTransform& OutRelativeTransform) override;
		virtual bool SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FEulerTransform& InRelativeTransform) override;

	private:
		static USceneComponent* GetSceneComponent(const FTypedElementHandle& InElementHandle);
	};

	/**
	* Handles get / set relative transform operations UActorElementWorldInterface & UActorElementEditorWorldInterface
	*/
	struct FComponentRelativeTransformInterface : IRelativeTransformInterface
	{
		virtual ~FComponentRelativeTransformInterface() override = default;
		
		virtual bool GetRelativeTransform(const FTypedElementHandle& InElementHandle, FEulerTransform& OutRelativeTransform) override;
		virtual bool SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FEulerTransform& InRelativeTransform) override;

	private:
		static USceneComponent* GetSceneComponent(const FTypedElementHandle& InElementHandle);
	};

	/**
 * Returns the relative transform of an object by storing its explicit relative rotation in the provided FRotationContext
 */
	UE_API bool GetRelativeTransform(const TTypedElement<ITypedElementWorldInterface>& InElement, FTransform& OutTransform, FRotationContext& InOutRotationContext);
}

#undef UE_API
