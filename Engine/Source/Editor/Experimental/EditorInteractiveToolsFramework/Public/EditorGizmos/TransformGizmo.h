// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseGizmos/GizmoElementHitTargets.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/TransformProxy.h"
#include "Containers/EnumAsByte.h"
#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementShared.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"
#include "GizmoElementRotate.h"
#include "GizmoElementScale.h"
#include "GizmoElementTranslate.h"
#include "InputState.h"
#include "InteractiveGizmo.h"
#include "InteractiveToolChange.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Math/Axis.h"
#include "Math/Color.h"
#include "Math/Quat.h"
#include "Math/Ray.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"

#include "TransformGizmo.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class IGizmoAxisSource;
class IGizmoStateTarget;
class IGizmoTransformSource;
class IToolContextTransactionProvider;
class IToolsContextRenderAPI;
class UMultiButtonClickDragBehavior;
class UGizmoConstantFrameAxisSource;
class UGizmoElementArrow;
class UGizmoElementBase;
class UGizmoElementBox;
class UGizmoElementCircle;
class UGizmoElementCone;
class UGizmoElementCylinder;
class UGizmoElementGroup;
class UGizmoElementHitMultiTarget;
class UGizmoElementRectangle;
class UGizmoElementRoot;
class UGizmoElementTorus;
class UInteractiveGizmoManager;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UObject;
class UTransformGizmo;
class UTransformProxy;
class UGizmoElementGimbal;

//
// Interface for the Transform gizmo.
//


//
// Part identifiers are used to associate transform gizmo parts with their corresponding representation 
// in the render and hit target. The render and hit target should use the default identifier for 
// any of their internal elements that do not correspond to transform gizmo parts, for example non-hittable
// visual guide elements.
//
UENUM()
enum class ETransformGizmoPartIdentifier
{
	Default,
	TranslateAll,
	TranslateXAxis,
	TranslateYAxis,
	TranslateZAxis,
	TranslateXYPlanar,
	TranslateYZPlanar,
	TranslateXZPlanar,
	TranslateScreenSpace,
	RotateAll,
	RotateXAxis,
	RotateYAxis,
	RotateZAxis,
	RotateScreenSpace,
	RotateArcball,
	RotateXGimbal,
	RotateYGimbal,
	RotateZGimbal,
	ScaleAll,
	ScaleXAxis,
	ScaleYAxis,
	ScaleZAxis,
	ScaleXYPlanar,
	ScaleYZPlanar,
	ScaleXZPlanar,
	ScaleUniform,
	Max
};

ENUM_RANGE_BY_COUNT(ETransformGizmoPartIdentifier, ETransformGizmoPartIdentifier::Max);

namespace UE::Editor::InteractiveToolsFramework
{
	/** Friended class to access UTransformGizmo internals, used externally. */
	struct FTransformGizmoAccessor;
}

/** Controls the (fixed) visibility of various visual elements in the TransformGizmo. */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ETransformGizmoShowFlags : uint32
{
	None = 0 UMETA(Hidden),
	DeltaLabel = 1 << 0,
	DeltaLine = 1 << 1,
	DeltaArc = 1 << 2,
	DeltaOrigin = 1 << 3,

	All = DeltaLabel | DeltaLine | DeltaArc | DeltaOrigin
};
ENUM_CLASS_FLAGS(ETransformGizmoShowFlags);

/** Contains all styling properties for TransformGizmo. */
USTRUCT(MinimalAPI)
struct FTransformGizmoStyle
{
	GENERATED_BODY()

public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_API FTransformGizmoStyle();
	FTransformGizmoStyle(const FTransformGizmoStyle&) = default;
	FTransformGizmoStyle& operator=(const FTransformGizmoStyle&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Clamps and otherwise corrects values within the style. */
	UE_API void MakeValid();

public:
	// Uniforms

	UE_DEPRECATED(5.6, "Use AxisDisplayInfo::GetAxisColor(EAxisList::X) instead")
	static constexpr FLinearColor AxisColorX = FLinearColor(0.594f, 0.0197f, 0.0f);
	UE_DEPRECATED(5.6, "Use AxisDisplayInfo::GetAxisColor(EAxisList::Y) instead")
	static constexpr FLinearColor AxisColorY = FLinearColor(0.1349f, 0.3959f, 0.0f);
	UE_DEPRECATED(5.6, "Use AxisDisplayInfo::GetAxisColor(EAxisList::Z) instead")
	static constexpr FLinearColor AxisColorZ = FLinearColor(0.0251f, 0.207f, 0.85f);

public:
	// Varying

#pragma region Shared
	/** Applies to axis lengths and radii. Use the GetModifiedAxisSizeMultiplier to account for various other project/editor settings. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (ClampMin = 0.1f, ClampMax = 100.0f, DisplayPriority = -1))
	float AxisSizeMultiplier = 1.0f;

	/** Applies to arrow heads, planar and uniform handles. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (ClampMin = 0.01f, ClampMax = 4.0f, DisplayPriority = -1))
	float HandleSizeMultiplier = 1.0f;

	/** Applies to both translate and scale planar handles. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (ClampMin = 0.01f, ClampMax = 300.0f, DisplayPriority = -1))
	float PlanarAxisOffsetFromOrigin = 50.0f;

	/** Used to prevent sub-pixel line sizes. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (ClampMin = 0.01f))
	float MinLineThickness = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Shared", meta = (ClampMin = 0.01f, ClampMax = 3.0f))
	float LineThicknessMultiplier = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Shared", meta = (ClampMin = 0.01f))
	float HoverLineThicknessMultiplier = 1.5f;

	/** The current color to use. */
	UE_DEPRECATED(5.7, "Use HoverColor instead")
	UPROPERTY(meta = (HideAlphaChannel, DeprecatedProperty, DeprecationMessage = "Use HoverColor instead."))
	FLinearColor CurrentColor = FLinearColor(1.0f, 1.0f, 0.0f);

	/** The color to use on hover. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (HideAlphaChannel))
	FLinearColor HoverColor = FLinearColor(1.0f, 0.552619f, 0.024453f);

	/** The color to use on select. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (HideAlphaChannel))
	FLinearColor SelectColor = FLinearColor(0.8f, 0.8f, 0.0f);

	/** The color to use on interact. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (HideAlphaChannel))
	FLinearColor InteractColor = FLinearColor(1.0f, 1.0f, 0.0f);

	/** The color to use on subdue. Depending on the effect, we may only use the alpha value. */
	UPROPERTY(EditAnywhere, Category = "Shared")
	FLinearColor SubdueColor = FLinearColor::Black.CopyWithNewOpacity(0.7f);

	UPROPERTY(EditAnywhere, Category = "Shared", meta = (HideAlphaChannel))
	FLinearColor GreyColor = FLinearColor(0.6f, 0.6f, 0.6f);

	UPROPERTY(EditAnywhere, Category = "Shared", meta = (HideAlphaChannel))
	FLinearColor WhiteColor = FLinearColor(0.9f, 0.9f, 0.9f);

	/** Whether to show shading on the gizmo, or unlit (if false). */
	UPROPERTY(EditAnywhere, Category = "Shared")
	bool bUseShading = true;

	UPROPERTY(EditAnywhere, Category = "Shared", meta = (EditCondition = "bUseShading", ClampMin = 0.0f, ClampMax = 2.0f))
	float ShadingAmbient = 0.8f;

	UPROPERTY(EditAnywhere, Category = "Shared", meta = (EditCondition = "bUseShading", ClampMin = 0.0f, ClampMax = 1.0f))
	float ShadingSpecularGlossiness = 0.1f;

	/* Pixel hit distance threshold - where the default is generally 7px. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (ClampMin = 0.1f))
	float PixelHitDistanceThreshold = 15.0f;

	/* Minimum pixel hit distance threshold. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (ClampMin = 0.0f))
	float MinimumPixelHitDistanceThreshold = 0.0f;

	/** Represents the background color for text labels. */
	UPROPERTY(EditAnywhere, Category = "Shared")
	FLinearColor DeltaTextBackgroundColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.65f);

	/** Synchronizes the cursor size to that specified for the platform/OS. If false, CursorSize is used. */
	UPROPERTY(EditAnywhere, Category = "Shared")
	bool bUsePlatformCursorSize = true;

	/** Defines the size of the custom cursor. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (EditCondition = "!bUsePlatformCursorSize"))
	float CursorSize = 16.0f;

	/** Sets the fixed visibility of various elements. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (Bitmask, BitmaskEnum = "/Script/EditorInteractiveToolsFramework.ETransformGizmoShowFlags"))
	uint32 ShowFlags = static_cast<uint32>(ETransformGizmoShowFlags::All);
#pragma endregion Shared

#pragma region Translate
	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ShowOnlyInnerProperties, DisplayPriority = 10))
	FGizmoElementTranslateStyle TranslateStyle;
#pragma endregion Translate

#pragma region Rotate
	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ShowOnlyInnerProperties, DisplayPriority = 11))
	FGizmoElementRotateStyle RotateStyle;
#pragma endregion Rotate

#pragma region Scale
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ShowOnlyInnerProperties, DisplayPriority = 12))
	FGizmoElementScaleStyle ScaleStyle;
#pragma endregion Scale

public:
#pragma region Shared
	/** Gets the AxisSizeMultiplier, modified by project/editor level settings, etc., or the given Modifier if specified. */
	UE_API float GetModifiedAxisSizeMultiplier(const TOptional<TUniqueFunction<float(const float)>>& InModifierOverride = { }) const;
#pragma endregion Shared
};

USTRUCT(MinimalAPI)
struct FTransformGizmoInteraction
{
	GENERATED_BODY()

	/** Whether to allow handle selection to persist and reference for indirect operations. */ 
	UPROPERTY()
	bool bPersistHandleSelection = false;

	/** Stores additional coordinate systems enabled for this gizmo. By default, all are available. */
	UPROPERTY(EditAnywhere, Category = "Shared", meta = (Bitmask, BitmaskEnum = "/Script/EditorFramework.ECoordSystem"))
	int32 AdditionalCoordinateSystems = INDEX_NONE;

	UPROPERTY(EditAnywhere, Category = "Translate", meta = (ShowOnlyInnerProperties, DisplayPriority = 10))
	FGizmoElementTranslateInteraction TranslateInteraction;

	UPROPERTY(EditAnywhere, Category = "Rotate", meta = (ShowOnlyInnerProperties, DisplayPriority = 11))
	FGizmoElementRotateInteraction RotateInteraction;

	UPROPERTY(EditAnywhere, Category = "Scale", meta = (ShowOnlyInnerProperties, DisplayPriority = 12))
	FGizmoElementScaleInteraction ScaleInteraction;

	/** Returns true if the given coordinate system is enabled for this gizmo. */
	UE_API bool IsCoordinateSystemEnabled(const ECoordSystem InCoordinateSystem) const;

	/** Sets whether the given coordinate system is enabled for this gizmo. */
	UE_API void SetCoordinateSystemEnabled(const ECoordSystem InCoordinateSystem, const bool bInIsEnabled);
};

USTRUCT()
struct FGizmosParameters
{
	GENERATED_BODY()

	/** Determines how dragging the rotate gizmo affects the selected objects */
	UPROPERTY(EditAnywhere, Category = "Experimental")
	TEnumAsByte<EAxisRotateMode::Type> RotateMode = EAxisRotateMode::ScreenArc;
	
	/** Determines how scale is calculated when applied. */
	UPROPERTY(EditAnywhere, Category = "Experimental")
	EGizmoTransformScaleType ScaleType = EGizmoTransformScaleType::PercentageBased;

	/**
	 * When enabled, Ctrl+clicking multiple mouse buttons allows you to indirectly manipulate multiple axes at once. 
	 */
	UPROPERTY(EditAnywhere, Category = "Experimental", DisplayName = "Additive Indirect Manipulation")
	bool bAdditiveIndirectAxes = false;

	/**
	 * When enabled and in an orthographic view, Ctrl+LMB-Drag will move or scale in screen space, and Ctrl+RMB-Drag will rotate in screen space.  
	 */
	UPROPERTY(EditAnywhere, Category = "Experimental", DisplayName = "Screen Space Indirect Orthographic Manipulation")
	bool bScreenSpaceIndirectOrthographicManipulation = true;

	/**
	 * When disabled, Ctrl+LMB manipulates X, Ctrl+RMB manipulates Y, and Ctrl+LMB+RMB or Ctrl+MMB manipulates Z.
	 * When enabled LMB is mapped to X, MMB is mapped to Y, and RMB is mapped to Z. 
	 */
	UPROPERTY(EditAnywhere, Category = "Experimental")
	bool bSequentialIndirectAxesButtons = false;

	/**
	 * When enabled, all indirect scale manipulations will trigger uniform scaling
	 */
	UPROPERTY(EditAnywhere, Category = "Experimental", DisplayName = "Indirect Scale is always Uniform")
	bool bUniformIndirectScale = false;
	
	/**
	 * When enabled, the list of coordinate spaces in the viewport toolbar offers rig space: a coordinate system that is
	 * similar to parent space but uses gimbal rotations.
	 */
	UPROPERTY(EditAnywhere, Category = "Experimental")
	bool bEnableExplicit = false;

	/** Multiplies the values of TranslateAxisLength, ScaleAxisLength, and RotateAxisRadius. */
	UPROPERTY()
	float AxisSizeMultiplier = 1.0f;

	/** Whether to allow persistent handle selection. If disabled, implicit indirect modification will also be disabled. Explicit (ctrl+mouse) will still work.  */
	UPROPERTY(EditAnywhere, Category = "Experimental")
	bool bPersistHandleSelection = false;

	/** When enabled, drag duplication will work on both translation and rotation (currently Level Editor only). */
	UPROPERTY(EditAnywhere, Category=GizmoInteraction)
	bool bDragDuplicateOnRotation = false;

	/** 
	 * When enabled, translate and scale nudging will be done based on camera view angle (nudge to the right will try to match what's "right" on the screen). 
	 * When option is disabled, controls will be axis-based: Left/Right will control X, Up/Down will control Y, secondary Up/Down will control Z.
	 * Rotation nudging always does axis-based controls.
	 */
	UPROPERTY(EditAnywhere, Category=GizmoInteraction)
	bool bScreenSpaceNudge = true;

	UPROPERTY(Config, EditAnywhere, Category = "Appearance")
	FTransformGizmoStyle Style;

	UPROPERTY(Config, EditAnywhere, Category = "Interaction")
	FTransformGizmoInteraction Interaction;

	UPROPERTY(Config, EditAnywhere, Category = "Debug", meta=(HideInDetailPanel))
	FGizmoDebugSettings Debug;
};

/**
 * FGizmoInteractionDescription provides a way to pass some interaction description in and out before/after starting/ending a gizmo manipulation
 */
struct FGizmoInteractionDescription
{
	// ray based on the current input device
	FInputDeviceRay Ray;
	// whether the interaction is direct or indirect
	bool bIndirect = false;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnGizmoInteraction, const FGizmoInteractionDescription&);

/**
 * UTransformGizmo provides standard Transformation Gizmo interactions,
 * applied to a UTransformProxy target object. By default the Gizmo will be
 * a standard XYZ translate/rotate Gizmo (axis and plane translation).
 */
UCLASS(MinimalAPI)
class UTransformGizmo : public UInteractiveGizmo, public IHoverBehaviorTarget, public IClickDragBehaviorTarget
{
	GENERATED_BODY()

	friend UE::Editor::InteractiveToolsFramework::FTransformGizmoAccessor;
	friend class UTransformGizmoDebug;

public:

	/** Contains all styling properties. */
	struct FGizmoStyle
	{
		/** Applies to axis lengths and radii */
		float AxisSizeMultiplier = 1.0f;

		/** Applies to lines */
		float LineThicknessMultiplier = 1.0f;

		static constexpr float AxisRadius = 1.5f;
		static constexpr float AxisLengthOffset = 20.0f;

		// Transform
		float TranslateAxisLength = 70.0f;
		static constexpr float TranslateAxisConeAngle = 16.0f;
		static constexpr float TranslateAxisConeHeight = 22.0f;
		static constexpr float TranslateAxisConeRadius = 7.0f;
		static constexpr float TranslateScreenSpaceHandleSize = 14.0f;

		float RotateArcballSphereRadius = 70.0f;
		float RotateAxisRadius = 70.0f;
		float RotateCircleRadius = 70.0f;

		// Rotation
		static constexpr float RotateAxisInnerRadius = 1.25f;
		static constexpr int32 RotateAxisNumSegments = 64;
		static constexpr int32 RotateAxisInnerSlices = 8;

		static constexpr float RotateAxisOuterRadiusOffset = 3.0f;  // Offset from RotateAxisRadius
		static constexpr float RotateOuterCircleRadiusOffset = 3.0f; // Offset from RotateCircleRadius
		static constexpr float RotateScreenSpaceRadiusOffset = 13.0f; // Offset from RotateCircleRadius

		// Scale
		float ScaleAxisLength = 70.0f;
		static constexpr float ScaleAxisCubeDim = 12.0f;

		static constexpr float PlanarHandleOffset = -15.0f; // Offset from TranslateAxisLength
		static constexpr float PlanarHandleSize = 15.0f;
		static constexpr float PlanarHandleThickness = 2.0f;

		static constexpr float AxisTransparency = 0.8f;
		UE_DEPRECATED(5.6, "Use AxisDisplayInfo::GetAxisColor(EAxisList::X) instead")
		static constexpr FLinearColor AxisColorX = FLinearColor(0.594f, 0.0197f, 0.0f);
		UE_DEPRECATED(5.6, "Use AxisDisplayInfo::GetAxisColor(EAxisList::Y) instead")
		static constexpr FLinearColor AxisColorY = FLinearColor(0.1349f, 0.3959f, 0.0f);
		UE_DEPRECATED(5.6, "Use AxisDisplayInfo::GetAxisColor(EAxisList::Z) instead")
		static constexpr FLinearColor AxisColorZ = FLinearColor(0.0251f, 0.207f, 0.85f);
		static constexpr FLinearColor ScreenAxisColor = FLinearColor(0.76f, 0.72f, 0.14f);
		static constexpr FLinearColor PlaneColorXY = FLinearColor(1.0f, 1.0f, 0.0f);
		static constexpr FLinearColor ArcBallColor = FLinearColor(0.5f, 0.5f, 0.5f, 0.03f);
		static constexpr FLinearColor ScreenSpaceColor = FLinearColor(0.765f, 0.765f, 0.765f);
		static constexpr FLinearColor CurrentColor = FLinearColor(1.0f, 1.0f, 0.0f);
		static constexpr FLinearColor GreyColor = FLinearColor(0.50f, 0.50f, 0.50f);
		static constexpr FLinearColor WhiteColor = FLinearColor(1.0f, 1.0f, 1.0f);

		static constexpr FLinearColor RotateScreenSpaceCircleColor = WhiteColor;
		static constexpr FLinearColor RotateOuterCircleColor = GreyColor;
		static constexpr FLinearColor RotateArcballCircleColor = WhiteColor.CopyWithNewOpacity(0.1f);

		static constexpr float LargeOuterAlpha = 0.5f;
	} Style;


public:

	/**
	 * By default, the nonuniform scale components can scale negatively. However, they can be made to clamp
	 * to zero instead by passing true here. This is useful for using the gizmo to flatten geometry.
	 *
	 * TODO: Should this affect uniform scaling too?
	 */
	UE_API virtual void SetDisallowNegativeScaling(bool bDisallow);

	// UInteractiveGizmo overrides
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown() override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	/** Returns true if the gizmo should be rendered. */
	UE_API virtual bool CanRender(IToolsContextRenderAPI* InRenderAPI) const;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

	// IClickDragBehaviorTarget implementation
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	UE_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	UE_API virtual void OnTerminateDragSequence() override;

	/**
	 * Set the active target object for the Gizmo
	 * @param Target active target
	 * @param TransactionProvider optional IToolContextTransactionProvider implementation to use - by default uses GizmoManager
	 * @param InStateTarget optional IGizmoStateTarget implementation to use - will create a new UGizmoObjectModifyStateTarget none provided
	 */
	UE_API virtual void SetActiveTarget(UTransformProxy* Target, IToolContextTransactionProvider* TransactionProvider = nullptr, IGizmoStateTarget* InStateTarget = nullptr);

	/**
	 * Clear the active target object for the Gizmo
	 */
	UE_API virtual void ClearActiveTarget();

	/**
	 * Explicitly set the child scale. Mainly useful to "reset" the child scale to (1,1,1) when re-using Gizmo across multiple transform actions.
	 * @warning does not generate change/modify events!!
	 */
	UE_API virtual void SetNewChildScale(const FVector& NewChildScale);

	/**
	 * Set visibility for this Gizmo
	 */
	UE_API virtual void SetVisibility(bool bVisible);

	/**
	 * Set customization function for this Gizmo
	 */
	UE_API void SetCustomizationFunction(const TFunction<const FGizmoCustomization()>& InFunction);

	/**
	 * Handle widget mode changed.
	 */
	UE_API virtual void HandleWidgetModeChanged(UE::Widget::EWidgetMode InWidgetMode);

	/**
	 * Handle user parameters changes
	 */
	UE_API virtual void OnParametersChanged(const FGizmosParameters& InParameters);

	/**
	 * True if gizmo is being used for indirect manipulation
	 */
	bool IsIndirectManipulation() const
	{
		return bIndirectManipulation;
	}

public:

	/** The active target object for the Gizmo */
	UPROPERTY()
	TObjectPtr<UTransformProxy> ActiveTarget;

	/** The hit target object */
	UPROPERTY()
	TObjectPtr<UGizmoElementHitMultiTarget> HitTarget;

	/** The multi button mouse click behavior is accessible so that it can be modified to use different mouse keys. */
	UPROPERTY()
	TObjectPtr<UMultiButtonClickDragBehavior> MultiIndirectClickDragBehavior;

	/** Transform Gizmo Source */
	UPROPERTY()
	TScriptInterface<ITransformGizmoSource> TransformGizmoSource;

	/** Root of renderable gizmo elements */
	UPROPERTY()
	TObjectPtr<UGizmoElementGroup> GizmoElementRoot;

	/** Gizmo view context, needed for screen space interactions */
	UPROPERTY()
	TObjectPtr<UGizmoViewContext> GizmoViewContext;

	/** Whether gizmo is visible. */
	UPROPERTY()
	bool bVisible = false;

	/** Whether gizmo is interacting. */
	UPROPERTY()
	bool bInInteraction = false;

	/** If true, then when using world frame, Axis and Plane translation snap to the world grid via the ContextQueriesAPI (in PositionSnapFunction) */
	UPROPERTY()
	bool bSnapToWorldGrid = false;

	/**
	 * Optional grid size which overrides the Context Grid
	 */
	UPROPERTY()
	bool bGridSizeIsExplicit = false;
	UPROPERTY()
	FVector ExplicitGridSize;

	/**
	 * Optional grid size which overrides the Context Rotation Grid
	 */
	UPROPERTY()
	bool bRotationGridSizeIsExplicit = false;
	UPROPERTY()
	FRotator ExplicitRotationGridSize;

	/**
	 * If true, then when using world frame, Axis and Plane translation snap to the world grid via the ContextQueriesAPI (in RotationSnapFunction)
	 */
	UPROPERTY()
	bool bSnapToWorldRotGrid = false;

	/** Broadcast at the end of a SetActiveTarget call. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSetActiveTarget, UTransformGizmo*, UTransformProxy*);
	FOnSetActiveTarget OnSetActiveTarget;
	
	/** Broadcast at the beginning of a ClearActiveTarget call, when the ActiveTarget is not yet disconnected. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnClearActiveTarget, UTransformGizmo*, UTransformProxy*);
	FOnClearActiveTarget OnAboutToClearActiveTarget;

protected:

	//
	// Gizmo Objects, used for rendering and hit testing
	//

	/** Translate X Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementArrow> TranslateXAxisElement;

	/** Translate Y Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementArrow> TranslateYAxisElement;

	/** Translate Z Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementArrow> TranslateZAxisElement;

	/** Translate screen-space */
	UPROPERTY()
	TObjectPtr<UGizmoElementRectangle> TranslateScreenSpaceElement;

	/** Translate planar XY handle */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> TranslatePlanarXYElement;

	/** Translate planar YZ handle */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> TranslatePlanarYZElement;

	/** Translate planar XZ handle */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> TranslatePlanarXZElement;

	/** Rotate X Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementTorus> RotateXAxisElement;

	/** Rotate Y Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementTorus> RotateYAxisElement;

	/** Rotate Z Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementTorus> RotateZAxisElement;

	/** Gimbal Rotate X Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementTorus> RotateXGimbalElement;

	/** Gimbal Rotate Y Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementTorus> RotateYGimbalElement;
	
	/** Gimbal Rotate Z Axis */
	UPROPERTY()
	TObjectPtr<UGizmoElementTorus> RotateZGimbalElement;

	/** Gimbal Rotate root element */
	UPROPERTY()
	TObjectPtr<UGizmoElementGimbal> RotateGimbalElement;

	/** Rotate arcball inner circle */
	UPROPERTY()
	TObjectPtr<UGizmoElementCircle> RotateArcballElement;

	/** Rotate screen space circle */
	UPROPERTY()
	TObjectPtr<UGizmoElementCircle> RotateScreenSpaceElement;

	/** Scale X Axis object */
	UPROPERTY()
	TObjectPtr<UGizmoElementArrow> ScaleXAxisElement;

	/** Scale Y Axis object */
	UPROPERTY()
	TObjectPtr<UGizmoElementArrow> ScaleYAxisElement;

	/** Scale Z Axis object */
	UPROPERTY()
	TObjectPtr<UGizmoElementArrow> ScaleZAxisElement;

	/** Scale planar XY handle */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> ScalePlanarXYElement;

	/** Scale planar YZ handle */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> ScalePlanarYZElement;

	/** Scale planar XZ handle */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> ScalePlanarXZElement;

	/** Uniform scale object */
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> ScaleUniformElement;

	/** Axis that points towards camera, X/Y plane tangents aligned to right/up. Shared across Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoConstantFrameAxisSource> CameraAxisSource;

	// internal function that updates CameraAxisSource by getting current view state from GizmoManager
	UE_API void UpdateCameraAxisSource();

	/** The state target is created internally during SetActiveTarget() if no one is provided. */
	UPROPERTY()
	TScriptInterface<IGizmoStateTarget> StateTarget;
	
	/**
	 * These are used to let the translation subgizmos use raycasts into the scene to align the gizmo with scene geometry.
	 * See comment for SetWorldAlignmentFunctions().
	 */
	TUniqueFunction<bool()> ShouldAlignDestination = []() { return false; };
	TUniqueFunction<bool(const FRay&, FVector&)> DestinationAlignmentRayCaster = [](const FRay&, FVector&) {return false; };

	bool bDisallowNegativeScaling = false;

protected:

	/** Renders various debug visualizations, if enabled. */
	void RenderDebug(IToolsContextRenderAPI* RenderAPI);

	/** Setup behaviors */
	UE_API virtual void SetupBehaviors();

	/** Setup indirect behaviors */
	UE_API virtual void SetupIndirectBehaviors();

	/** Setup materials */
	UE_API virtual void SetupMaterials();

	/** Setup on click functions */
	UE_API virtual void SetupOnClickFunctions();

	/** Update current gizmo mode based on transform source */
	UE_API virtual void UpdateMode();

	/** Update current gizmo rotation mode (default or gimbal)*/
	UE_API virtual void UpdateRotationMode();

	/** Enable the given mode with the specified axes, EAxisList::Type::None will hide objects associated with mode */
	UE_API virtual void EnableMode(EGizmoTransformMode InGizmoMode, EAxisList::Type InAxisListToDraw);

	/** Enable translate using specified axis list */
	UE_API virtual void EnableTranslate(EAxisList::Type InAxisListToDraw);

	/** Enable rotate using specified axis list */
	UE_API virtual void EnableRotate(EAxisList::Type InAxisListToDraw);

	/** Enable scale using specified axis list */
	UE_API virtual void EnableScale(EAxisList::Type InAxisListToDraw);

	/** Enable planar handles used by translate and scale */
	UE_API virtual void EnablePlanarObjects(bool bTranslate, bool bEnableX, bool bEnableY, bool bEnableZ);

	/** Construct translate axis handle */
	UE_API virtual UGizmoElementArrow* MakeTranslateAxis(ETransformGizmoPartIdentifier InPartId, const FVector& InAxisDir, const FVector& InSideDir, UMaterialInterface* InMaterial);

	/** Updates the given translate axis handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateTranslateAxis(UGizmoElementArrow* InElement);

	/** Updates the given translate axis handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateTranslateAxis(UGizmoElementArrow* InElement, const EAxis::Type InAxis);

	/** Updates the given translate axis handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateTranslateAxis(UGizmoElementArrow* InElement, const FVector& InAxisDir, const FVector& InSideDir);

	/** Construct scale axis handle */
	UE_API virtual UGizmoElementArrow* MakeScaleAxis(ETransformGizmoPartIdentifier InPartId, const FVector& InAxisDir, const FVector& InSideDir, UMaterialInterface* InMaterial);

	/**	Updates the given scale axis handle with the current parameters (ie. size coefficient). */
	UE_API virtual void UpdateScaleAxis(UGizmoElementArrow* InElement);

	/**	Updates the given scale axis handle with the current parameters (ie. size coefficient). */
	UE_API virtual void UpdateScaleAxis(UGizmoElementArrow* InElement, const EAxis::Type InAxis);

	/**
	 * Updates the given scale axis handle with the current parameters (ie. size coefficient).
	 */
	UE_API virtual void UpdateScaleAxis(UGizmoElementArrow* InElement, const FVector& InAxisDir, const FVector& InSideDir);

	/** Updates all scale axis handles with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateAllScaleAxis();

	/** Construct the default rotate axis handle based on the ETransformGizmoPartIdentifier. */
	UE_API UGizmoElementTorus* MakeDefaultRotateAxis(const ETransformGizmoPartIdentifier InPartId);

	/** Construct rotate axis handle */
	UE_API virtual UGizmoElementTorus* MakeRotateAxis(ETransformGizmoPartIdentifier InPartId, const FVector& TorusAxis0, const FVector& TorusAxis1,
		UMaterialInterface* InMaterial, UMaterialInterface* InCurrentMaterial);

	/** Updates the given rotate axis handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateRotateAxis(UGizmoElementTorus* InElement);

	/** Construct uniform scale handle */
	UE_API virtual UGizmoElementBox* MakeUniformScaleHandle();

	/** Updates the given uniform scale handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateUniformScaleHandle(UGizmoElementBox* InElement);

	/** Construct planar axis handle */
	UE_API virtual UGizmoElementBox* MakePlanarHandle(ETransformGizmoPartIdentifier InPartId, const FVector& InUpDirection, const FVector& InSideDirection, const FVector& InPlaneNormal,
		UMaterialInterface* InMaterial);

	/** Updates the given uniform scale handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdatePlanarHandle(UGizmoElementBox* InElement, const FVector& InUpDirection, const FVector& InSideDirection);

	/** Construct translate screen space handle */
	UE_API virtual UGizmoElementRectangle* MakeTranslateScreenSpaceHandle();

	/** Updates the given translate screen space handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateTranslateScreenSpaceHandle(UGizmoElementRectangle* InElement);

	/** Construct arcball screen space handle */
	UE_API virtual UGizmoElementCircle* MakeArcballCircleHandle(ETransformGizmoPartIdentifier InPartId, float InRadius, const FLinearColor& InColor);

	/** Construct rotate screen space handle */
	UE_API virtual UGizmoElementCircle* MakeRotateCircleHandle(ETransformGizmoPartIdentifier InPartId, float InRadius, const FLinearColor& InColor,
		const EGizmoElementDrawType InDrawType = EGizmoElementDrawType::Fill, const TOptional<FLinearColor>& InLineColorOverride = TOptional<FLinearColor>());

	/** Updates the given rotate screen space handle with the current parameters (ie. size coefficient) */
	UE_API virtual void UpdateRotateCircleHandle(UGizmoElementCircle* InElement, float InRadius);

	/** Updates all gizmo elements based on current parameters */
	UE_API virtual void UpdateElements();

	/** Get gizmo transform based on cached current transform. */
	UE_API virtual FTransform GetGizmoTransform() const;

	/** Determine hit part and update hover state based on current input ray */
	UE_API virtual FInputRayHit UpdateHoveredPart(const FInputDeviceRay& DevicePos);

	/** Get current interaction axis */
	UE_API virtual FVector GetWorldAxis(const FVector& InAxis) const;

	/** Get gimbal rotation axis (InAxis is expected to be in [0,2] range) */
	UE_API virtual FVector GetGimbalRotationAxis(const int32 InAxis) const;

	/** Get nearest param along input ray to the current interaction axis */
	UE_API virtual float GetNearestRayParamToInteractionAxis(const FInputDeviceRay& InRay);

	/** Return true if input ray intersects current interaction plane and return param along ray in OutHitParam */
	UE_API virtual bool GetRayParamIntersectionWithInteractionPlane(const FInputDeviceRay& InRay, FVector::FReal& OutHitParam);

	/** Update hover state for given part id */
	UE_API virtual void UpdateHoverState(const bool bInHover, const ETransformGizmoPartIdentifier InPartId);

	/** Reset all hover states related to the transform mode to false */
	UE_API virtual void ResetHoverStates(const EGizmoTransformMode InMode);

	/** Update interacting state for given part id */
	UE_API virtual void UpdateInteractingState(const bool bInInteracting, const ETransformGizmoPartIdentifier InPartId, const bool bIdOnly = false);

	/** Reset all interacting states related to the transform mode to false */
	UE_API virtual void ResetInteractingStates(const EGizmoTransformMode InMode);

	/** Called at the start of a sequence of gizmo transform edits */
	UE_API virtual void BeginTransformEditSequence();

	/** Called at the end of a sequence of gizmo transform edits. */
	UE_API virtual void EndTransformEditSequence();
	
	/**
	 * Translate axis click-drag handling methods 
	 */ 

	/** Handle click press for translate X axis */
	UE_API virtual void OnClickPressTranslateXAxis(const FInputDeviceRay& PressPos);

	/** Handle click press for translate Y axis */
	UE_API virtual void OnClickPressTranslateYAxis(const FInputDeviceRay& PressPos);

	/** Handle click press for translate Z axis */
	UE_API virtual void OnClickPressTranslateZAxis(const FInputDeviceRay& PressPos);

	/** Handle click press for translate axes */
	UE_API virtual void OnClickPressAxis(const FInputDeviceRay& PressPos);

	/** Handle click drag for translate axes */
	UE_API virtual void OnClickDragTranslateAxis(const FInputDeviceRay& DragPos);

	/** Handle click release for translate axes */
	UE_API virtual void OnClickReleaseTranslateAxis(const FInputDeviceRay& ReleasePos);

	/**
	 * Translate and scale planar click-drag handling methods
	 */

	 /** Handle click press for translate XY planar */
	UE_API virtual void OnClickPressTranslateXYPlanar(const FInputDeviceRay& PressPos);

	/** Handle click press for translate YZ planar */
	UE_API virtual void OnClickPressTranslateYZPlanar(const FInputDeviceRay& PressPos);

	/** Handle click press for translate XZ planar */
	UE_API virtual void OnClickPressTranslateXZPlanar(const FInputDeviceRay& PressPos);

	/** Handle click press for generic planar */
	UE_API virtual void OnClickPressPlanar(const FInputDeviceRay& PressPos);

	/** Handle click press for translate planar */
	UE_API virtual void OnClickPressTranslatePlanar(const FInputDeviceRay& PressPos);

	/** Handle click drag for translate planar */
	UE_API virtual void OnClickDragTranslatePlanar(const FInputDeviceRay& DragPos);

	/** Handle click release for translate planar */
	UE_API virtual void OnClickReleaseTranslatePlanar(const FInputDeviceRay& ReleasePos);

	/** Compute translate axis delta based on start/end params */
	UE_API virtual FVector ComputeAxisTranslateDelta(double InStartParam, double InEndParam);

	/** Compute translate planar delta based on world space start/end points */
	UE_API virtual FVector ComputePlanarTranslateDelta(const FVector& InStartPoint, const FVector& InEndPoint);

	/**
	 * Screen-space translate interaction methods
	 */

	 /** Handle click press for screen-space translate */
	UE_API virtual void OnClickPressScreenSpaceTranslate(const FInputDeviceRay& PressPos);

	/** Handle click drag for screen-space translate */
	UE_API virtual void OnClickDragScreenSpaceTranslate(const FInputDeviceRay& DragPos);

	/** Handle click release for screen-space translate */
	UE_API virtual void OnClickReleaseScreenSpaceTranslate(const FInputDeviceRay& ReleasePos);

	/**
	 * Rotate interaction methods
	 */

	/** Handle click press for rotate X axis */
	UE_API virtual void OnClickPressRotateXAxis(const FInputDeviceRay& InPressPos);

	/** Handle click press for rotate Y axis */
	UE_API virtual void OnClickPressRotateYAxis(const FInputDeviceRay& InPressPos);

	/** Handle click press for rotate Z axis */
	UE_API virtual void OnClickPressRotateZAxis(const FInputDeviceRay& InPressPos);

	/** Handle click press for any rotate axis */
	UE_API virtual void OnClickPressRotateAxis(const FInputDeviceRay& InPressPos);

	/** Handle click drag for rotate axis */
	UE_API virtual void OnClickDragRotateAxis(const FInputDeviceRay& InDragPos);

	/** Handle click release for rotate axes */
	UE_API virtual void OnClickReleaseRotateAxis(const FInputDeviceRay& InReleasePos);

	/** Get screen-space axis for rotation drag */
	UE_API virtual FVector2D GetScreenRotateAxisDir(const FInputDeviceRay& InPressPos);
	
	/** Get screen-space axis for gimbal rotation drag */
	UE_API virtual FVector2D GetScreenGimbalRotateAxisDir(const FInputDeviceRay& InPressPos);

	/** Get screen-space axis based on a world space axis */
	UE_API FVector2D GetWorldToScreenRotateAxisDir(const FInputDeviceRay& InPressPos, const FVector& InWorldAxis);

	/** Compute angle (degrees) delta based on screen-space start/end positions. Intended to be used in combination with ComputeAxisRotateDelta */
	UE_API virtual double ComputeAxisRotateDeltaAngle(const FVector2D& InStartPos, const FInputDeviceRay& InDragPos);

	/** Compute rotate delta based on screen-space angle delta (in degrees) */
	UE_API virtual FQuat ComputeAxisRotateDelta(const double& InDeltaAngle);

	/** Compute gimbal rotate delta based on screen-space start/end positions */
	UE_API virtual FQuat ComputeGimbalRotateDelta(const FVector2D& InStartPos, const FVector2D& InEndPos, const double InAngleSign = 1.0);

	/** Prepares data for arc rotation. This will return false if this is not possible (the rotate handle is perpendicular to the view) */
	UE_API virtual bool OnClickPressRotateArc( const FInputDeviceRay& InPressPos,
		const FVector& InPlaneNormal, const FVector& InPlaneAxis1, const FVector& InPlaneAxis2, const EAxis::Type InAxis = EAxis::None);

	/**
	 * Gimbal rotate interaction methods
	 * note that gimbal rotation release functions use OnClickReleaseRotateAxis
	 */

	/** Handle click press for any gimbal rotate axis */
	UE_API virtual void OnClickPressGimbalRotateAxis(const FInputDeviceRay& InPressPos);

	/** Handle click drag for gimbal rotate axis */
	UE_API virtual void OnClickDragGimbalRotateAxis(const FInputDeviceRay& InDragPos);
	
	/**
	 * Screen-space rotate interaction methods
	 */

	 /** Handle click press for screen-space rotate */
	UE_API virtual void OnClickPressScreenSpaceRotate(const FInputDeviceRay& PressPos);

	/** Handle click drag for screen-space rotate */
	UE_API virtual void OnClickDragScreenSpaceRotate(const FInputDeviceRay& InDragPos);

	/** Handle click release for screen-space rotate */
	UE_API virtual void OnClickReleaseScreenSpaceRotate(const FInputDeviceRay& ReleasePos);

	/** Compute rotate delta based on start/end angles */
	UE_API virtual FQuat ComputeAngularRotateDelta(FQuat::FReal InStartAngle, FQuat::FReal InEndAngle);
	
	/**
	 * Arc ball rotate interaction methods
	 */

	/** Handle click press for arc ball rotate */
	UE_API virtual void OnClickPressArcBallRotate(const FInputDeviceRay& PressPos);

	/** Handle click drag for arc ball rotate */
	UE_API virtual void OnClickDragArcBallRotate(const FInputDeviceRay& DragPos);

	/** Handle click release for arc ball rotate */
	UE_API virtual void OnClickReleaseArcBallRotate(const FInputDeviceRay& ReleasePos);

	/** Get the arc ball sphere world radius */
	UE_API float GetWorldRadius(const float InRadius) const;

	/** */
	UE_API float GetSizeCoefficient() const;
	
	/**
	* Scale click-drag handling methods
	*/

	/** Handle click press for scale X axis */
	UE_API virtual void OnClickPressScaleXAxis(const FInputDeviceRay& PressPos);

	/** Handle click press for scale Y axis */
	UE_API virtual void OnClickPressScaleYAxis(const FInputDeviceRay& PressPos);

	/** Handle click press for scale Z axis */
	UE_API virtual void OnClickPressScaleZAxis(const FInputDeviceRay& PressPos);

	/** Handle click press for scale XY planar */
	UE_API virtual void OnClickPressScaleXYPlanar(const FInputDeviceRay& PressPos);

	/** Handle click press for scale YZ planar */
	UE_API virtual void OnClickPressScaleYZPlanar(const FInputDeviceRay& PressPos);

	/** Handle click press for scale XZ planar */
	UE_API virtual void OnClickPressScaleXZPlanar(const FInputDeviceRay& PressPos);

	/** Handle click press for uniform scale */
	UE_API virtual void OnClickPressScaleXYZ(const FInputDeviceRay& PressPos);

	/** Handle click press for all scale methods */
	UE_API virtual void OnClickPressScale(const FInputDeviceRay& PressPos);

	/** Handle click drag for scale axes */
	UE_API virtual void OnClickDragScaleAxis(const FInputDeviceRay& DragPos);

	/** Handle click drag for scale planar */
	UE_API virtual void OnClickDragScalePlanar(const FInputDeviceRay& DragPos);

	/** Handle click drag for uniform scale */
	UE_API virtual void OnClickDragScaleXYZ(const FInputDeviceRay& DragPos);

	/** Handle click drag for all scale */
	UE_API virtual void OnClickDragScale(const FInputDeviceRay& DragPos);

	/** Handle click release for scale axes */
	UE_API virtual void OnClickReleaseScaleAxis(const FInputDeviceRay& ReleasePos);

	/** Handle click release for scale planar */
	UE_API virtual void OnClickReleaseScalePlanar(const FInputDeviceRay& ReleasePos);

	/** Handle click release for uniform scale */
	UE_API virtual void OnClickReleaseScaleXYZ(const FInputDeviceRay& ReleasePos);

	/** Compute scale delta based on screen space start/end positions */
	UE_API virtual FVector ComputeScaleDelta(const FVector2D& InStartPos, const FVector2D& InEndPos, FVector2D& OutScreenDelta);

	/**
	 * Screen-space helper method
	 */

	/** Returns 2D vector projection of input axis onto input view plane */
	static UE_API FVector2D GetScreenProjectedAxis(const UGizmoViewContext* View, const FVector& InLocalAxis, const FTransform& InLocalToWorld = FTransform::Identity);

	/**
	 * Apply transform delta methods
	 */
	 
	/** Apply translate delta to transform proxy */
	UE_API virtual void ApplyTranslateDelta(const FVector& InTranslateDelta);

	/** Apply rotate delta to transform proxy */
	UE_API virtual void ApplyRotateDelta(const FQuat& InRotateDelta);

	/** Apply scale delta to transform proxy */
	UE_API virtual void ApplyScaleDelta(const FVector& InScaleDelta);

	// Axis and Plane TransformSources use this function to execute worldgrid snap queries
	UE_API bool PositionSnapFunction(const FVector& WorldPosition, FVector& SnappedPositionOut) const;
	UE_API FQuat RotationSnapFunction(const FQuat& DeltaRotation) const;

	UE_API void SnapTranslateDelta(FVector& InOutWorldDelta) const;
	UE_API void SnapRotateDelta(FQuat& InOutWorldDelta) const;
	UE_API void SnapRotateAngleDelta(double& InOutAngleDelta) const;
	UE_API void SnapScaleDelta(FVector& InOutLocalScaleDelta) const;

	// Get max part identifier.
	UE_API virtual uint32 GetMaxPartIdentifier() const;

	// Verify part identifier is within recognized range of transform gizmo part ids
	UE_API virtual bool VerifyPartIdentifier(uint32 InPartIdentifier) const;

	// Returns whether the gizmo is visible in the viewport.
	UE_API bool IsVisible(const EViewportContext InViewportContext = EViewportContext::Focused) const;

	// Returns whether the gizmo can interact. Note that this can be true even if the gizmo is not visible to support indirect manipulation.
	UE_API virtual bool CanInteract(const EViewportContext InViewportContext = EViewportContext::Focused) const;

	// Returns the current rotation context.
	UE_API const FRotationContext& GetRotationContext() const;
	
protected:

	/** Materials and colors to be used when drawing the items for each axis */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> TransparentVertexColorMaterial;
	UPROPERTY()
	TObjectPtr<UMaterialInterface> GridMaterial;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialX;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialY;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialZ;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> CurrentAxisMaterial;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> GreyMaterial;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> WhiteMaterial;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> OpaquePlaneMaterialXY;

	/** Array of function pointers, indexed by gizmo part id, to handle click press behavior */
	TArray<TFunction<void(UTransformGizmo* TransformGizmo, const FInputDeviceRay& PressPos)> > OnClickPressFunctions;

	/** Array of function pointers, indexed by gizmo part id, to handle click drag behavior */
	TArray<TFunction<void(UTransformGizmo* TransformGizmo, const FInputDeviceRay& DragPos)> > OnClickDragFunctions;

	/** Array of function pointers, indexed by gizmo part id, to handle click release behavior */
	TArray<TFunction<void(UTransformGizmo* TransformGizmo, const FInputDeviceRay& ReleasePos)> > OnClickReleaseFunctions;

	/** Customization function (to override default material or increment gizmo size for example) */
	TFunction<const FGizmoCustomization()> CustomizationFunction;

	/** Percentage-based scale multiplier */
	UPROPERTY()
	double ScaleMultiplier = 0.01;

	/** Current transform */
	UPROPERTY()
	FTransform CurrentTransform = FTransform::Identity;

	/** Currently rendered transform mode */
	UPROPERTY()
	EGizmoTransformMode CurrentMode = EGizmoTransformMode::None;

	/** Currently rendered axis list */
	UPROPERTY()
	TEnumAsByte<EAxisList::Type> CurrentAxisToDraw = EAxisList::None;

	/** Last hit part */
	UPROPERTY()
	ETransformGizmoPartIdentifier LastHitPart = ETransformGizmoPartIdentifier::Default;

	/** Last hit part per mode */
	ETransformGizmoPartIdentifier LastHitPartPerMode[static_cast<uint8>(EGizmoTransformMode::Max)];
	
	UE_API ETransformGizmoPartIdentifier GetCurrentModeLastHitPart() const;
	UE_API void SetModeLastHitPart(const EGizmoTransformMode InMode, const ETransformGizmoPartIdentifier InIdentifier);

	UE_API EAxisList::Type GetInteractionAxisList() const;
	UE_API virtual void SetInteractionAxisList(const EAxisList::Type InAxisList);

	//
	// The values below are used in the context of a single click-drag interaction, ie if bInInteraction = true
	// They otherwise should be considered uninitialized
	//

	UE_DEPRECATED(5.8, "Direct access to InteractionAxisList is deprecated. Please use the getter/setter.")
	/** Active axis type (only valid between state target BeginModify/EndModify) */
	UPROPERTY(Getter, Setter)
	TEnumAsByte<EAxisList::Type> InteractionAxisList;

	/** Active world space axis origin (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector InteractionAxisOrigin;

	/** Active world space axis (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector InteractionAxisDirection;

	/** Active interaction start hit param (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	double InteractionAxisStartParam;

	/** Active interaction current hit param (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	double InteractionAxisCurrParam;

	/** Active world space planar origin (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector InteractionPlanarOrigin;

	/** Active world space normal used for planar (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector InteractionPlanarNormal;

	/** Active normal to remove from axis translation (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector NormalToRemove;

	/** Active world space axis X used for planar (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector InteractionPlanarAxisX;

	/** Active world space axis Y used for planar (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector InteractionPlanarAxisY;

	/** Active interaction start point planar (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector InteractionPlanarStartPoint;

	/** Active interaction current point planar (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector InteractionPlanarCurrPoint;

	/** Active interaction rotation start angle (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	double InteractionStartAngle;

	/** Active interaction rotation curr angle (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	double InteractionCurrAngle;

	/** Active interaction screen axis dir (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector2D InteractionScreenAxisDirection;

	/** Active normal projection to remove from drag when rotating */
	UPROPERTY()
	FVector2D NormalProjectionToRemove;

	/** Active interaction screen start pos (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector2D InteractionScreenStartPos;

	/** Active interaction screen end pos (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector2D InteractionScreenEndPos;

	/** Active interaction screen current pos (only valid between state target BeginModify/EndModify) */
	UPROPERTY()
	FVector2D InteractionScreenCurrPos;

	/** Active interaction arc ball start point */
	UPROPERTY()
	FVector InteractionArcBallStartPoint;

	/** Active interaction arc ball current point */
	UPROPERTY()
	FVector InteractionArcBallCurrPoint;

	/** Arc ball start rotation */
	UPROPERTY()
	FQuat StartRotation = FQuat::Identity;

	/** Arc ball current rotation */
	FQuat CurrentRotation = FQuat::Identity;

	/** Indirect manipulation */
	UPROPERTY()
	bool bIndirectManipulation;

	/** Defer drag function on tick to avoid firing too many drag moves */
	UPROPERTY()
	bool bDeferDrag = true;

	/** Pending drag function to be called if bDeferDrag is true */
	TFunction<void()> PendingDragFunction;

	/**
	 * When disabled, Ctrl+LMB manipulates X, Ctrl+RMB manipulates Y, and Ctrl+LMB+RMB manipulates Z.
	 * When enabled LMB is mapped to X, MMB is mapped to Y, and RMB is mapped to Z. 
	 */
	UPROPERTY()
	bool bSequentialIndirectAxesButtons = true;

	/** When enabled, all indirect scale manipulations will trigger uniform scaling */
	UPROPERTY()
	bool bUniformIndirectScale = false;
	
	/**
	 * When enabled and in an orthographic view, Ctrl+LMB-Drag will move or scale in screen space, and Ctrl+RMB-Drag will rotate in screen space.  
	 */
	UPROPERTY()
	bool bScreenSpaceIndirectOrthographicManipulation = true;

	/** Default rotate mode used when using axis rotation handles. */
	UPROPERTY()
	TEnumAsByte<EAxisRotateMode::Type> DefaultRotateMode = EAxisRotateMode::ScreenArc;

	/** Actual rotate mode used (based on view dependant information). */
	TEnumAsByte<EAxisRotateMode::Type> RotateMode = EAxisRotateMode::ScreenArc;

	/** Switch from tangential to normal projection based on the first mouse drag. */
	bool bTrySwitchingToNormalPull = false;

	/** Used to check if the gimbal mode is currently active (this is updated when ticking the gizmo) */
	bool bGimbalRotationMode = false;
	
	/** pre-click delegate called before a click & drag interaction can start. */
	UE_INTERNAL FOnGizmoInteraction OnPreCanInteractDelegate;
	
	/** conditional post-release delegate after a click & drag interaction ends. */
	UE_INTERNAL FOnGizmoInteraction OnPostInteractionDelegate;
	
public:
	
	/** Delegate to a pre-click called delegate before a click & drag interaction can start. */
	UE_INTERNAL UE_API FOnGizmoInteraction& OnPreCanInteract();
	
	/** Delegate to a conditional post-release delegate after a click & drag interaction ends. */
	UE_INTERNAL UE_API FOnGizmoInteraction& OnPostInteraction();
	
private:

	struct FGizmoDebugData
	{
		/** Determines whether certain data is displayed, ie. drag operation deltas. */
		bool bIsEditing = false;

		FTransform TransformStart;
		FTransform TransformCurrent;

		FTransform InteractionStart;
		FTransform InteractionCurrent;
		FVector InteractionPlaneNormal;

		/** Can indicate a 2D drag direction, etc. */
		FVector2D InteractionScreenDirection;

		/** Debug attributes to display the pull direction */
		bool bDebugRotate = false;
		FVector DebugDirection = FVector::ZeroVector;
		FVector DebugClosest = FVector::ZeroVector;
		FVector DebugNormalRemoved = FVector::ZeroVector;
		FVector DebugNormalSkip = FVector::ZeroVector;
		double InteractionAngleStart;
		double InteractionAngleCurrent;
		double InteractionRadius;
	} DebugData;
};

#undef UE_API
