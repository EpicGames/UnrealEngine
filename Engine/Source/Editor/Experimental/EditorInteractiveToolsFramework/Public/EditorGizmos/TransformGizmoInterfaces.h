// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolContextInterfaces.h"
#include "UnrealWidgetFwd.h"
#include "EulerTransform.h"
#include "TransformGizmoInterfaces.generated.h"

class UMaterial;

//
// Interface for the Transform gizmo.
//
UENUM()
enum class EGizmoTransformMode : uint8
{
	None = 0,
	Translate,
	Rotate,
	Scale,
	Max
};

UENUM()
namespace EAxisRotateMode
{
	enum Type : uint8
	{
		Pull = 0,
		Arc = 1,
		ScreenArc = 2, // Like Arc, but in screen space, rather than aligned to the associated axis
	};
}

UENUM()
enum class EGizmoTransformScaleType : uint8
{
	/** Apply scale as an additive change to the current scale value. */
	Default = 0 UMETA(DisplayName="Offset"),

	/** Apply scale proportionally, multiplying the starting scale value. */	
	PercentageBased = 1,

	/** Use the legacy editor settings to determine how scale is applied. */
	Legacy = 2
};

enum class EViewportContext : uint8
{
	Hovered,
	Focused
};

/**
 * FRotationContext is a datastore for rotation data sent to the gizmo
 */

USTRUCT()
struct FRotationContext
{
	GENERATED_BODY()

	/** Enable explicit rotation axis */
	UPROPERTY()
	bool bUseExplicitRotator = false;

	/** Order in which this rotation is expressed */
	UPROPERTY()
	EEulerRotationOrder RotationOrder = EEulerRotationOrder::XYZ;

	/** Rotation values */
	UPROPERTY()
	FRotator Rotation = FRotator::ZeroRotator;

	/** Parent offset transform */
	UPROPERTY()
	FTransform Offset = FTransform::Identity;
};

UINTERFACE(MinimalAPI)
class UTransformGizmoSource : public UInterface
{
	GENERATED_BODY()
};

/**
 * ITransformGizmoSource is an interface for providing gizmo mode configuration information.
 */
class ITransformGizmoSource
{
	GENERATED_BODY()
public:
	/**
	 * Returns the current Editor gizmo mode
	 */
	virtual EGizmoTransformMode GetGizmoMode() const PURE_VIRTUAL(ITransformGizmoSource::GetGizmoMode, return EGizmoTransformMode::None;);

	/**
	 * Returns the current gizmo axes to draw
	 */
	virtual EAxisList::Type GetGizmoAxisToDraw(EGizmoTransformMode InGizmoMode) const PURE_VIRTUAL(ITransformGizmoSource::GetGizmoAxisToDraw, return EAxisList::None;);

	/**
	 * Returns the coordinate system space (world or local) in which to display the gizmo
	 */
	virtual EToolContextCoordinateSystem GetGizmoCoordSystemSpace() const PURE_VIRTUAL(ITransformGizmoSource::GetGizmoCoordSystemSpace, return EToolContextCoordinateSystem::World;);

	/**
	 * Returns a scale factor for the gizmo
	 */
	virtual float GetGizmoScale() const PURE_VIRTUAL(ITransformGizmoSource::GetGizmoScale, return 1.0f;);

	/* 
	 * Returns whether the gizmo should be visible. 
	 */
	virtual bool GetVisible(const EViewportContext InViewportContext = EViewportContext::Focused) const PURE_VIRTUAL(ITransformGizmoSource::GetVisible, return false;);

	/* 
	 * Returns whether the gizmo can interact.
	 */
	virtual bool CanInteract(const EViewportContext InViewportContext = EViewportContext::Focused) const PURE_VIRTUAL(ITransformGizmoSource::CanInteract, return false;);

	/*
	 * Get current scale type
	 */
	virtual EGizmoTransformScaleType GetScaleType() const PURE_VIRTUAL(ITransformGizmoSource::GetScaleType, return EGizmoTransformScaleType::Default;);

	/*
	 * Get rotation context
	 */
	virtual const FRotationContext& GetRotationContext() const PURE_VIRTUAL(ITransformGizmoSource::GetRotationContext, static const FRotationContext Context; return Context;);
};

/**
 * FGizmoCustomization is a struct that can be used to make some display overrides (currently material and size) 
 */

struct FGizmoCustomization
{
	TObjectPtr<UMaterial> Material = nullptr;
	float SizeCoefficient = 1.f;
};
