// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveGizmoBuilder.h"
#include "ToolContextInterfaces.h"
#include "EditorInteractiveGizmoConditionalBuilder.generated.h"

/**
 * FEditorGizmoTypePriority is used to establish relative priority between conditional 
 * gizmo builders. It is up to the gizmo manager to determine how the priority is used. 
 * In the EditorInteractiveGizmoManager, if more than one gizmo builder returns true 
 * from SatsifiesCondition(), the gizmo builder with highest priority will be used. If 
 * there are multiple builders the highest priority, multiple gizmos will be built.
 */
struct FEditorGizmoTypePriority
{
	static constexpr int DEFAULT_GIZMO_TYPE_PRIORITY = 50;

	/** Constant priority value */
	int Priority;

	FEditorGizmoTypePriority(int InPriority = DEFAULT_GIZMO_TYPE_PRIORITY)
	{
		Priority = InPriority;
	}

	/** @return a priority lower than this priority */
	FEditorGizmoTypePriority MakeLower(int DeltaAmount = 1) const
	{
		return FEditorGizmoTypePriority(Priority + DeltaAmount);
	}

	/** @return a priority higher than this priority */
	FEditorGizmoTypePriority MakeHigher(int DeltaAmount = 1) const
	{
		return FEditorGizmoTypePriority(Priority - DeltaAmount);
	}

	friend bool operator<(const FEditorGizmoTypePriority& l, const FEditorGizmoTypePriority& r)
	{
		return l.Priority < r.Priority;
	}
	friend bool operator==(const FEditorGizmoTypePriority& l, const FEditorGizmoTypePriority& r)
	{
		return l.Priority == r.Priority;
	}
	friend bool operator>(const FEditorGizmoTypePriority& l, const FEditorGizmoTypePriority& r)
	{
		return l.Priority > r.Priority;
	}
};

/** 
 * Gizmo builders which should be built once a condition is satisfied in the current scene state
 * implement the IEditorInteractiveGizmoConditionalBuilder interface. The SatisfiesCondition method
 * should return true whenever the builder is buildable based on scene state, most commonly based
 * on selection. The builder's priority should be specified such that when more than one builder
 * is discovered, the builder with highest priority will be built.
 */
UINTERFACE(MinimalAPI)
class UEditorInteractiveGizmoConditionalBuilder : public UInterface
{
	GENERATED_BODY()
};

class IEditorInteractiveGizmoConditionalBuilder
{
	GENERATED_BODY()

public:

	/** Returns the priority for this gizmo type.  */
	virtual FEditorGizmoTypePriority GetPriority() const = 0;

	/** Update the priority for this gizmo type.  */
	virtual void SetPriority(const FEditorGizmoTypePriority& InPriority) = 0;

	/** Returns true if this gizmo is valid for creation based on the current state. */
	virtual bool SatisfiesCondition(const FToolBuilderState& SceneState) const = 0;

};
