// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "EditorInteractiveGizmoRegistry.h"
#include "InteractiveGizmoManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "EditorGizmos/TransformGizmo.h"

#include "EditorInteractiveGizmoManager.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class FCanvas;
class FEditorModeTools;
class IToolsContextQueriesAPI;
class IToolsContextRenderAPI;
class IToolsContextTransactionsAPI;
class UEdModeInteractiveToolsContext;
class UInputRouter;
class UInteractiveGizmo;
class UInteractiveGizmoBuilder;
class UObject;
class UTypedElementSelectionSet;
struct FToolBuilderState;
class UTransformGizmo;

USTRUCT()
struct FActiveEditorGizmo
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UInteractiveGizmo> Gizmo = nullptr;
	void* Owner = nullptr;
};

/**
 * UEditorInteractiveGizmoManager allows users of the Tools framework to register and create selection-based Gizmo instances.
 * For each selection-based Gizmo, a builder derived from UInteractiveGizmoSelectionBuilder is registered with the GizmoManager.
 * When the section changes, the highest priority builders for which SatisfiesCondition() return true, will be used to
 * build gizmos.
 */
UCLASS(MinimalAPI, Transient)
class UEditorInteractiveGizmoManager : public UInteractiveGizmoManager
{
	GENERATED_BODY()

protected:
	friend class UEditorInteractiveToolsContext;		// to call Initialize/Shutdown

	UE_API UEditorInteractiveGizmoManager();

	/** Initialize the GizmoManager with the necessary Context-level state. UEdModeInteractiveToolsContext calls this, you should not. */
	UE_API virtual void InitializeWithEditorModeManager(IToolsContextQueriesAPI* QueriesAPI, IToolsContextTransactionsAPI* TransactionsAPI, UInputRouter* InputRouter, FEditorModeTools* InEditorModeManager);

	// UInteractiveGizmoManager interface
	UE_API virtual void Shutdown() override;

public:

	// UInteractiveGizmoManager interface
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	/**
	 * Register a new Editor gizmo type.
	 * @param InGizmoCategory category in which to register gizmo builder
	 * @param InGizmonBuilder new Editor gizmo builder
	 * - Accessory and Primary gizmo builders must derive from UInteractiveGizmoBuilder (of from a builder derived from it)
	 *   and must implement the IEditorInteractiveConditionalGizmoBuilder and IEditorInteractiveSelectionGizmoBuilder interfaces.
	 */
	UE_API void RegisterEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder);

	/**
	* Remove an Editor gizmo type from the set of known Editor gizmo types
	* @param InGizmoBuilder same object pointer that was passed to RegisterEditorGizmoType()
	* @return true if gizmo type was found and deregistered
	*/
	UE_API void DeregisterEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder);

	/**
	 * Get all qualified Editor gizmo builders for the specified category, based on the current state. Qualification is determined by the gizmo builder
	 * returning true from SatisfiesCondition() and relative priority. All qualified builders at the highest found priority
	 * will be returned.
	 * @param InGizmoCategory category in which to search for qualified builders
	 * @param InToolBuilderState current selection and other state
	 * @return array of qualified Gizmo selection builders based on current state
	 */
	UE_API void GetQualifiedEditorGizmoBuilders(EEditorGizmoCategory InGizmoCategory, const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& InFoundBuilders);

	/** 
	 * Set how auto gizmo resolution should occur when CreateGizmosForCurrentState is invoked. If bSearchLocalOnly is true, only the current
	 * @param bLocalOnly - if true, only the current gizmo manager registry will be searched for candidate gizmos. If false,
	 *   both the gizmo manager registry and any higher gizmo manager or gizmo subsystem (in the case of selection builders) will be searched
	 */
	virtual void SetEditorGizmoBuilderResolution(bool bLocalOnly)
	{
		bSearchLocalBuildersOnly = bLocalOnly;
	}

	/**
	 * Returns the current auto gizmo resolution setting 
	 */
	virtual bool GetEditorGizmoBuilderResolution() const
	{
		return bSearchLocalBuildersOnly;
	}

	/**
	 * Gets whether editor gizmos are allowed
	 */
	UE_API bool AreEditorGizmosAllowed() const; 

	/**
	 * Sets whether editor gizmos are allowed
	 */
	UE_API void SetAllowEditorGizmos(bool bInAllowed);

	/**
	 * Shutdown and remove a selection-based Gizmo
	 * @param Gizmo the Gizmo to shutdown and remove
	 * @return true if the Gizmo was found and removed
	 */
	UE_API virtual bool DestroyEditorGizmo(UInteractiveGizmo* Gizmo);

	/**
	 * Shutdown and remove all active auto gizmos
	 */
	UE_API virtual void DestroyAllEditorGizmos();

	/** Try to activate a new Gizmo instance (UInteractiveGizmoManager override) */	
	UE_API virtual UInteractiveGizmo* CreateGizmo(
		const FString& BuilderIdentifier, const FString& InstanceIdentifier = FString(), void* Owner = nullptr) override;

	/** Shutdown and remove a Gizmo (UInteractiveGizmoManager override) */
	UE_API virtual bool DestroyGizmo(UInteractiveGizmo* InGizmo) override;

	/** instance/builder identifiers for transform gizmo */
	static UE_API const FString& TransformInstanceIdentifier();
	static UE_API const FString& TransformBuilderIdentifier();

	/**
	 * Returns true if editor gizmos are enabled for the provided ModeTools instance.
	 */
	static UE_API bool AreEditorGizmosAllowed(const FEditorModeTools* ModeTools);

	/**
	 * Returns true if the new TRS gizmos are used.
	 */
	static UE_API bool UsesNewTRSGizmos();

	/**
	 * Updates the current New TRS Gizmo state and notifies that change using OnUsesNewTRSGizmosChangedDelegate  
	 */
	static UE_API void SetUsesNewTRSGizmos(const bool bUseNewTRSGizmos);

	/**
	 * Returns the current gizmo version. 0 = legacy, 1 = TransformGizmo, 2 = EditorTRSGizmo.
	 * Reads from existing settings (bUseExperimentalGizmo, bUseEditorTRSGizmo).
	 */
	static UE_API int32 GetTransformGizmoVersion();

	/**
	 * Sets the gizmo version by writing to the existing settings.
	 * Writes to existing settings (bUseExperimentalGizmo, bUseEditorTRSGizmo).
	 */
	static UE_API void SetTransformGizmoVersion(const int32 InVersion);

	/**
	 * Delegate to notify from bUseNewTRSGizmos changes
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUsesNewTRSGizmosChanged, const bool bUseNewTRSGizmos);
	static UE_API FOnUsesNewTRSGizmosChanged& OnUsesNewTRSGizmosChangedDelegate();

	/**
	 * Notifies FGizmosParameters changes across bound transform gizmos
	 */
	static UE_API void SetGizmosParameters(const FGizmosParameters& InParameters);

	/**
	 * Delegate to notify from FGizmosParameters changes  
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGizmosParametersChanged, const FGizmosParameters& InParameters);
    static UE_API FOnGizmosParametersChanged& OnGizmosParametersChangedDelegate();

	/**
	 * Returns the default gizmos parameters if set
	 */
    static UE_API const TOptional<FGizmosParameters>& GetDefaultGizmosParameters();

	/**
	 * Returns true if the Explicit mode is enabled;
	 */
	static UE_API bool IsExplicitModeEnabled();

protected:

	/**
	 * Returns true if selection gizmos should be visible. 
	 * @todo move this to a gizmo context object
	 */
	UE_API virtual bool GetShowEditorGizmos();

	/**
	 * Returns true if gizmos should be visible based on the current view's engine show flag.
	 * @todo move this to a gizmo context object
	 */
	UE_API virtual bool GetShowEditorGizmosForView(IToolsContextRenderAPI* RenderAPI);

	/**
	 * Updates active selection gizmos when show selection state changes
	 */
	UE_API void UpdateActiveEditorGizmos();

	/** Actual registry */
	UPROPERTY()
	TObjectPtr<UEditorInteractiveGizmoRegistry> Registry;

	/** set of Currently-active Gizmos */
	UPROPERTY()
	TArray<FActiveEditorGizmo> ActiveEditorGizmos;

	UPROPERTY()
	TOptional<FGizmosParameters> DefaultGizmoParameters;

	/** If false, only search gizmo builders in current gizmo manager. If true, also search gizmo subsystem */
	bool bSearchLocalBuildersOnly = false;

	/** Cache for already built gizmos, currently this only caches the transform */
	UPROPERTY()
	TMap<TObjectPtr<UInteractiveGizmoBuilder>, TObjectPtr<UInteractiveGizmo>> CachedGizmoMap;

	/** @todo: remove when GetShowEditorGizmos() is moved to gizmo context object */
	FEditorModeTools* EditorModeManager = nullptr;

	/** Whether Editor gizmos are ever allowed in this context. Layered on top of UsesNewTRSGizmos() */
	bool bAllowEditorGizmos = true;
	
	/** Whether Editor gizmos are enabled. UpdateActiveEditorGizmos() determines this value each tick and updates if it has changed. */
	bool bShowEditorGizmos = false;

private:

	/** Returns the existing default Gizmo instance if any. */
	UE_API UTransformGizmo* FindDefaultTransformGizmo() const;
};

#undef UE_API
