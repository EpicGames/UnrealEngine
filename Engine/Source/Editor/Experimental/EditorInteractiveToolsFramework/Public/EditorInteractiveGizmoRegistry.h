// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EditorInteractiveGizmoSelectionBuilder.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "EditorInteractiveGizmoRegistry.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UInteractiveGizmoBuilder;
struct FToolBuilderState;
template <typename T> struct TObjectPtr;

/** Gizmo category used for registering Editor gizmo selection-based builders */
UENUM()
enum class EEditorGizmoCategory :uint8
{
	/** Accessory gizmos, built simultaneously with the Level Editor TRS gizmo. */
	Accessory,

	/** Primary gizmos, built in place of the Level Editor TRS gizmo.  */
	Primary
};

UCLASS(Abstract)
class UEditorGizmoRegistryCategoryEntry : public UObject
{
	GENERATED_BODY()
public:

	/** 
	 * Gets qualified gizmo builders, replacing those already found if necessary.
	 * @param FoundBuilders inputs qualified builders found so far, and outputs qualified builders for this category
	 *                      replacing 
	 */
	virtual void GetQualifiedGizmoBuilders(const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& InFoundBuilders) PURE_VIRTUAL(UEditorGizmoRegistryCategoryEntry::GetQualifiedGizmoBuilders, );

	virtual void RegisterGizmoType(UInteractiveGizmoBuilder* InGizmoBuilder);

	virtual void DeregisterGizmoType(UInteractiveGizmoBuilder* InGizmoBuilder);

	virtual void ClearGizmoTypes();

	UPROPERTY()
	TArray<TObjectPtr<UInteractiveGizmoBuilder>> GizmoTypes;

protected:

	UPROPERTY()
	FString CategoryName;
};

UCLASS(Abstract)
class UEditorGizmoRegistryCategoryEntry_ConditionalSelection : public UEditorGizmoRegistryCategoryEntry
{
	GENERATED_BODY()
public:

	virtual void GetQualifiedGizmoBuilders(const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& InFoundBuilders) PURE_VIRTUAL(UEditorGizmoRegistryCategoryEntry::GetQualifiedGizmoBuilders, );

	virtual void RegisterGizmoType(UInteractiveGizmoBuilder* InGizmoBuilder);
};


UCLASS()
class UEditorGizmoRegistryCategoryEntry_Primary : public UEditorGizmoRegistryCategoryEntry_ConditionalSelection
{
	GENERATED_BODY()
public:
	UEditorGizmoRegistryCategoryEntry_Primary();

	virtual void GetQualifiedGizmoBuilders(const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& FoundBuilders);
};

UCLASS()
class UEditorGizmoRegistryCategoryEntry_Accessory : public UEditorGizmoRegistryCategoryEntry_ConditionalSelection
{
	GENERATED_BODY()
public:
	UEditorGizmoRegistryCategoryEntry_Accessory();

	virtual void GetQualifiedGizmoBuilders(const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& FoundBuilders);
};

/**
 * Gizmo types should be registered in either UEditorInteractiveGizmoSubsystem or
 * UEditorInteractiveGizmoManager. This registry class is used internally by the
 * subsystem and manager which each maintain its own registry at different scopes:
 * the subystem is global to the Editor, the manager is local to the Interactive
 * Tools Context.
 */
UCLASS(MinimalAPI)
class UEditorInteractiveGizmoRegistry : public UObject
{
	GENERATED_BODY()

public:

	UE_API UEditorInteractiveGizmoRegistry();

	/** Shutdown the registry, called by the gizmo subsystem and gizmo manager when they are shutdown/deinitialized. */
	UE_API virtual void Shutdown();

	/**
	 * Register a new Editor gizmo type.
	 * @param InGizmoCategory category in which to register gizmo builder
	 * @param InGizmoBuilder new Editor gizmo builder
	 * - Accessory and Primary gizmo builders must derive from UInteractiveGizmoBuilder (of from a builder derived from it)
	 *   and must implement the IEditorInteractiveConditionalGizmoBuilder and IEditorInteractiveSelectionGizmoBuilder interfaces.
	 */
	UE_API void RegisterEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder);

	/**
	* Remove an Editor gizmo type from the set of known Editor gizmo types
	* @param InGizmoBuilder same object pointer that was passed to RegisterEditorGizmoType()
	* @return true if gizmo type was found and deregistered
	*/
	UE_API void DeregisterEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder);

	/**
	 * Clear all registered gizmo types
	 */
	UE_API void ClearEditorGizmoTypes();

	/**
	 * Get all qualified Editor gizmo builders for the specified category, based on the current state. Qualification is determined by the gizmo builder
	 * returning true from SatisfiesCondition() and relative priority. All qualified builders at the highest found priority
	 * will be returned.
	 * @param InGizmoCategory category in which to search for qualified builders
	 * @param InToolBuilderState current selection and other state
	 * @param FoundBuilders input and output parameter, inputs currently qualified builders, outputs resulting qualified 
	 *                      builders which may add to or replace the input found builders.
	 * @return array of qualified Gizmo selection builders based on current state
	 */
	UE_API void GetQualifiedEditorGizmoBuilders(EEditorGizmoCategory InGizmoCategory, const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& FoundBuilders);

private:

	/** Current set of Gizmo Builders */
	UPROPERTY()
	TMap<EEditorGizmoCategory, TObjectPtr<UEditorGizmoRegistryCategoryEntry>> GizmoCategoryMap;
};

#undef UE_API
