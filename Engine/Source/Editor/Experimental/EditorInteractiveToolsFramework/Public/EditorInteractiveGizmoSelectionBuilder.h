// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorInteractiveGizmoConditionalBuilder.h"
#include "ToolContextInterfaces.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorInteractiveGizmoSelectionBuilder.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UInteractiveGizmo;
class UObject;
class UTransformProxy;
struct FToolBuilderState;

class FEditorGizmoSelectionBuilderHelper
{
public:
	/**
	 * Utility method that creates a transform proxy based on the current selection.
	 */
	static UE_API UTransformProxy* CreateTransformProxyForSelection(const FToolBuilderState& SceneState);
};

UINTERFACE(MinimalAPI)
class UEditorInteractiveGizmoSelectionBuilder : public UInterface
{
	GENERATED_BODY()
};

class IEditorInteractiveGizmoSelectionBuilder
{
	GENERATED_BODY()

public:

	/**
	 * Update gizmo's active target based on the current Editor selection and scene state.  This method creates 
	 * a transform proxy for the current selection and sets the gizmo's active target to the new transform proxy. 
	 * This method is called after a gizmo is automatically built based upon selection and also to update the 
	 * existing gizmo when the selection changes.
	 */
	virtual void UpdateGizmoForSelection(UInteractiveGizmo* Gizmo, const FToolBuilderState& SceneState) = 0;

};

#undef UE_API
