// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EditorInteractiveGizmoRegistry.h"
#include "EditorInteractiveGizmoSelectionBuilder.h"
#include "EditorSubsystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "EditorInteractiveGizmoSubsystem.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class FSubsystemCollectionBase;
class UInteractiveGizmoBuilder;
class UObject;
struct FToolBuilderState;

// @NOTE: The following changes should be made (reverted) prior to 5.7 Release:
// - Remove RegisterTransformGizmoBuilder

/**
 * The InteractiveGizmoSubsystem provides methods for registering and unregistering 
 * selection-based gizmo builders. This subsystem will be queried for qualified 
 * builders based on the current selection.
 *
 * This subsystem should be used to register gizmo selection-based builders which are not specific
 * to an ed mode or asset editor. 
 * For gizmo selection-based builders which are specific to an ed mode or asset editor,
 * register with the UEditorinteractiveGizmoManager instead, when the ed mode or asset editor
 * starts up (and deregister when the mode or asset editor shuts down).
 * 
 * Plugins registering gizmo types should bind to the delegates returned by:
 * - OnEditorGizmoSubsystemRegisterEditorGizmoTypes()
 * - OnEditorGizmoSubsystemDeregisterEditorGizmoTypes()
 * to register and dergister their gizmo builders.
 */
 UCLASS(MinimalAPI)
class UEditorInteractiveGizmoSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	UE_API UEditorInteractiveGizmoSubsystem();

	//~ Begin USubsystem interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	UE_API virtual void Deinitialize() override;
	//~ End USubsystem interface

	/**
	 * Event which is broadcast just after default types are registered in the gizmo subsystem
	 */
	DECLARE_EVENT(UEditorInteractiveSelectionGizmoSubsystem, FOnEditorGizmoSubsystemRegisterGlobalEditorGizmoTypes);
	FOnEditorGizmoSubsystemRegisterGlobalEditorGizmoTypes& OnEditorGizmoSubsystemRegisterGlobalEditorGizmoTypes() { return RegisterGlobalEditorGizmoTypesDelegate; }

	/**
	 * Event which is broadcast just before default types are deregistered in the gizmo subsystem
	 */
	DECLARE_EVENT(UEditorInteractiveSelectionGizmoSubsystem, FOnEditorGizmoSubsystemDeregisterGlobalEditorGizmoTypes);
	FOnEditorGizmoSubsystemDeregisterGlobalEditorGizmoTypes& OnEditorGizmoSubsystemDeregisterGlobalEditorGizmoTypes() { return DeregisterGlobalEditorGizmoTypesDelegate; }

	/**
	 * Registers all built-in Editor gizmo types and broadcast registration event.
	 */
	UE_API void RegisterBuiltinEditorGizmoTypes();

	/**
	 * Removes all built-in Editor gizmo types and broadcast deregistration event.
	 */
	UE_API void DeregisterBuiltinEditorGizmoTypes();

	/**
	 * Register a new Editor gizmo type which will be global to the Editor.
	 * @param InGizmoCategory category in which to register gizmo builder
	 * @param InGizmonBuilder new Editor gizmo builder
	 * - Accessory and Primary gizmo builders must derive from UInteractiveGizmoBuilder (of from a builder derived from it)
	 *   and must implement the IEditorInteractiveConditionalGizmoBuilder and IEditorInteractiveSelectionGizmoBuilder interfaces.
	 */
	UE_API void RegisterGlobalEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder);

	/**
	* Remove an Editor gizmo type from the set of known Editor gizmo types
	* @param InGizmoBuilder same object pointer that was passed to RegisterEditorGizmoType()
	* @return true if gizmo type was found and deregistered
	*/
	UE_API void DeregisterGlobalEditorGizmoType(EEditorGizmoCategory InGizmoCategory, UInteractiveGizmoBuilder* InGizmoBuilder);

	/**
	 * Get all qualified Editor gizmo builders for the specified category, based on the current state. Qualification is determined by the gizmo builder
	 * returning true from SatisfiesCondition() and relative priority. All qualified builders at the highest found priority
	 * will be returned.
	 * @param InGizmoCategory category in which to search for qualified builders
	 * @param InToolBuilderState current selection and other state
	 * @return array of qualified Gizmo selection builders based on current state
	 */
	UE_API void GetQualifiedGlobalEditorGizmoBuilders(EEditorGizmoCategory InGizmoCategory, const FToolBuilderState& InToolBuilderState, TArray<UInteractiveGizmoBuilder*>& InFoundBuilders);

	/**
	 * Register a transform gizmo builder used to build the Level Editor TRS gizmo, allowing the replacement of the default TRS gizmo builder.
	 */
 	UE_API void RegisterTransformGizmoBuilder(UInteractiveGizmoBuilder* InTransformGizmoBuilder);
 	
	/** 
	 * Get transform gizmo builder used to build the Level Editor TRS gizmo. 
	 */
	UInteractiveGizmoBuilder* GetTransformGizmoBuilder()
	{
		return TransformGizmoBuilder;
	}

private:

	/** TRS gizmo builder */
	UPROPERTY()
	TObjectPtr<UInteractiveGizmoBuilder> TransformGizmoBuilder;

	/** Actual registry */
	UPROPERTY()
	TObjectPtr<UEditorInteractiveGizmoRegistry> Registry;

	/** Call to register gizmo types */
	FOnEditorGizmoSubsystemRegisterGlobalEditorGizmoTypes RegisterGlobalEditorGizmoTypesDelegate;

	/** Call to deregister gizmo types */
	FOnEditorGizmoSubsystemDeregisterGlobalEditorGizmoTypes DeregisterGlobalEditorGizmoTypesDelegate;

};

#undef UE_API
