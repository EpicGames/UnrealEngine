// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class SDockTab;

class FEditorInteractiveToolsFrameworkGlobals
{
public:
	// This is the key returned by AddComponentTargetFactory() for the FStaticMeshComponentTargetFactory created/registered
	// in StartupModule() below. Use this key to find/remove that module registration if you need to.
	static UE_API int32 RegisteredStaticMeshTargetFactoryKey;
};

class FEditorInteractiveToolsFrameworkModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:

	// --- TRS Gizmo CVar management (from EditorTRSGizmoModule) ---

	static void OnUsesNewTRSGizmosChanged(bool bInUsesNewTRSGizmos);
	void OnPostEngineInit();

	FDelegateHandle TRSGizmoChangeDelegate;

	// --- Settings UI (from EditorTRSGizmoSettingsModule) ---

	void RegisterConsoleCommands();
	void UnregisterConsoleCommands();
	void RegisterCustomizations();
	void UnregisterCustomizations();
	void RegisterTabs();
	void UnregisterTabs();

	void ExecuteShowSettings(const TArray<FString>& InArgs);
	void ExecuteShowGizmoTree(const TArray<FString>& InArgs);

	static const FLazyName GizmoSettingsTabId;
	static const FLazyName GizmoTreeTabId;

	TArray<IConsoleObject*> ConsoleCommands;
	TWeakPtr<SDockTab> WeakGizmoSettingsTab;
	TWeakPtr<SDockTab> WeakGizmoTreeTab;
};

#undef UE_API
