// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorGizmos/TransformGizmoInterfaces.h"
#include "UObject/Interface.h"

#include "GizmoEdModeInterface.generated.h"

/**
 * Gizmo state structure used to pass gizmo data if needed
 */

struct FGizmoState
{
	EGizmoTransformMode TransformMode = EGizmoTransformMode::None;
};

/**
 * Interface for the new editor TRS gizmos
 */

UINTERFACE(NotBlueprintable, MinimalAPI)
class UGizmoEdModeInterface : public UInterface
{
	GENERATED_BODY()
};

class IGizmoEdModeInterface
{
	GENERATED_BODY()

public:
	
	/** Called when a transform operation begins. */
	virtual bool BeginTransform(const FGizmoState& InState) = 0;
	
	/** Called when a transform operation ends. */
	virtual bool EndTransform(const FGizmoState& InState) = 0;

	/**
	 * Called after a transform operation is canceled.
	 */
	virtual bool OnTransformCanceled(const FGizmoState& InState) { return false; }
};
