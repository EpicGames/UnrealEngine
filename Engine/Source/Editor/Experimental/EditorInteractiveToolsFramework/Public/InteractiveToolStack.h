// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "HAL/Platform.h"
#include "InputBehaviorSet.h"
#include "InteractiveTool.h"
#include "InteractiveToolActionSet.h"
#include "Templates/SharedPointer.h"
#include "ToolContextInterfaces.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "InteractiveToolStack.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class FText;
class FUICommandInfo;
class UEdMode;
class UEditorInteractiveToolsContext;
class UInteractiveToolBuilder;
struct FSlateIcon;

/**
 * FInteractiveToolStack represents a bundle of tools that may or may not share the same input chord
 * Should some tools share a chord the most recently used tool will be activated by the chord.
 * If no tools have been activated yet, the chord will go by order added to the stack.
 * 
 * To use tool stacks, simply add a UToolStackContext to your relevant interactive tools context.
 * This can be done by calling 'AddContextObject' to the context object store of your tool context.
 * See: 'UWidgetEditorToolPaletteMode::Enter' as reference.
 */
USTRUCT()
struct FInteractiveToolStack
{
	GENERATED_BODY()

public:

	/** 
	 * Helper method to generate tool stacks
	 * 
	 * @param EdMode				EdMode registering this tool stack
	 * @param UseToolsContext		Tool context for this stack 
	 * @param UICommand				Command to map tool start / stop actions to
	 * @param ToolStackIdentifier	Unique string identifier for the tool stack
	 */
	static UE_API void RegisterToolStack(UEdMode* EdMode, UEditorInteractiveToolsContext* UseToolsContext, TSharedPtr<FUICommandInfo> UICommand, FString ToolStackIdentifier);

	/**
	 * Helper method to add tools to stacks
	 *
	 * @param EdMode				EdMode registering tool to this tool stack
	 * @param UseToolsContext		Tool context for this stack
	 * @param UICommand				Command to map tool start / stop actions to
	 * @param ToolStackIdentifier	Unique string identifier for the tool stack
	 * @param ToolIdentifier		Unique string identifier for the tool, used to check if tool is active
	 * @param Builder				Builder for tool to be used by actions
	 */
	static UE_API void AddToolToStack(UEdMode* EdMode, UEditorInteractiveToolsContext* UseToolsContext, TSharedPtr<FUICommandInfo> UICommand, FString ToolStackIdentifier, FString ToolIdentifier, UInteractiveToolBuilder* Builder);

public:

	/** Add the tool to our internal tool array */
	UE_API int32 AddTool(const FString& InToolName, const TSharedPtr<FUICommandInfo>& InUICommand);

	/** Signal that a tool was activated increasing its priority in the toolstack for future chord inputs */
	UE_API void NotifyToolActivated(const FString& InToolName);

	/** Get Tool that should be activated for the given input chord */
	UE_API const FString GetLastActiveToolByChord(const FInputChord& InInputChord) const;

	/** Get most recently activated tool within this stack */
	UE_API const FString GetLastActiveTool() const;

	/** Get most recently activated tool's icon within this stack */
	UE_API const FSlateIcon& GetLastActiveToolIcon() const;

	/** Get most recently activated tool's label within this stack */
	UE_API const FText& GetLastActiveToolLabel() const;

	/** Get most recently activated tool's description within this stack */
	UE_API const FText& GetLastActiveToolDescription() const;

	/** Get the original tool stack */
	UE_API const TArray<TPair<FString, TSharedPtr<FUICommandInfo>>>& GetStack() const;

protected:

	/** Tools contained within this stack along with their input chord, sorted by most recently used */
	TArray<TPair<FString, TSharedPtr<FUICommandInfo>>> SortedToolStack;

	/** Original stack ordering */
	TArray<TPair<FString, TSharedPtr<FUICommandInfo>>> OriginalStack;
};

#undef UE_API
