// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "TickableEditorObject.h"

#include "EditorSnappingManager.generated.h"

class UInteractiveToolsContext;
struct FSceneSnapQueryResult;
struct FSceneSnapQueryRequest;
struct FSceneHitQueryResult;
struct FSceneHitQueryRequest;
class USceneSnappingManager;
class UInteractiveGizmoManager;
class IToolsContextQueriesAPI;

class UEditorSceneSnappingManager;

namespace UE::Editor::Gizmos
{
	EDITORINTERACTIVETOOLSFRAMEWORK_API bool RegisterSceneSnappingManager(UInteractiveToolsContext* InToolsContext);

	EDITORINTERACTIVETOOLSFRAMEWORK_API bool DeregisterSceneSnappingManager(const UInteractiveToolsContext* InToolsContext);

	EDITORINTERACTIVETOOLSFRAMEWORK_API UEditorSceneSnappingManager* FindSceneSnappingManager(const UInteractiveToolManager* InToolManager);

	namespace Private
	{
		class FEditorSceneSnappingManagerDebug;
	}
}

UCLASS(MinimalAPI)
class UEditorSceneSnappingManager : public USceneSnappingManager
{
	GENERATED_BODY()

public:
	UEditorSceneSnappingManager();

	void Initialize(const TObjectPtr<UInteractiveToolsContext>& InToolsContext);
	void Shutdown();

	virtual bool ExecuteSceneHitQuery(const FSceneHitQueryRequest& InRequest, FSceneHitQueryResult& OutResult) const override;
	virtual bool ExecuteSceneHitQuery(const FSceneHitQueryRequest& InRequest, TArray<FSceneHitQueryResult>& OutResults) const override;
	virtual bool ExecuteSceneSnapQuery(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const override;

	/** Register a snapping handler for a given snap target. */
	EDITORINTERACTIVETOOLSFRAMEWORK_API void RegisterSnapQueryTargetHandler(TUniquePtr<FSceneSnapQueryTargetHandler>&& InHandler);
	EDITORINTERACTIVETOOLSFRAMEWORK_API void DeregisterSnapQueryTargetHandler(const FName InHandlerName);

protected:
	bool ExecuteSceneHitQueryPrimitiveTransform(const FSceneHitQueryRequest& InRequest, TArray<FSceneHitQueryResult>& OutResults) const;

	bool ExecuteSceneSnapQueryPosition(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const;
	bool ExecuteSceneSnapQueryRotation(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const;
	bool ExecuteSceneSnapQueryRotationAngle(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const;
	bool ExecuteSceneSnapQueryScale(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const;

protected:
	TArray<TUniquePtr<FSceneSnapQueryTargetHandler>> QueryTargetHandlers;

	const IToolsContextQueriesAPI* QueriesAPI = nullptr;

	TUniquePtr<UE::Editor::Gizmos::Private::FEditorSceneSnappingManagerDebug> Debug;
};
