// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "InteractiveToolManager.h"
#include "InteractiveToolStack.h"
#include "Templates/SharedPointer.h"
#include "Tools/UEdMode.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "ToolStackContext.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class FUICommandInfo;
class UInteractiveToolBuilder;
struct FInteractiveToolStack;

/**
 * Context needed to support tool stacks, add this to your ContextObjectStore if you want to use tool stacks
 */
UCLASS(MinimalAPI, Transient)
class UToolStackContext : public UObject
{
	GENERATED_BODY()

	UE_API virtual ~UToolStackContext();

public:
	/**
	 * Registers and maps the provided UI command to actions that start / stop multiple tools within a tool stack
	 * Later on this UI command can be referenced to add the tool stack to a toolbar.
	 *
	 * @param	UICommand			Command to map tool stack to, should not have any chords due to child tools having hotkeys
	 * @param	ToolStackIdentifier	Unique string identifier for the tool stack
	 * @param	ToolScope			Scope to determine appropriate tool context for stack
	 */
	UE_API virtual void RegisterToolStack(TSharedPtr<FUICommandInfo> UICommand, const FString& ToolStackIdentifier, EToolsContextScope ToolScope = EToolsContextScope::Default);

	/**
	 * Registers and maps the provided UI command to actions that control a tool within a tool stack
	 *
	 * @param	UICommand			Command containing info such as icon, text, & chord for the tool
	 * @param	ToolStackIdentifier	Unique string identifier for the tool stack to add this tool to
	 * @param	ToolIdentifier		Unique string identifier for the tool, used to check if tool is active
	 * @param	Builder				Builder for tool to be used by actions
	 * @param	ToolScope			Scope to determine lifetime of tool (Editor, Mode, etc)
	 */
	UE_API virtual void AddToolToStack(TSharedPtr<FUICommandInfo> UICommand, const FString& ToolStackIdentifier, const FString& ToolIdentifier, UInteractiveToolBuilder* Builder, EToolsContextScope ToolScope = EToolsContextScope::Default);

	/**
	 * Unregister a Tool stack, unregistration of individual tools is handled seperately
	 * Note: Currently this is never called since we properly free resources on destruction
	 * @param Identifier string used to identify this stack
	 */
	UE_API virtual void UnregisterToolStack(const FString& Identifier);

	/**
	 * Access an existing tool stack
	 * @param	Identifier string used to identify this stack
	 * @return	Ptr to stack if it exists, nullptr otherwise
	 */
	UE_API virtual FInteractiveToolStack* AccessToolStack(const FString& Identifier);

public:
	/** EdMode owning the tools referenced by this stack */
	UPROPERTY()
	TWeakObjectPtr<UEdMode> EdMode;

	/** Scope for this context object */
	EToolsContextScope ToolContextScope;

protected:
	/** Current set of named ToolStacks */
	UPROPERTY()
	TMap<FString, FInteractiveToolStack> ToolStacks;

	/** List of Tools this Mode has registered in the EditorToolsContext, to be unregistered on Mode shutdown */
	TArray<TPair<TSharedPtr<FUICommandInfo>, FString>> RegisteredEditorTools;

	/** List of Tools stacks this Mode has registered in the EditorToolsContext, to be unregistered on Mode shutdown */
	TArray<TPair<TSharedPtr<FUICommandInfo>, FString>> RegisteredEditorToolStacks;
};

#undef UE_API
