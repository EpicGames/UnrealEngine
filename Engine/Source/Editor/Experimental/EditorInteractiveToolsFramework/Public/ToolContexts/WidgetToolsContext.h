// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/EdModeInteractiveToolsContext.h"
#include "HAL/Platform.h"
#include "Input/Events.h"
#include "Layout/Geometry.h"
#include "Rendering/DrawElements.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "WidgetToolsContext.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class UObject;
struct FCaptureLostEvent;
struct FCharacterEvent;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;

/**
 * UWidgetToolsContext extends UModeManagerInteractiveToolsContext with methods needed for
 * tools operating on general widgets that do not have a viewport.
 */
UCLASS(MinimalAPI, Transient)
class UWidgetToolsContext : public UModeManagerInteractiveToolsContext
{
	GENERATED_BODY()

public:

	// ~Begin SWidget "Proxy" Interface
	UE_API bool OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent);
	UE_API bool OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	UE_API bool OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	UE_API bool OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UE_API bool OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UE_API bool OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UE_API bool OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UE_API bool OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);
	UE_API bool OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UE_API void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent);
	UE_API int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;
	// ~End SWidget "Proxy" Interface
};

#undef UE_API
