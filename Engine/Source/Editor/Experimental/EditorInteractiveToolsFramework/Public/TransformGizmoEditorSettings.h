// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorGizmos/EditorTRSGizmo.h"
#include "Engine/DeveloperSettings.h"
#include "EditorGizmos/TransformGizmo.h"

#include "TransformGizmoEditorSettings.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

UCLASS(MinimalAPI, BlueprintType, Config = EditorPerProjectUserSettings, meta = (DisplayName = "Gizmos"))
class UTransformGizmoEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UTransformGizmoEditorSettings(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostReloadConfig(class FProperty* PropertyThatWasLoaded) override;
#endif

	UPROPERTY(Config, EditAnywhere, Category = "Transform Gizmo", DisplayName = "Transform Gizmo Size", meta = (ClampMin = "-10.0", ClampMax = "150.0"))
	float TransformGizmoSize = 0.0f;

	/**
	 * Allow arcball rotation with rotate widget
	 * (updates the setting with the same name found in Level Editor Viewport Settings)
	 */
	UPROPERTY(EditAnywhere, Category = "Transform Gizmo", DisplayName = "Enable Arcball Rotate")
	bool bEnableArcballRotate;

	/**
	 * Allow screen rotation with rotate widget
	 * (updates the setting with the same name found in Level Editor Viewport Settings)
	 */
	UPROPERTY(EditAnywhere, Category = "Transform Gizmo", DisplayName = "Enable Screen Rotate")
	bool bEnableScreenRotate;

	/**
	 * If true, the Edit widget of a transform will display the axis
	 * (updates the setting with the same name found in Level Editor Viewport Settings)
	 */
	UPROPERTY(EditAnywhere, Category = "Transform Gizmo", DisplayName = "Enable Axis drawing for Transform Edit Widget")
	bool bEnableAxisDrawing;

	/**
	 * Allow translate/rotate widget
	 * (updates the setting with the same name found in Level Editor Viewport Settings)
	 */
	UPROPERTY(EditAnywhere, Category = "Transform Gizmo", DisplayName = "Enable Combined Translate/Rotate Widget")
	bool bEnableCombinedTranslateRotate;
	//~ End Transform Gizmo Category

	//~ Begin Experimental Gizmo Category
	/** If true, the new TRS gizmos will be used. UI controls set both flags together; the intermediate state is CVar-only. */
	UPROPERTY(Config, EditAnywhere, Category = "Experimental", Getter = "UsesExperimentalGizmo", Setter = "SetUseExperimentalGizmo")
	bool bUseExperimentalGizmo = true;

	UPROPERTY(Config, EditAnywhere, Category = "Experimental", Getter = "UsesNewTRSGizmo", Setter = "SetUseNewTRSGizmo")
	bool bUseEditorTRSGizmo = true;

	UPROPERTY(Config, EditAnywhere, Category = "Experimental", meta = (ShowOnlyInnerProperties))
	FGizmosParameters GizmosParameters;
	//~ End Experimental Gizmo Category

	UE_API void SetUseExperimentalGizmo(bool bInUseExperimentalGizmo);
	UE_API void SetUseNewTRSGizmo(bool bInUseNewTRSGizmo);

	UE_API bool UsesLegacyGizmo() const;
	UE_API bool UsesExperimentalGizmo() const;
	UE_API bool UsesNewTRSGizmo() const;

	UE_API void SetGizmosParameters(const FGizmosParameters& InGizmosParameters);

	/** 
	 * Updates gizmo axis size multiplier. 
	 * bInteractive set to true means the new value will not be saved to current configuration.
	 * Useful to get good performance while the value is being interactively changed, e.g. through a UI slider.
	 */
	UE_API void SetTransformGizmoAxisMultiplier(float InAxisMultiplier, bool bInteractive = false);
	UE_API float GetTransformGizmoAxisMultiplier() const;

	/** Use this for legacy gizmos */
	UE_API void SetTransformGizmoSize(float InTransformGizmoSize);

private:
	UE_API void BroadcastNewTRSGizmoChange() const;
	UE_API void BroadcastGizmosParametersChange() const;

	/** Writes to Gizmos settings from the current LevelEditorSettings. */
	UE_API void SyncFromLevelEditorSettings();

	UE_API void OnLegacySettingChanged(FName InPropertyName);
};

#undef UE_API
