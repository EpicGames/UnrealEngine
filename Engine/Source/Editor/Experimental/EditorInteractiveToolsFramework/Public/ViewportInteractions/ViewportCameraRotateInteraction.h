// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ViewportInteraction.h"

#include "ViewportCameraRotateInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UKeyInputBehavior;
struct FViewportClientNavigationHelper;

/**
 * A viewport interaction to Rotate the camera using keyboard inputs. Only works while the camera is being controlled using the mouse
 */
UCLASS(MinimalAPI, Transient)
class UViewportCameraRotateInteraction final
	: public UViewportInteraction
	, public IKeyInputBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API UViewportCameraRotateInteraction();

	//~ Begin IKeyInputBehaviorTarget
	UE_API virtual void OnKeyPressed(const FKey& InKeyID) override;
	UE_API virtual void OnKeyReleased(const FKey& InKeyID) override;
	//~ End IKeyInputBehaviorTarget

	//~ Begin UViewportInteraction
	UE_API virtual void Tick(float InDeltaTime) const override;
	//~ End UViewportInteraction

protected:
	//~ Begin UViewportInteractionBase
	UE_API virtual void OnCommandChordChanged() override;
	UE_API virtual TArray<TSharedPtr<FUICommandInfo>> GetCommands() const override;
	//~ End UViewportInteractionBase

	UE_API void UpdateKeyState(const FKey& InKeyID, bool bInIsPressed);
	UE_API TArray<FKey> GetKeys() const;

private:
	TWeakObjectPtr<UKeyInputBehavior> KeyInputBehaviorWeak;

	float RotatePitchImpulse = 0.0f;
	float RotateYawImpulse = 0.0f;
};

#undef UE_API
