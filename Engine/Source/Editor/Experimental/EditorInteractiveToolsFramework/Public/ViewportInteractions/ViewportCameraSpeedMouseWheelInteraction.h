// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportInteraction.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ViewportCameraSpeedMouseWheelInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/**
 *
 */
UCLASS(MinimalAPI)
class UViewportCameraSpeedMouseWheelInteraction
	: public UViewportInteraction
	, public IMouseWheelBehaviorTarget
{
	GENERATED_BODY()
public:

	UE_API UViewportCameraSpeedMouseWheelInteraction();

	//~ Begin IMouseWheelBehaviorTarget
	UE_API virtual FInputRayHit ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos) override;
	UE_API virtual void OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos) override;
	UE_API virtual void OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos) override;
	//~ Begin IMouseWheelBehaviorTarget

	/**
 * Increments the current camera speed using the specified change factor, e.g. Speed += Speed * UpdateFactor
 * @param InUpdateFactor positive or negative factor
 */
	UE_API void UpdateCameraSpeed(float InUpdateFactor) const;
};

#undef UE_API
