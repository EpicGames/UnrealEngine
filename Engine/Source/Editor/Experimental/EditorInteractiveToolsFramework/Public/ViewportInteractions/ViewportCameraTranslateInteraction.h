// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ViewportInteraction.h"

#include "ViewportCameraTranslateInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class FUICommandInfo;
class UKeyInputBehavior;
struct FViewportClientNavigationHelper;

/**
 * A viewport interaction used to Translate the camera using keyboard inputs while the camera is being moved using the mouse
 */
UCLASS(MinimalAPI, Transient)
class UViewportCameraTranslateInteraction final
	: public UViewportInteraction
	, public IKeyInputBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API UViewportCameraTranslateInteraction();

	//~ Begin IKeyInputBehaviorTarget
	virtual void OnKeyPressed(const FKey& InKeyID) override {}
	virtual void OnKeyReleased(const FKey& InKeyID) override {}
	UE_API virtual void OnForceEndCapture() override;
	//~ End IKeyInputBehaviorTarget

	//~ Begin UViewportInteraction
	UE_API virtual void Tick(float InDeltaTime) const override;
	//~ End UViewportInteraction

protected:
	//~ Begin UViewportInteractionBase
	UE_API virtual void OnCommandChordChanged() override;
	UE_API virtual TArray<TSharedPtr<FUICommandInfo>> GetCommands() const override;
	//~ End UViewportInteractionBase

	UE_API TArray<FKey> GetKeys() const;

private:
	TWeakObjectPtr<UKeyInputBehavior> KeyInputBehaviorWeak;

	bool bUseNumpadKey = true;
};

#undef UE_API
