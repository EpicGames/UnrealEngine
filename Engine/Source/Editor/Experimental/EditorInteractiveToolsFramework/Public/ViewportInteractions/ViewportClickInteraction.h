// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportInteraction.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Behaviors/ViewportClickBehavior.h"
#include "ViewportClickInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

namespace UE::InteractiveTools
{
enum class EViewportClickButton : uint8
{
	Left,
	Middle,
	Right
};
} 

class HHitProxy;
struct FViewportClick;

/**
 * Provides click-on-up interactions with binding-complexity-based priority compatible with FViewportClick.
 */
UCLASS(MinimalAPI, Abstract)
class UViewportClickInteraction
	: public UViewportInteraction
	, public IViewportClickBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize(UViewportInteractionsBehaviorSource* const InViewportInteractionsBehaviorSource) override;
	
	// ~Begin IViewportClickBehaviorTarget
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& InClickPos) override;
	UE_API virtual void OnClickDown(const FInputDeviceRay& InClickPos) override; 
	UE_API virtual void OnClickUp(const FInputDeviceRay& InClickPos) override;
	UE_API virtual void OnForceEndCapture() override;
	
	UE_API virtual void OnStateUpdated(const FInputDeviceState& InInputDeviceState) override;
	// ~End IViewportClickBehaviorTarget

protected:
	UE_API virtual void BuildBehaviors();

	UE_API HHitProxy* GetHitProxy(const FInputDeviceRay& InClickPos) const;

	/** Override this function to provide custom functionality */
	virtual void ProcessClick_Internal(const FViewportClick& InViewportClick, HHitProxy* InHitProxy, FSceneView& View) {}
	
	TWeakObjectPtr<UViewportClickBehavior> ClickBehaviorWeak;
	
	FKey LastChangedMouseButton = FKey();
	EInputEvent LastInputEvent = EInputEvent::IE_MAX;
};

#undef UE_API
