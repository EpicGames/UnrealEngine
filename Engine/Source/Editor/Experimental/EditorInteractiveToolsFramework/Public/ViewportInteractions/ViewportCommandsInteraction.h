// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ViewportInteraction.h"
#include "ViewportCommandsInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class FUICommandInfo;
class FUICommandList;
class UKeyInputBehavior;

/**
 * A viewport interaction class used to add support for a list of FUICommandInfo Commands to the current viewport
 */
UCLASS(MinimalAPI)
class UViewportCommandsInteraction : public UViewportInteraction, public IKeyInputBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API UViewportCommandsInteraction();

	/**
	 * Sets which commands will be handled by this Command Interaction
	 */
	UE_API void SetCommands(const TSharedPtr<FUICommandList>& InEditorCommandList, const TArray<TSharedPtr<FUICommandInfo>>& InCommands);

	// ~ Begin IKeyInputBehaviorTarget
	UE_API virtual void OnKeyPressed(const FKey& InKeyID) override;
	UE_API virtual void OnKeyReleased(const FKey& InKeyID) override;
	// ~ End IKeyInputBehaviorTarget

	/** Set to true to execute commands on Press instead of Release */
	bool bExecuteOnPress = false;

protected:
	// ~ Begin UViewportInteractionBase
	UE_API virtual void OnCommandChordChanged() override;
	UE_API virtual TArray<TSharedPtr<FUICommandInfo>> GetCommands() const override;
	// ~ End UViewportInteractionBase

private:
	UE_API TArray<FKey> GetKeys();

	UE_API void ExecuteCommand(const FKey& InKeyID);
	UE_API bool MatchesCommandKey(const TSharedPtr<FUICommandInfo>& InCommandInfo, const FKey& InKeyID) const;

	UE_API bool CanBeActivated() const;

	TSharedPtr<FUICommandList> EditorCommandList;
	TArray<TSharedPtr<FUICommandInfo>> Commands;

	TWeakObjectPtr<UKeyInputBehavior> KeyInputBehaviorWeak;
};

#undef UE_API
