// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportDragInteraction.h"

#include "ViewportDollyInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

UCLASS(MinimalAPI, Transient)
class UViewportDollyInteraction : public UViewportDragInteraction
{
	GENERATED_BODY()
	
public:
	UE_API UViewportDollyInteraction();
	
	UE_API virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) override;
	UE_API virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) override;
	UE_API virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY) override;
	UE_API virtual bool CanBeActivated() const override;
	
private:
	UE::Editor::ViewportInteractions::FButtonBindings NormalBindings;
	UE::Editor::ViewportInteractions::FButtonBindings CameraLockedBindings;
	const UE::Editor::ViewportInteractions::FButtonBindings& GetBindings() const;
};

#undef UE_API