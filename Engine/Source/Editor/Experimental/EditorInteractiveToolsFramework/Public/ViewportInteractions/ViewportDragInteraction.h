// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ViewportInteraction.h"
#include "Behaviors/ViewportClickDragBehavior.h"

#include "ViewportDragInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UClickDragInputBehavior;
class USingleClickOrDragInputBehavior;
class USingleDoubleClickOrDragBehavior;
class UViewportClickDragBehavior;

/**
 * A base class which can be used to implement interactions based on mouse drag actions
 */
UCLASS(MinimalAPI, Transient, Abstract)
class UViewportDragInteraction
	: public UViewportInteraction
	, public IViewportClickDragBehaviorTarget
{
private:
	GENERATED_BODY()
public:
	UE_API UViewportDragInteraction();
	
	UE_API virtual bool IsActive() const override { return bIsDragging; }

	//~ Begin IViewportClickDragBehaviorTarget
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& InPressPos) override;
	UE_API virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) override;
	UE_API virtual void OnDrag(const FDragArgs& InDrag) override;
	UE_API virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) override;
	virtual void OnStateUpdated(const FInputDeviceState& InInputDeviceState) override {}
	UE_API virtual void OnEndCapture(EEndCaptureReason InReason) override;
	//~ End IViewportClickDragBehaviorTarget

	virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
	{
	}

	virtual bool CanBeActivated() const
	{
		return IsEnabled();
	}

protected:

	TWeakObjectPtr<UViewportClickDragBehavior> ViewportClickDragBehavior;

	bool bIsDragging;
};

#undef UE_API
