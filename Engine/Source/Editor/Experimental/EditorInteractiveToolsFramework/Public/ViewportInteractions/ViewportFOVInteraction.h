// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ViewportInteraction.h"

#include "ViewportFOVInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UKeyInputBehavior;

/**
 * A viewport interaction used to change the camera Field of View using either keyboard inputs or the mouse wheel
 */
UCLASS(MinimalAPI, Transient)
class UViewportFOVInteraction
	: public UViewportInteraction
	, public IKeyInputBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API UViewportFOVInteraction();
	UE_API virtual void BeginDestroy() override;

	//~ Begin IKeyInputBehaviorTarget
	UE_API virtual void OnKeyPressed(const FKey& InKeyID) override;
	UE_API virtual void OnKeyReleased(const FKey& InKeyID) override;
	UE_API virtual void OnForceEndCapture() override;
	//~ End IKeyInputBehaviorTarget

	//~ Begin UViewportInteraction
	UE_API virtual void Tick(float InDeltaTime) const override;
	UE_API virtual void Initialize(UViewportInteractionsBehaviorSource* const InViewportInteractionsBehaviorSource) override;
	//~ End UViewportInteraction

protected:
	//~ Begin UViewportInteractionBase
	UE_API virtual void OnCommandChordChanged() override;
	UE_API virtual TArray<TSharedPtr<FUICommandInfo>> GetCommands() const override;
	//~ End UViewportInteractionBase

	UE_API void UpdateKeyState(const FKey& InKeyID, bool bInIsPressed);
	UE_API TArray<FKey> GetKeys() const;
	UE_API TArray<FKey> GetNumpadKeys() const;

private:
	UE_API void OnMouseLookingChanged(bool bInIsMouseLooking);
	UE_API void ResetImpulse();

	bool bUseNumpadKey = true;
	float ZoomOutInImpulse = 0;

	TWeakObjectPtr<UKeyInputBehavior> KeyInputBehaviorWeak;
};

#undef UE_API
