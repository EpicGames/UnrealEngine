// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputBehavior.h"
#include "Tools/EdModeInteractiveToolsContext.h"

#include "ViewportInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class FEditorViewportClient;
class FUICommandInfo;
class FViewportCameraMover;
class FViewportClickHandler;
class FViewportCommandsHandler;
class UInputBehavior;
class UTransformProxy;
class UViewportInteractionsBehaviorSource;

namespace UE::Editor::ViewportInteractions
{
constexpr int ShiftKeyMod = 1;
constexpr int AltKeyMod = 2;
constexpr int CtrlKeyMod = 3;

constexpr int LeftMouseButtonMod = 4;
constexpr int RightMouseButtonMod = 5;
constexpr int MiddleMouseButtonMod = 6;

static constexpr int VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY = FInputCapturePriority::DEFAULT_TOOL_PRIORITY - 5;

/*
 * Binding complexity is chosen so that "tool" behaviors win out over viewport interactions on the same complexity footing.
 * Tool priority in this diagram is assumed to have a grandfathered single point of complexity.
 *
 * These statements should be true:
 *		- A tool beats an unmodified click.
 *		- A modified click can beat a tool.
 *		- An unmodified drag loses to a click until it is confirmed.
 *		- A modified, unconfirmed drag can beat an unmodified click.
 *
 * Offsetting clicks by 1/3rd and unconfirmed drags by another 1/3rd from that allows this:
 *
 *     |--   Tool
 *      |--  Click
 *   |--|--  Modified Click
 *       |-- Drag (unconfirmed)
 *    |--|-- Drag confirmed (or modified unconfirmed drag)
 * |--|--|-- Modified drag confirmed
 */
static constexpr int VIEWPORT_INTERACTION_PARTIAL_STEP = static_cast<int>(FInputCapturePriority::BINDING_COMPLEXITY_PRIORITY_STEP * (1.f/3.f));
static_assert(VIEWPORT_INTERACTION_PARTIAL_STEP >= 1);

static constexpr int CLICK_PRIORITY = FInputCapturePriority::DEFAULT_TOOL_PRIORITY + FInputCapturePriority::BINDING_COMPLEXITY_PRIORITY_STEP + VIEWPORT_INTERACTION_PARTIAL_STEP;
static constexpr int DRAG_PRIORITY = FInputCapturePriority::DEFAULT_TOOL_PRIORITY + FInputCapturePriority::BINDING_COMPLEXITY_PRIORITY_STEP + VIEWPORT_INTERACTION_PARTIAL_STEP * 2;
static constexpr int DRAG_CONFIRMED_PRIORITY = DRAG_PRIORITY - FInputCapturePriority::BINDING_COMPLEXITY_PRIORITY_STEP;

/**
 * Possible types of Viewport Interactions.
 * Use by FEditorViewportClient to check whether a specific interaction is handled by an ITF-based interaction, or if it
 * should be handled by legacy logic
 */

// Camera Movement
static FName Zoom = TEXT("Zoom");
static FName Orbit = TEXT("Orbit");
static FName PerspectiveViewAngle = TEXT("PerspectiveCameraAngle");
static FName PerspectiveDolly = TEXT("PerspectiveDolly");
static FName PerspectivePan = TEXT("PerspectivePan");
static FName PerspectiveMoveYaw = TEXT("PerspectiveMoveYaw");
static FName OrthographicPan = TEXT("OrthographicPan");
static FName CameraTranslate = TEXT("CameraTranslate");
static FName CameraRotate = TEXT("CameraRotate");
static FName FOV = TEXT("FOV");
static FName Commands = TEXT("Commands");

// Drag Tools
static FName BoxSelect = TEXT("BoxSelect");
static FName FrustumSelect = TEXT("FrustumSelect");
static FName Measure = TEXT("Measure");
static FName ViewportChange = TEXT("ViewportChange");
static FName DuplicateDrag = TEXT("DuplicateDrag");
static FName DragMovesCamera = TEXT("DragMovesCamera");

// Groups
/** Interactions that click on the viewport */
static const FName ViewportClick = TEXT("ViewportClick");
/** Interactions that move the camera via mouse motion */
static const FName CameraDrag = TEXT("CameraDrag");
/** Interactions the enable FPS camera movement */
static const FName CameraFly = TEXT("CameraFly");

} // namespace UE::Editor::ViewportInteractions

/**
 * Base class used to implement viewport interactions.
 * Has some helper functions to check if Shift, Alt, Ctrl modifier keys and Mouse Button keys are down.
 * Tick can be used for code which needs to affect things like camera movements in Editor Viewport Client
 */
UCLASS(MinimalAPI, Transient, Abstract)
class UViewportInteraction : public UObject
{
	GENERATED_BODY()

public:
	UE_API UViewportInteraction();

	UE_API bool IsShiftDown() const;
	UE_API bool IsAltDown() const;
	UE_API bool IsCtrlDown() const;

	UE_API bool IsLeftMouseButtonDown() const;
	UE_API bool IsMiddleMouseButtonDown() const;
	UE_API bool IsRightMouseButtonDown() const;

	UE_API bool IsAnyMouseButtonDown() const;

	UE_API bool IsMouseLooking() const;

	UE_API void SetEnabled(bool bInEnabled);
	UE_API bool IsEnabled() const;
	
	virtual bool IsActive() const { return false; }

	/** Called when the interaction is removed from its behavior source. Override to make sure the interaction is properly cleaned up */
	virtual void Shutdown() {}

	/** Called every frame. */
	virtual void Tick(float InDeltaTime) const {}
	
	/** Called every frame to render tools to world space */
	virtual void Render(IToolsContextRenderAPI* InRenderAPI) { }

	/** Called every frame to render tools to a 2D canvas */
	virtual void Draw(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) { }

	TArray<TObjectPtr<UInputBehavior>> GetInputBehaviors()
	{
		return InputBehaviors;
	}

	UE_API FName GetInteractionName() const;

	/**
	 * The Groups this interaction belongs to, if any
	 */
	UE_API const TArray<FName>& GetInteractionGroups() const;

	/**
	 * Initialization code which can be expanded by derived classes
	 */
	UE_API virtual void Initialize(UViewportInteractionsBehaviorSource* const InViewportInteractionsBehaviorSource);

	UE_API UViewportInteractionsBehaviorSource* GetViewportInteractionsBehaviorSource() const;

	//~ Begin UObject
	UE_API virtual void BeginDestroy() override;
	//~ End UObject

	/**
	 * Checks if the provided Mode is supported by this interaction.
	 * Used to know whether this interaction should be added or not to the current Mode.
	 * Should be called on the class CDO to avoid instantiating interactions which might not be used.
	 */
	bool IsCurrentModeSupported(const FEditorModeTools* InModeTools) const;

protected:
	UE_API FEditorModeTools* GetModeTools() const;
	UE_API FEditorViewportClient* GetEditorViewportClient() const;
	
	UE_API void SetMouseCursorOverride(EMouseCursor::Type InMouseCursor);
	UE_API void ClearMouseCursorOverride();

	/**
	 * Define what should happen to UInputBehavior(s) when an "observed" command chord changes
	 */
	virtual void OnCommandChordChanged()
	{
	}

	/**
	 * Return a list of commands used by a key-based UInputBehavior.
	 * Used for its initialization, or its refresh, whenever a command chord changes
	 */
	virtual TArray<TSharedPtr<FUICommandInfo>> GetCommands() const
	{
		return {};
	}

	UE_API void RegisterInputBehavior(UInputBehavior* InBehavior);
	UE_API void RegisterInputBehaviors(TArray<UInputBehavior*> InBehaviors);

	virtual TArray<FEditorModeID> GetUnsupportedModes() const { return {}; }

	UPROPERTY()
	TWeakObjectPtr<UViewportInteractionsBehaviorSource> ViewportInteractionsBehaviorSource;

	bool bEnabled = true;

	/**
	 * Name for this interaction, used by Viewport Interactions Behavior Source to identify it.
	 * If NAME_None, static class name will be used.
	 */
	FName InteractionName = NAME_None;

	/**
	 * The groups this interaction belongs to. Interactions can belong to many groups.
	 */
	TArray<FName> Groups;

private:
	UE_API void OnUserDefinedChordChanged(const FUICommandInfo& InCommandInfo);

	UPROPERTY()
	TArray<TObjectPtr<UInputBehavior>> InputBehaviors;

	FDelegateHandle OnChordChangedDelegateHandle;
};

#undef UE_API
