// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputCoreTypes.h"
#include "InputState.h"
#include "Framework/Commands/UICommandInfo.h"

namespace UE::Editor::ViewportInteractions
{

enum class EInputStage
{
	/** Where the behavior has not yet started. */
	Start,
	/** Where the behavior has started and needs to validate that it can continue. */
	Continue
};

struct FButtonBinding
{
	FButtonBinding(const FKey& InButton)
		: Chord(InButton)
		, Command(nullptr)
	{}
	
	FButtonBinding(const TSharedPtr<FUICommandInfo>& InCommand)
		: Chord()
		, Command(InCommand)
	{}
	
	FButtonBinding(const FButtonBinding& InBinding) = default;

	FInputChord Chord;
	TWeakPtr<FUICommandInfo> Command;
	
	// TODO: These could be a flag-based enum, or just bitwise values
	/** This button must be down for the behavior begin capture. */
	bool bRequiredToStart = true;
	/** If any binding is set to true, the behavior can capture when this button is pressed and all required buttons are active. */
	bool bTriggersStart = false;
	/** This button must be down for the behavior to continue capture. Should be a subset of bRequiredToStart. */
	bool bRequiredToContinue = true;
	
	FButtonBinding& TriggersStart(bool bTriggers = true)
	{
		bTriggersStart = bTriggers;
		return *this;
	}
	
	FButtonBinding& Required(bool bRequired = true)
	{
		bRequiredToContinue = bRequired;
		bRequiredToStart = bRequired;
		return *this;
	}
	
	FButtonBinding& RequiredToStart(bool bRequired = true)
	{
		bRequiredToStart = bRequired;
		return *this;
	}
	
	FButtonBinding& RequiredToContinue(bool bRequired = true)
	{
		bRequiredToContinue = bRequired;
		return *this;
	}
};


struct FButtonBindings
{
	FButtonBindings() = default;
	FButtonBindings(const FButtonBindings& OtherBindings) = default;
	FButtonBindings(const TArray<FButtonBinding>& InBindings);

	TArray<FButtonBinding> Bindings;
	bool bRequiresTrigger = false;
	
	int32 GetBindingComplexity(EInputStage Stage, const FInputDeviceState& InputDeviceState) const;
	bool DidAnyBindingStateChange(const FInputDeviceState& InputDeviceState) const;
	
protected:
	static void ProcessBinding(EInputStage Stage, const FInputDeviceState& InputDeviceState, const FButtonBinding& Binding, int32& OutComplexity, bool& bOutTrigger);
	static void ProcessBindingChord(EInputStage Stage, const FInputDeviceState& InputDeviceState, const FButtonBinding& Binding, const FInputChord& Chord, int32& OutComplexity, bool& bOutTrigger);
};

}
