// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "InputBehaviorSet.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ViewportInteraction.h"

#include "ViewportInteractionsBehaviorSource.generated.h"

class FCameraControllerUserImpulseData;
class FEditorCameraController;
class FEditorViewportClient;
class FUICommandInfo;
class FViewportCameraMover;
class FViewportClickHandler;
class UEditorInteractiveToolsContext;
class UEditorTransformGizmo;
class UInputBehaviorSet;
class UTransformGizmo;
class UTransformProxy;
class UViewportInteraction;

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

namespace UE::Editor::ViewportInteractions
{

DECLARE_MULTICAST_DELEGATE(FOnEditorViewportInteractionsToggleDelegate)

/**
 * Returns true if ITF-based input tools should be used.
 */
UE_API bool UseEditorViewportInteractions();

/**
 * Helps hiding unwanted logs. For testing.
 */
UE_API bool IsVerbose();

/**
 * Toggles ITF Viewport Interactions On/Off
 */
UE_API void ToggleEditorViewportInteractions(bool bInEnable);

UE_API FOnEditorViewportInteractionsToggleDelegate& OnEditorViewportInteractionsActivated();
UE_API FOnEditorViewportInteractionsToggleDelegate& OnEditorViewportInteractionsDeactivated();

/**
 * Checks whether the specified Command should be triggered by the specified Key
 */
bool CommandMatchesKey(const TSharedPtr<FUICommandInfo>& InCommandInfo, const FKey& InKeyID);

} // namespace UE::Editor::ViewportInteractions

DECLARE_LOG_CATEGORY_EXTERN(LogITFViewportInteractions, Log, All);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnMouseLookingChanged, bool /* bIsMouseLooking*/)

DECLARE_MULTICAST_DELEGATE_OneParam(FOnModifierKeyChanged, bool /* bIsDown*/)

USTRUCT()
struct FViewportInteractionGroup
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<TObjectPtr<UViewportInteraction>> Interactions;
};

/**
 * This class hosts a list of ITF Viewport Interactions and a UBehaviorSet containing the UInteractionBehavior
 * instances required by those Viewport Interactions.
 * It also acts as a behavior target, registering the status of Shift, Alt, Ctrl modifier keys. That and other
 * pieces of information can be accessed externally, so that e.g. Viewport Interactions can know about modifier
 * states without having to implement themselves what would be duplicate logic to handle those inputs. This class
 * can also be used to know whether the Viewport camera is being moved using the mouse,or to mark it as such.
 */
UCLASS(Transient, MinimalAPI)
class UViewportInteractionsBehaviorSource final
	: public UObject
	, public IInputBehaviorSource
	, public IModifierToggleBehaviorTarget
{
	GENERATED_BODY()

public:
	//~ Begin IInputBehaviorSource
	virtual const UInputBehaviorSet* GetInputBehaviors() const override
	{
		return BehaviorSet;
	}
	//~ End IClickDragBehaviorTarget

	//~ Begin IModifierToggleBehaviorTarget
	virtual void OnUpdateModifierState(int InModifierID, bool bInIsOn) override;
	virtual void OnForceEndCapture() override;
	//~ End IModifierToggleBehaviorTarget

	/**
	 * Will initialize the required behaviors and the Behavior Set
	 */
	UE_API void Initialize(UEditorInteractiveToolsContext* InInteractiveToolsContext);

	/**
	 * Register this Input Behavior Source to the InputRouter
	 */
	UE_API void RegisterBehaviorSources();

	/**
	 * Deregister this Input Behavior Source from the InputRouter
	 */
	UE_API void DeregisterBehaviorSources();

	/**
	 * Call Tick on those Viewport Interactions which require it (e.g. Camera Mover)
	 */
	UE_API void Tick(float InDeltaTime);

	/**
	 * Renders the active tool on the specified View/World
	 */
	UE_API void RenderTools(IToolsContextRenderAPI* InRenderAPI) const;

	/**
	 * Draws the active tool on the specified View/Canvas
	 */
	UE_API void DrawTools(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) const;

	/**
	 * Add the specified interactions to this Behavior Source
	 */
	UE_API void AddInteractions(const TArray<const UClass*>& InInteractions, bool bInReregister = false);

	/**
	 * Add the specified interaction to this Behavior Source
	 */
	UE_API UViewportInteraction* AddInteraction(const UClass* InInteractionClass, bool bInReregister = false);

	/**
	 * Finds an interaction with a given class if one exists
	 */
	UE_API UViewportInteraction* FindInteraction(const UClass* InInteractionClass) const;

	/**
	 * Finds an interaction with a given name if one exists
	 */
	UE_API UViewportInteraction* FindInteraction(const FName& InName) const;

	/**
	 * Returns true if any interaction in the given group is enabled. An enabled interaction can potentially take action.
	 */
	UE_API bool IsAnyInteractionInGroupEnabled(const FName& InName) const;
	
	/**
	 * Returns true if any interaction in the given group is active. An active interaction is currently taking action.
	 */
	UE_API bool IsAnyInteractionInGroupActive(const FName& InName) const;

	/**
	 * Removes the specified interaction from the active ones.
	 */
	UE_API void RemoveInteraction(const FName InInteractionName, bool bInReregister = false);

	/**
	 * De-registers behaviors, removes viewport interactions.
	 * Behaviors added by Initialize will be kept around.
	 */
	UE_API void Reset();

	/**
	 * Returns the first Viewport Interaction of type T
	 */
	template<class T>
	T* GetTypedViewportInteraction() const
	{
		T* Ret = nullptr;
		for (const TPair<FName, TObjectPtr<UViewportInteraction>>& Pair : ViewportInteractions)
		{
			if (UViewportInteraction* Interaction = Pair.Value)
			{
				if (Interaction->IsA<T>())
				{
					Ret = Cast<T>(Interaction);
				}
			}
		}

		return Ret;
	}

	/**
	 * Returns all Viewport Interactions of type T
	 */
	template<class T>
	TArray<T*> GetTypedViewportInteractions() const
	{
		TArray<T*> Return;

		for (const TPair<FName, TObjectPtr<UViewportInteraction>>& Pair : ViewportInteractions)
		{
			if (UViewportInteraction* Interaction = Pair.Value)
			{
				if (Interaction->IsA<T>())
				{
					Return.AddUnique(Cast<T>(Interaction));
				}
			}
		}

		return Return;
	}

	/**
	 * Sets current mouse cursor override
	 */
	UE_API void SetMouseCursorOverride(EMouseCursor::Type InMouseCursor);

	UE_API void ClearMouseCursorOverride();

	/**
	 * Use the templated version in case there is some class specific initialization required by the newly created interaction,
	 * since this method will return the newly instantiated interaction
	 */
	template<class T>
	T* AddInteraction(bool bInReregister = false)
	{
		return static_cast<T*>(AddInteraction(T::StaticClass(), bInReregister));
	}

	bool IsShiftDown() const
	{
		return bIsShiftDown;
	}

	bool IsAltDown() const
	{
		return bIsAltDown;
	}

	bool IsCtrlDown() const
	{
		return bIsCtrlDown;
	}

	bool IsLeftMouseButtonDown() const
	{
		return bIsLeftMouseButtonDown;
	}

	bool IsMiddleMouseButtonDown() const
	{
		return bIsMiddleMouseButtonDown;
	}

	bool IsRightMouseButtonDown() const
	{
		return bIsRightMouseButtonDown;
	}

	UE_API bool IsMouseLooking() const;

	UE_API void SetIsMouseLooking(bool bInIsLooking);

	bool HasCameraMoved() const
	{
		return bCameraHasMoved;
	}

	void SetCameraHasMoved(bool bInHasMoved);

	/** Whether the Editor Transform Gizmo is being interacted with */
	bool IsGizmoDragging() const
	{
		return bGizmoDragging;
	}

	/** If available, return the current Editor Transform Gizmo */
	UEditorTransformGizmo* GetEditorTransformGizmo() const
	{
		return TransformGizmo;
	}

	FEditorModeTools* GetModeTools() const;

	UE_API FEditorViewportClient* GetEditorViewportClient() const;

	UE_API FEditorViewportClient* GetHoveredEditorViewportClient() const;

	FOnMouseLookingChanged& OnMouseLookingStateChanged()
	{
		return OnMouseLookingChangedDelegate;
	}

	FOnModifierKeyChanged& OnAltKeyStateChanged()
	{
		return OnAltKeyStateChangedDelegate;
	}

	FOnModifierKeyChanged& OnCtrlKeyStateChanged()
	{
		return OnCtrlKeyStateChangedDelegate;
	}

	FOnModifierKeyChanged& OnShiftKeyStateChanged()
	{
		return OnShiftKeyStateChangedDelegate;
	}

	UE_API void SetUnsupportedViewportInteractions(const TArray<FName>& InUnsupportedInteractions);

	const TArray<FName>& GetUnsupportedViewportInteractions() const
	{
		return UnsupportedInteractions;
	}

private:
	/**
	 * All Input Behaviors from registered Viewport Interactions should be added to this Behavior Set (done with ::AddInteraction() )
	 */
	UPROPERTY(Transient)
	TObjectPtr<UInputBehaviorSet> BehaviorSet;
	
	/** Behaviors that are guaranteed to always be present */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UInputBehavior>> CoreBehaviors;

	/**
	 * A collection of ITF-based Viewport Interactions
	 * Key is the interaction name (class name when no Interaction name is specified)
	 */
	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UViewportInteraction>> ViewportInteractions;

	/**
	 * Groups <-> Interactions is a many-to-many relationship. This provides easy access to a list of interactions by group.
	 */
	UPROPERTY(Transient)
	TMap<FName, FViewportInteractionGroup> ViewportInteractionGroups;
	
	void SetAltKeyState(bool bInDown);
	void SetCtrlKeyState(bool bInDown);
	void SetShiftKeyState(bool bInDown);

	void RegisterProxyDelegates();
	void DeregisterProxyDelegates();

	void OnGizmoCreatedDelegate(UTransformGizmo* InTransformGizmo);
	void OnGizmoMovementBegin(UTransformProxy* InTransformProxy);
	void OnGizmoMovementEnd(UTransformProxy* InTransformProxy);
	
	UToolsContextCursorAPI* GetCursorAPI();
	
	void CacheCursorPosition();
	void RestoreCursorPosition();
	
	bool bIsShiftDown;
	bool bIsCtrlDown;
	bool bIsAltDown;

	bool bIsLeftMouseButtonDown;
	bool bIsMiddleMouseButtonDown;
	bool bIsRightMouseButtonDown;

	TOptional<EMouseCursor::Type> CursorOverride;
	TOptional<EMouseCursor::Type> LastCursorOverride;

	/**
	 * Caches cursor position whenever we hide it, so we can place it back where it was as soon as we show it again
	 */
	TOptional<FIntPoint> HiddenCursorPosition;

	UPROPERTY(Transient)
	TWeakObjectPtr<UEditorInteractiveToolsContext> EditorInteractiveToolsContextWeak;
	
	UPROPERTY(Transient)
	TWeakObjectPtr<UToolsContextCursorAPI> WeakCursorAPI = nullptr;

	FOnMouseLookingChanged OnMouseLookingChangedDelegate;

	FOnModifierKeyChanged OnAltKeyStateChangedDelegate;
	FOnModifierKeyChanged OnCtrlKeyStateChangedDelegate;
	FOnModifierKeyChanged OnShiftKeyStateChangedDelegate;

	UPROPERTY(Transient)
	TObjectPtr<UEditorTransformGizmo> TransformGizmo;

	bool bGizmoDragging = false;
	bool bCameraHasMoved = false;

	TArray<FName> UnsupportedInteractions;
};

#undef UE_API
