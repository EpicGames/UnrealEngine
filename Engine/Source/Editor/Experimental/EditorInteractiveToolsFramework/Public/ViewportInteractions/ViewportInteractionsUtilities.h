// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class UViewportInteractionsBehaviorSource;
class FEditorModeTools;

namespace UE::Editor::ViewportInteractions
{
/**
 * Convenience function to add "standard" movement interactions to the specified Viewport Interactions Behavior Source
 * e.g. Movement, Rotation, View Angle, Orbit, Panning, FOV, etc
 */
EDITORINTERACTIVETOOLSFRAMEWORK_API void AddDefaultCameraMovementInteractions(UViewportInteractionsBehaviorSource* InInteractionsBehaviorSource);

EDITORINTERACTIVETOOLSFRAMEWORK_API void AddDefaultDragToolsInteractions(UViewportInteractionsBehaviorSource* InInteractionsBehaviorSource);

}
