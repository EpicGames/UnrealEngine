// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ViewportClickInteraction.h"
#include "UObject/Object.h"
#include "ViewportLegacyClickInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/**
 * An interaction that passes through ITF clicks through to the legacy viewport click handlers.
 */
UCLASS(MinimalAPI)
class UViewportLegacyClickInteraction : public UViewportClickInteraction, public IHoverBehaviorTarget
{
	GENERATED_BODY()
	
public:
	UViewportLegacyClickInteraction();
	
	UE_API virtual void OnClickDown(const FInputDeviceRay& InClickPos) override;

	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

protected:

	virtual void BuildBehaviors() override;
	
	virtual void ProcessClick_Internal(const FViewportClick& InViewportClick, HHitProxy* InHitProxy, FSceneView& View) override;
};

#undef UE_API
