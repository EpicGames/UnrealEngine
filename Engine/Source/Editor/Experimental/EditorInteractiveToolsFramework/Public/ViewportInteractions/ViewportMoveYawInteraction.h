// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportDragInteraction.h"

#include "ViewportMoveYawInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UViewportInteractionsBehaviorSource;

/**
 * Implements the LMB + Drag Horizontal plane move + Yaw interaction
 */
UCLASS(MinimalAPI, Transient)
class UViewportMoveYawInteraction
	: public UViewportDragInteraction
{
	GENERATED_BODY()

public:
	UE_API UViewportMoveYawInteraction();
	
	//~ Begin IViewportClickDragBehaviorTarget
	UE_API virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) override;
	UE_API virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) override;
	//~ End IViewportClickDragBehaviorTarget

	//~ Begin UViewportDragInteraction
	UE_API virtual bool CanBeActivated() const override;
	UE_API virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY) override;
	//~ End UViewportDragInteraction

protected:
	virtual TArray<FEditorModeID> GetUnsupportedModes() const override;
};

#undef UE_API
