// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportDragInteraction.h"

#include "ViewportOrbitInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/**
 * Triggers orbiting around current viewport focus
 */
UCLASS(MinimalAPI, Transient)
class UViewportOrbitInteraction
	: public UViewportDragInteraction
{
	GENERATED_BODY()

public:
	UE_API UViewportOrbitInteraction();

	//~ Begin IViewportClickDragBehaviorTarget
	UE_API virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) override;
	UE_API virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) override;
	//~ End IViewportClickDragBehaviorTarget

	//~ Begin UViewportDragInteraction
	UE_API virtual bool CanBeActivated() const override;
	UE_API virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY) override;
	//~ End UViewportDragInteraction
	
private:
	UE::Editor::ViewportInteractions::FButtonBindings NormalBindings;
	UE::Editor::ViewportInteractions::FButtonBindings CameraLockedBindings;
	const UE::Editor::ViewportInteractions::FButtonBindings& GetBindings() const; 
};

#undef UE_API
