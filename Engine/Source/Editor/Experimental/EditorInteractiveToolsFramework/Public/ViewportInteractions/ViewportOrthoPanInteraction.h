// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportDragInteraction.h"

#include "ViewportOrthoPanInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/**
 * Implements panning for orthographic views. Holding Alt or Shift inverts the panning direction.
 */
UCLASS(MinimalAPI)
class UViewportOrthoPanInteraction : public UViewportDragInteraction
{
	GENERATED_BODY()

public:
	UE_API UViewportOrthoPanInteraction();

	//~ Begin IViewportClickDragBehaviorTarget
	UE_API virtual bool CanBeActivated() const override;
	UE_API virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) override;
	UE_API virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) override;
	//~ End IViewportClickDragBehaviorTarget

	//~ Begin UViewportDragInteraction
	UE_API virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY) override;
	//~ End UViewportDragInteraction
};

#undef UE_API
