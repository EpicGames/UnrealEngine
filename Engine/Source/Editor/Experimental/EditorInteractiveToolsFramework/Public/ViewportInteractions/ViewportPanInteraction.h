// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportDragInteraction.h"

#include "ViewportPanInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

class UViewportInteractionsBehaviorSource;

/**
 * Implements the middle mouse pan interaction
 */
UCLASS(MinimalAPI, Transient)
class UViewportPanInteraction : public UViewportDragInteraction
{
	GENERATED_BODY()

public:
	UE_API UViewportPanInteraction();

	//~ Begin IViewportClickDragBehaviorTarget
	UE_API virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) override;
	UE_API virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) override;
	//~ End IViewportClickDragBehaviorTarget

	//~ Begin UViewportDragInteraction
	UE_API virtual bool CanBeActivated() const override;
	UE_API virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY) override;
	//~ End UViewportDragInteraction
	
protected:
	static void PanCamera(FEditorViewportClient& Client, float MouseDeltaX, float MouseDeltaY, bool bInvert, bool bUseWorldSpaceUp);
};

/**
 * Implements the LMB + RMB Drag world-space-up Pan interaction
 */
UCLASS(MinimalAPI, Transient)
class UViewportMovePanInteraction : public UViewportPanInteraction
{
	GENERATED_BODY()
	
public:
	UE_API UViewportMovePanInteraction();
	
protected:
	virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY) override;
};

/**
 * Handles RMB + Trackpad movement for world-space-up Panning
 */
UCLASS(MinimalAPI, Transient)
class UViewportTrackpadPanInteraction : public UViewportMovePanInteraction
{
	GENERATED_BODY()
	
public:
	UE_API UViewportTrackpadPanInteraction();
	
	UE_API virtual bool CanBeActivated() const override;
};

#undef UE_API
