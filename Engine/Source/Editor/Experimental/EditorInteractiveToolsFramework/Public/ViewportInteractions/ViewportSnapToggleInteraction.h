// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviors/ViewportQuickToggleInputBehavior.h"
#include "UnrealWidgetFwd.h"
#include "ViewportInteraction.h"

#include "ViewportSnapToggleInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/**
 * A viewport interaction that handles temporary and permanent snapping toggles
 *
 * Hold: toggles the relevant snap setting while the key is held, restoring on release.
 * Quick-press: permanently toggles the snap setting
 */
UCLASS(MinimalAPI, Transient)
class UViewportSnapToggleInteraction : public UViewportInteraction, public IQuickToggleBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API UViewportSnapToggleInteraction();

	//~ Begin UViewportInteraction
	UE_API virtual void Shutdown() override;
	//~ End UViewportInteraction

	//~ Begin IQuickToggleBehaviorTarget
	UE_API virtual void OnQuickToggle(const FKey& InKey) override;
	//~ End IQuickToggleBehaviorTarget

	//~ Begin IKeyInputBehaviorTarget
	UE_API virtual void OnKeyPressed(const FKey& InKeyID) override;
	UE_API virtual void OnKeyReleased(const FKey& InKeyID) override;
	UE_API virtual void OnForceEndCapture() override;
	//~ End IKeyInputBehaviorTarget

protected:
	//~ Begin UViewportInteractionBase
	UE_API virtual void OnCommandChordChanged() override;
	UE_API virtual TArray<TSharedPtr<FUICommandInfo>> GetCommands() const override;
	//~ End UViewportInteractionBase

	UE_API TArray<FKey> GetKeys() const;

private:
	static void SetSurfaceSnapping(bool bEnable);
	static bool CanTemporaryDisableSnapping();

	void SetTRSSnapping(bool bEnable) const;
	UE::Widget::EWidgetMode GetWidgetMode() const; 
	void CacheTRSSnapping();
	void CacheSurfaceSnapping();


	TWeakObjectPtr<UViewportQuickToggleInputBehavior> QuickToggleBehaviorWeak;

	bool bCachedTRSSnapping = false;
	bool bCachedSurfaceSnapping = false;

	bool bTRSKeyHeld = false;
	bool bSurfaceKeyHeld = false;
};

#undef UE_API
