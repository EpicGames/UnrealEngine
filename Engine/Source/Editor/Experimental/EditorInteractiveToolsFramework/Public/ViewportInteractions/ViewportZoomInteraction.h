// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/MouseWheelBehavior.h"
#include "ViewportInteraction.h"

#include "ViewportZoomInteraction.generated.h"

#define UE_API EDITORINTERACTIVETOOLSFRAMEWORK_API

/**
 * Implements a viewport interaction to change the current zoom level, using either keyboard inputs or the mouse wheel
 */
UCLASS(MinimalAPI, Transient)
class UViewportZoomInteraction
	: public UViewportInteraction
	, public IMouseWheelBehaviorTarget
	, public IKeyInputBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API UViewportZoomInteraction();

	//~ Begin IMouseWheelBehaviorTarget
	UE_API virtual FInputRayHit ShouldRespondToMouseWheel(const FInputDeviceRay& InCurrentPos) override;
	UE_API virtual void OnMouseWheelScrollUp(const FInputDeviceRay& InCurrentPos) override;
	UE_API virtual void OnMouseWheelScrollDown(const FInputDeviceRay& InCurrentPos) override;
	//~ End IMouseWheelBehaviorTarget

	//~ Begin IKeyInputBehaviorTarget
	UE_API virtual void OnKeyPressed(const FKey& InKeyID) override;
	UE_API virtual void OnKeyReleased(const FKey& InKeyID) override;
	UE_API virtual void OnForceEndCapture() override;
	//~ End IKeyInputBehaviorTarget

	//~ Begin UViewportInteractionBase
	UE_API virtual void Tick(float InDeltaTime) const override;
	//~ End UViewportInteractionBase

private:
	UE_API void OrthographicZoom(bool bInZoomIn, float InZoomMultiplier = 1.0f, bool bInForceZoomToCenter = false) const;
	UE_API void PerspectiveZoom(bool bInZoomIn) const;
	UE_API void MouseWheelZoom(bool bInZoomIn) const;

	UE_API void OnDragStart();
	UE_API void OnDragEnd();
	UE_API void OnClickDrag(const FVector2D& InDelta);

	int16 ZoomDirection = 0;

	// TODO: make them customizable
	FKey ZoomIn = EKeys::Add;
	FKey ZoomOut = EKeys::Subtract;
};

#undef UE_API
